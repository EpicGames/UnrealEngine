// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/UnsetUVsAction.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h" // FDynamicMeshUVOverlay
#include "ToolTargets/UVEditorToolMeshInput.h"

#include "Parameterization/DynamicMeshUVEditor.h"
#include "Selection/UVToolSelection.h"
#include "Selection/UVToolSelectionAPI.h"
#include "Parameterization/UVUnwrapMeshUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UnsetUVsAction)

#define LOCTEXT_NAMESPACE "UUnsetUVsAction"

namespace UnsetUVsActionLocals
{
	using namespace UE::Geometry;

	const FText TransactionName(LOCTEXT("TransactionName", "UnsetUVs"));
}

bool UUnsetUVsAction::CanExecuteAction() const
{
	return SelectionAPI->HaveSelections()
		&& SelectionAPI->GetSelectionsType() == UE::Geometry::FUVToolSelection::EType::Triangle;
}
bool UUnsetUVsAction::ExecuteAction()
{
	using namespace UnsetUVsActionLocals;
	using namespace UE::Geometry;

	EmitChangeAPI->BeginUndoTransaction(TransactionName);
	
	bool bSuccess = true;

	// We make a copy of selection and clear it right away so that we don't have an inconsistent selection
	//  state while updating the unwrap.
	const TArray<FUVToolSelection> Selections = SelectionAPI->GetSelections();
	// Don't broadcast, but do emit the change, so that it is in the correct place in the transaction.
	SelectionAPI->ClearSelections(false, true);

	// This is a copy so that we can add to it and call the setter
	TArray<FUVToolSelection> UnsetSelections = SelectionAPI->GetUnsetElementAppliedMeshSelections();

	for (const FUVToolSelection& Selection : Selections)
	{
		if (!ensure(Selection.Target.IsValid() && Selection.Target->AppliedCanonical 
			&& !Selection.SelectedIDs.IsEmpty() && Selection.Type == FUVToolSelection::EType::Triangle))
		{
			continue;
		}

		FDynamicMesh3& Mesh = *Selection.Target->AppliedCanonical;
		FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->GetUVLayer(Selection.Target->UVLayerIndex);
		if (!ensure(UVOverlay))
		{
			continue;
		}

		for (int32 Tid : Selection.SelectedIDs)
		{
			UVOverlay->UnsetTriangle(Tid);
		}

		FDynamicMeshChangeTracker ChangeTracker(Selection.Target->UnwrapCanonical.Get());
		ChangeTracker.BeginChange();
		ChangeTracker.SaveTriangles(Selection.SelectedIDs, true);

		TArray<int32> ChangedTidsArray = Selection.SelectedIDs.Array();
		Selection.Target->UpdateAllFromAppliedCanonical(UUVEditorToolMeshInput::NONE_CHANGED_ARG, &ChangedTidsArray, &ChangedTidsArray);

		checkSlow(UVUnwrapMeshUtil::DoesUnwrapMatchOverlay(*UVOverlay, *Selection.Target->UnwrapCanonical, Selection.Target->UVToVertPosition, 0.01));

		EmitChangeAPI->EmitToolIndependentUnwrapCanonicalChange(Selection.Target.Get(),
			ChangeTracker.EndChange(), TransactionName);

		if (FUVToolSelection* ExistingUnsetSelection = UnsetSelections.FindByPredicate(
			[Selection](const FUVToolSelection& ExistingSelection) { return ExistingSelection.Target == Selection.Target; }))
		{
			ExistingUnsetSelection->SelectedIDs.Append(Selection.SelectedIDs);
		}
		else
		{
			UnsetSelections.Emplace(Selection);
		}
	}//end for each target that has selections

	// Emit the selection change from empty to new unset
	SelectionAPI->SetUnsetElementAppliedMeshSelections(UnsetSelections, true, true);
	EmitChangeAPI->EndUndoTransaction();
	return bSuccess;
}

#undef LOCTEXT_NAMESPACE
