// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Actions/UVToolAction.h"

#include "UnsetUVsAction.generated.h"

//~ Action that unsets UVs, intended for testing other tools. For now, this is
//~  in the editor module (instead of UVEditorTools) so that we can keep it
//~  unexposed.
UCLASS()
class UUnsetUVsAction : public UUVToolAction
{
	GENERATED_BODY()

public:
	virtual bool CanExecuteAction() const override;
	virtual bool ExecuteAction() override;
};