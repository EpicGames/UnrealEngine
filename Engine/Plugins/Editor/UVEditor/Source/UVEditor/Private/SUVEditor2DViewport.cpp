// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUVEditor2DViewport.h"

#include "EditorViewportCommands.h"
#include "UVEditor2DViewportClient.h"
#include "UVEditorCommands.h"
#include "UVEditorStyle.h"
#include "ToolMenus.h"
#include "UVEditor2DViewportContext.h"
#include "ContextObjects/UVToolContextObjects.h" // UUVToolViewportButtonsAPI::ESelectionMode
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#define LOCTEXT_NAMESPACE "SUVEditor2DViewport"

void SUVEditor2DViewport::BindCommands()
{
	SAssetEditorViewport::BindCommands();

	const FUVEditorCommands& CommandInfos = FUVEditorCommands::Get();
	CommandList->MapAction(
		CommandInfos.VertexSelection,
		FExecuteAction::CreateLambda([this]() {
			StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->SetSelectionMode(UUVToolViewportButtonsAPI::ESelectionMode::Vertex); 
		}),
		FCanExecuteAction::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->AreSelectionButtonsEnabled(); 
		}),
		FIsActionChecked::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->GetSelectionMode() == UUVToolViewportButtonsAPI::ESelectionMode::Vertex;
		}));

	CommandList->MapAction(
		CommandInfos.EdgeSelection,
		FExecuteAction::CreateLambda([this]() {
			StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->SetSelectionMode(UUVToolViewportButtonsAPI::ESelectionMode::Edge); 
		}),
		FCanExecuteAction::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->AreSelectionButtonsEnabled(); 
		}),
		FIsActionChecked::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->GetSelectionMode() == UUVToolViewportButtonsAPI::ESelectionMode::Edge;
		}));

	CommandList->MapAction(
		CommandInfos.TriangleSelection,
		FExecuteAction::CreateLambda([this]() {
			StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->SetSelectionMode(UUVToolViewportButtonsAPI::ESelectionMode::Triangle); 
		}),
		FCanExecuteAction::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->AreSelectionButtonsEnabled(); 
		}),
		FIsActionChecked::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->GetSelectionMode() == UUVToolViewportButtonsAPI::ESelectionMode::Triangle;
		}));

	CommandList->MapAction(
		CommandInfos.IslandSelection,
		FExecuteAction::CreateLambda([this]() {
			StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->SetSelectionMode(UUVToolViewportButtonsAPI::ESelectionMode::Island); 
		}),
		FCanExecuteAction::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->AreSelectionButtonsEnabled(); 
		}),
		FIsActionChecked::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->GetSelectionMode() == UUVToolViewportButtonsAPI::ESelectionMode::Island;
		}));

	CommandList->MapAction(
		CommandInfos.FullMeshSelection,
		FExecuteAction::CreateLambda([this]() {
			StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->SetSelectionMode(UUVToolViewportButtonsAPI::ESelectionMode::Mesh); 
		}),
		FCanExecuteAction::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->AreSelectionButtonsEnabled(); 
		}),
		FIsActionChecked::CreateLambda([this]() { 
			return StaticCastSharedPtr<FUVEditor2DViewportClient>(Client)->GetSelectionMode() == UUVToolViewportButtonsAPI::ESelectionMode::Mesh;
		}));
	
	CommandList->MapAction(
		FEditorViewportCommands::Get().Snapping,
		FExecuteAction::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FUVEditor2DViewportClient> Client = StaticCastSharedPtr<FUVEditor2DViewportClient>(WeakClient.Pin()))
			{
				Client->SetSnappingActive(!Client->IsSnappingActive());
			}
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FUVEditor2DViewportClient> Client = StaticCastSharedPtr<FUVEditor2DViewportClient>(WeakClient.Pin()))
			{
				return Client->IsSnappingActive();
			}
			return false;
		}));
		
	CommandList->MapAction(
		FEditorViewportCommands::Get().LocationGridSnap,
		FExecuteAction::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FUVEditor2DViewportClient> Client = StaticCastSharedPtr<FUVEditor2DViewportClient>(WeakClient.Pin()))
			{
				Client->SetLocationGridSnapEnabled(!Client->GetLocationGridSnapEnabled());
			}
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FUVEditor2DViewportClient> Client = StaticCastSharedPtr<FUVEditor2DViewportClient>(WeakClient.Pin()))
			{
				return Client->GetLocationGridSnapEnabled();
			}
			return false;
		}));
	
	CommandList->MapAction(
		FEditorViewportCommands::Get().RotationGridSnap,
		FExecuteAction::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FUVEditor2DViewportClient> Client = StaticCastSharedPtr<FUVEditor2DViewportClient>(WeakClient.Pin()))
			{
				Client->SetRotationGridSnapEnabled(!Client->GetRotationGridSnapEnabled());
			}
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FUVEditor2DViewportClient> Client = StaticCastSharedPtr<FUVEditor2DViewportClient>(WeakClient.Pin()))
			{
				return Client->GetRotationGridSnapEnabled();
			}
			return false;
		}));
		
	CommandList->MapAction(
		FEditorViewportCommands::Get().ScaleGridSnap,
		FExecuteAction::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FUVEditor2DViewportClient> Client = StaticCastSharedPtr<FUVEditor2DViewportClient>(WeakClient.Pin()))
			{
				Client->SetScaleGridSnapEnabled(!Client->GetScaleGridSnapEnabled());
			}
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FUVEditor2DViewportClient> Client = StaticCastSharedPtr<FUVEditor2DViewportClient>(WeakClient.Pin()))
			{
				return Client->GetScaleGridSnapEnabled();
			}
			return false;
		}));
}

TSharedPtr<SWidget> SUVEditor2DViewport::BuildViewportToolbar()
{
	const FName ToolbarName = "UVEditor2DToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* Toolbar = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Toolbar->StyleName = "ViewportToolbar";
		
		{
			FToolMenuSection& LeftSection = Toolbar->AddSection("Left");
			
			{
				FToolMenuEntry& TransformEntry = LeftSection.AddSubMenu(
					"Transform",
					LOCTEXT("TransformsSubmenuLabel", "Transform"),
					LOCTEXT("TransformsSubmenuTooltip", "Viewport-related transforms tools"),
					FNewToolMenuDelegate::CreateLambda([](UToolMenu* ToolMenu)
					{
						FToolMenuSection& TransformToolsSection = ToolMenu->AddSection(
							"TransformTools",
							LOCTEXT("TransformToolsLabel", "Transform Tools")
						);

						FToolMenuEntry& SelectMode = TransformToolsSection.AddMenuEntry(FEditorViewportCommands::Get().SelectMode);
						SelectMode.SetShowInToolbarTopLevel(true);
						SelectMode.ToolBarData.StyleNameOverride = "ViewportToolbar.TransformTools";
					
						FToolMenuEntry& TranslateMode = TransformToolsSection.AddMenuEntry(FEditorViewportCommands::Get().TranslateMode);
						TranslateMode.SetShowInToolbarTopLevel(true);
						TranslateMode.ToolBarData.StyleNameOverride = "ViewportToolbar.TransformTools";
					})
				);
				TransformEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.SelectMode");
				TransformEntry.ToolBarData.LabelOverride = FText();
				TransformEntry.ToolBarData.ResizeParams.ClippingPriority = 1000;
			}
			
			LeftSection.AddEntry(UE::UnrealEd::CreateSnappingSubmenu());
			
			{
				FToolMenuEntry& ElementSelectionEntry = LeftSection.AddSubMenu(
					"ElementSelection",
					LOCTEXT("MeshElementSelectionSubmenuLabel", "Mesh Element Selection"),
					LOCTEXT("MeshElementSelectionSubmenuTooltip", "Mesh Element Selection settings in the viewport"),
					FNewToolMenuDelegate::CreateLambda([](UToolMenu* ToolMenu)
					{
						FToolMenuSection& ElementSection = ToolMenu->AddSection(
							"Element Selection",
							LOCTEXT("ElementSelectionLabel", "Element Selection")
						);
						
						FToolMenuEntryToolBarData ToolBarData;
						ToolBarData.BlockGroupName = "ElementSelection";
						ToolBarData.LabelOverride = FText::GetEmpty();
						
						FToolMenuEntry& VertexEntry = ElementSection.AddMenuEntry(FUVEditorCommands::Get().VertexSelection);
						VertexEntry.ToolBarData = ToolBarData;
						VertexEntry.TutorialHighlightName = TEXT("VertexSelection");
						VertexEntry.SetShowInToolbarTopLevel(true);
						
						FToolMenuEntry& EdgeEntry = ElementSection.AddMenuEntry(FUVEditorCommands::Get().EdgeSelection);
						EdgeEntry.ToolBarData = ToolBarData;
						EdgeEntry.TutorialHighlightName = TEXT("EdgeSelection");
						EdgeEntry.SetShowInToolbarTopLevel(true);
						
						FToolMenuEntry& TriangleEntry = ElementSection.AddMenuEntry(FUVEditorCommands::Get().TriangleSelection);
						TriangleEntry.ToolBarData = ToolBarData;
						TriangleEntry.TutorialHighlightName = TEXT("TriangleSelection");
						TriangleEntry.SetShowInToolbarTopLevel(true);
						
						FToolMenuEntry& IslandEntry = ElementSection.AddMenuEntry(FUVEditorCommands::Get().IslandSelection);
						IslandEntry.ToolBarData = ToolBarData;
						IslandEntry.TutorialHighlightName = TEXT("IslandSelection");
						IslandEntry.SetShowInToolbarTopLevel(true);
						
						FToolMenuEntry& MeshEntry = ElementSection.AddMenuEntry(FUVEditorCommands::Get().FullMeshSelection);
						MeshEntry.ToolBarData = ToolBarData;
						MeshEntry.TutorialHighlightName = TEXT("FullMeshSelection");
						MeshEntry.SetShowInToolbarTopLevel(true);
					})
				);
				
				ElementSelectionEntry.ToolBarData.ResizeParams.ClippingPriority = 950;
				ElementSelectionEntry.ToolBarData.LabelOverride = FText();
				ElementSelectionEntry.Icon = FSlateIcon(FUVEditorStyle::StyleName, "UVEditor.ElementSelection");
			}
		}
		
		{
			FToolMenuSection& RightSection = Toolbar->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;
		}
	}
	
	FToolMenuContext Context;
	{
		UUVEditor2DViewportContext* ContextObject = NewObject<UUVEditor2DViewportContext>();
		ContextObject->Viewport = SharedThis(this);
		Context.AddObject(ContextObject);
		Context.AppendCommandList(GetCommandList());
	}
	
	return UToolMenus::Get()->GenerateWidget(ToolbarName, Context);
}

bool SUVEditor2DViewport::IsWidgetModeActive(UE::Widget::EWidgetMode Mode) const
{
	return static_cast<FUVEditor2DViewportClient*>(Client.Get())->AreWidgetButtonsEnabled() 
		&& Client->GetWidgetMode() == Mode;
}

#undef LOCTEXT_NAMESPACE
