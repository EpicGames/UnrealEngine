// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUVEditor3DViewport.h"

#include "EditorViewportClient.h"
#include "UVEditorCommands.h"
#include "UVEditor3DViewportClient.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"
#include "ToolMenus.h"
#include "UVEditorStyle.h"

#define LOCTEXT_NAMESPACE "SUVEditor3DViewport"

void SUVEditor3DViewport::BindCommands()
{
	SAssetEditorViewport::BindCommands();

	const FUVEditorCommands& CommandInfos = FUVEditorCommands::Get();
	CommandList->MapAction(
		CommandInfos.EnableOrbitCamera,
		FExecuteAction::CreateLambda([this]() {
			StaticCastSharedPtr<FUVEditor3DViewportClient>(Client)->SetCameraMode(EUVEditor3DViewportClientCameraMode::Orbit);
			}),
		FCanExecuteAction::CreateLambda([this]() { return true; }),
		FIsActionChecked::CreateLambda([this]() { return StaticCastSharedPtr<FUVEditor3DViewportClient>(Client)->GetCameraMode() == EUVEditor3DViewportClientCameraMode::Orbit; }));

	CommandList->MapAction(
		CommandInfos.EnableFlyCamera,
		FExecuteAction::CreateLambda([this]() {
			StaticCastSharedPtr<FUVEditor3DViewportClient>(Client)->SetCameraMode(EUVEditor3DViewportClientCameraMode::Fly);
			}),
		FCanExecuteAction::CreateLambda([this]() { return true; }),
		FIsActionChecked::CreateLambda([this]() { return StaticCastSharedPtr<FUVEditor3DViewportClient>(Client)->GetCameraMode() == EUVEditor3DViewportClientCameraMode::Fly; }));

	CommandList->MapAction(
		CommandInfos.SetFocusCamera,
		FExecuteAction::CreateLambda([this]() {
			StaticCastSharedPtr<FUVEditor3DViewportClient>(Client)->FocusCameraOnSelection();
		}),
		FCanExecuteAction::CreateLambda([this]() { 
			return true;
		}));
}

TSharedPtr<SWidget> SUVEditor3DViewport::BuildViewportToolbar()
{
	const FName ToolbarName = "UVEditor3DViewportToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* Toolbar = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Toolbar->StyleName = "ViewportToolbar";
		
		Toolbar->AddSection("Left");
		
		FToolMenuSection& RightSection = Toolbar->AddSection("Right");
		RightSection.Alignment = EToolMenuSectionAlign::Last;
		
		{
			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu());
			
			UToolMenu* CameraMenu = UToolMenus::Get()->ExtendMenu(UToolMenus::JoinMenuPaths(ToolbarName, "Camera"));
			FToolMenuSection& MovementSection = CameraMenu->FindOrAddSection("Movement");
			
			FToolMenuEntry& OrbitEntry = MovementSection.AddMenuEntry(FUVEditorCommands::Get().EnableOrbitCamera);
			OrbitEntry.SetShowInToolbarTopLevel(true);
			OrbitEntry.Icon = FSlateIcon(FUVEditorStyle::Get().GetStyleSetName(), "UVEditor.OrbitCamera");
			OrbitEntry.ToolBarData.LabelOverride = FText();
			OrbitEntry.TutorialHighlightName = "OrbitCamera";
			
			FToolMenuEntry& FlyEntry = MovementSection.AddMenuEntry(FUVEditorCommands::Get().EnableFlyCamera);
			FlyEntry.SetShowInToolbarTopLevel(true);
			FlyEntry.Icon = FSlateIcon(FUVEditorStyle::Get().GetStyleSetName(), "UVEditor.FlyCamera");
			FlyEntry.ToolBarData.LabelOverride = FText();
			OrbitEntry.TutorialHighlightName = "FlyCamera";
			
			MovementSection.AddSeparator("FocusSeparator");
			
			FToolMenuEntry& FocusEntry = MovementSection.AddMenuEntry(FUVEditorCommands::Get().SetFocusCamera);
			FocusEntry.SetShowInToolbarTopLevel(true);
			FocusEntry.Icon = FSlateIcon(FUVEditorStyle::Get().GetStyleSetName(), "UVEditor.FocusCamera");
			FocusEntry.ToolBarData.LabelOverride = FText();
			FocusEntry.TutorialHighlightName = "FocusCamera";
		}
		
		RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
	}
	
	FToolMenuContext Context;
	
	{
		UUnrealEdViewportToolbarContext* ContextObject = UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
		Context.AddObject(ContextObject);
		Context.AppendCommandList(GetCommandList());
	}
	
	return UToolMenus::Get()->GenerateWidget(ToolbarName, Context);
}


#undef LOCTEXT_NAMESPACE
