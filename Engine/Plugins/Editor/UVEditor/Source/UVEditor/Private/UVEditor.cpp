// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditor.h"

#include "AdvancedPreviewScene.h"
#include "ToolContextInterfaces.h"
#include "UVEditorToolkit.h"
#include "UVEditorSubsystem.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"

#include "AssetEditorModeManager.h"
#include "UVEditorMode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditor)

void UUVEditor::Initialize(const TArray<TObjectPtr<UObject>>& InObjects)
{
	// Make sure we have valid targets.
	UUVEditorSubsystem* UVSubsystem = GEditor->GetEditorSubsystem<UUVEditorSubsystem>();
	check(UVSubsystem && UVSubsystem->AreObjectsValidTargets(InObjects));

	OriginalObjectsToEdit = InObjects;

	// This will do a variety of things including registration of the asset editor, creation of the toolkit
	// (via CreateToolkit()), and creation of the editor mode manager within the toolkit.
	// The asset editor toolkit will do the rest of the necessary initialization inside its PostInitAssetEditor.
	UAssetEditor::Initialize();
}
void UUVEditor::Initialize(const TArray<TObjectPtr<UObject>>& InObjects, const TArray<FTransform3d>& InTransforms)
{
	check(InTransforms.Num() == InObjects.Num());
	ObjectWorldspaceOffsets = InTransforms;
	Initialize(InObjects);
}

void UUVEditor::GetWorldspaceRelativeTransforms(TArray<FTransform3d>& OutTransforms)
{
	OutTransforms.Append(ObjectWorldspaceOffsets);
}

IAssetEditorInstance* UUVEditor::GetInstanceInterface() 
{ 
	return ToolkitInstance; 
}

void UUVEditor::GetObjectsToEdit(TArray<UObject*>& OutObjects)
{
	OutObjects.Append(OriginalObjectsToEdit);
	check(OutObjects.Num() > 0);
}

TSharedPtr<FBaseAssetToolkit> UUVEditor::CreateToolkit()
{
	TSharedPtr<FUVEditorToolkit> Toolkit = MakeShared<FUVEditorToolkit>(this);

	return Toolkit;
}

