// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditor2DViewportClient.h"

#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseWheelBehavior.h"
#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Drawing/MeshDebugDrawing.h"
#include "FrameTypes.h"
#include "UVEditorMode.h"
#include "UVEditorUXSettings.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Engine/Canvas.h"
#include "Math/Box.h"
#include "MathUtil.h"
#include "CameraController.h"
#include "SceneView.h"


namespace FUVEditor2DViewportClientLocals {

	enum class ETextAnchorPosition : uint8
	{
		None = 0,
		VerticalTop = 1 << 0,
		VerticalMiddle = 1 << 1,
		VerticalBottom = 1 << 2,
		HorizontalLeft = 1 << 3,
		HorizontalMiddle = 1 << 4,
		HorizontalRight = 1 << 5,
		TopLeftCorner = VerticalTop | HorizontalLeft,
		TopRightCorner = VerticalTop | HorizontalRight,
		BottomLeftCorner = VerticalBottom | HorizontalLeft,
		BottomRightCorner = VerticalBottom | HorizontalRight,
		TopCenter = VerticalTop | HorizontalMiddle,
		BottomCenter = VerticalBottom | HorizontalMiddle,
		LeftCenter = VerticalMiddle | HorizontalLeft,
		RightCenter = VerticalMiddle | HorizontalRight,
		Center = VerticalMiddle | HorizontalMiddle,
	};
	ENUM_CLASS_FLAGS(ETextAnchorPosition);

	struct FTextPadding
	{
		float TopPadding = 0.0;
		float BottomPadding = 0.0;
		float LeftPadding = 0.0;
		float RightPadding = 0.0;
	};

	FText ConvertToExponentialNotation(double InValue, double MinExponent = 0.0)
	{
		double Exponent = FMath::LogX(10, InValue < 0 ? -InValue : InValue);
		Exponent = FMath::Floor(Exponent);
		if (FMath::IsFinite(Exponent) && FMath::Abs(Exponent) > MinExponent)
		{
			double Divisor = FMath::Pow(10.0, Exponent);
			double Base = InValue / Divisor;
			return FText::Format(FTextFormat::FromString("{0}E{1}"), FText::AsNumber(Base), FText::AsNumber(Exponent));
		}
		else
		{
			return FText::AsNumber(InValue);
		}
	}

	FVector2D ComputeAnchoredPosition(const FVector2D& InPosition, const FText& InText, const UFont* InFont, UCanvas& Canvas, ETextAnchorPosition InAnchorPosition, const FTextPadding& InPadding, bool bClampInsideScreenBounds) {
		FVector2D OutPosition = InPosition;

		FVector2D TextSize(0, 0), TextShift(0, 0);
		Canvas.TextSize(InFont, InText.ToString(), TextSize[0], TextSize[1]);

		FBox2D TextBoundingBox(FVector2D(0, 0), TextSize);
		FBox2D ScreenBoundingBox(FVector2D(0, 0), FVector2D(Canvas.SizeX, Canvas.SizeY));

		if ((InAnchorPosition & ETextAnchorPosition::VerticalTop) != ETextAnchorPosition::None)
		{
			TextShift[1] = InPadding.TopPadding; // We are anchored to the top left hand corner by default
		}
		else if ((InAnchorPosition & ETextAnchorPosition::VerticalMiddle) != ETextAnchorPosition::None)
		{
			TextShift[1] = -TextSize[1] / 2.0;
		}
		else if ((InAnchorPosition & ETextAnchorPosition::VerticalBottom) != ETextAnchorPosition::None)
		{
			TextShift[1] = -TextSize[1] - InPadding.BottomPadding;
		}

		if ((InAnchorPosition & ETextAnchorPosition::HorizontalLeft) != ETextAnchorPosition::None)
		{
			TextShift[0] = InPadding.LeftPadding; // We are anchored to the top left hand corner by default
		}
		else if ((InAnchorPosition & ETextAnchorPosition::HorizontalMiddle) != ETextAnchorPosition::None)
		{
			TextShift[0] = -TextSize[0] / 2.0;
		}
		else if ((InAnchorPosition & ETextAnchorPosition::HorizontalRight) != ETextAnchorPosition::None)
		{
			TextShift[0] = -TextSize[0] - InPadding.RightPadding;
		}
		
		TextBoundingBox = TextBoundingBox.ShiftBy(InPosition + TextShift);
		
		if (bClampInsideScreenBounds && !ScreenBoundingBox.IsInside(TextBoundingBox)) {
			FVector2D TextCenter, TextExtents;
			TextBoundingBox.GetCenterAndExtents(TextCenter, TextExtents);
			FBox2D ScreenInsetBoundingBox(ScreenBoundingBox);
			ScreenInsetBoundingBox = ScreenInsetBoundingBox.ExpandBy(TextExtents*-1.0);
			FVector2D MovedCenter = ScreenInsetBoundingBox.GetClosestPointTo(TextCenter);
			TextBoundingBox = TextBoundingBox.MoveTo(MovedCenter);
		}

		return TextBoundingBox.Min;
	}

	FCanvasTextItem CreateTextAnchored(const FVector2D& InPosition, const FText& InText, const UFont* InFont,
		const FLinearColor& InColor, UCanvas& Canvas, ETextAnchorPosition InAnchorPosition, const FTextPadding& InPadding, bool bClampInsideScreenBounds=true)
	{
		FVector2D OutPosition = ComputeAnchoredPosition(InPosition, InText, InFont, Canvas, InAnchorPosition, InPadding, bClampInsideScreenBounds);
		return FCanvasTextItem(OutPosition, InText, InFont, InColor);
	}

	bool ConvertUVToPixel(const FVector2D& UVIn, FVector2D& PixelOut, const FSceneView& View)
	{
		FVector WorldPosition = FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition((FVector2f)UVIn);
		FVector4 ProjectedHomogenous = View.WorldToScreen(WorldPosition);
		bool bValid = View.ScreenToPixel(ProjectedHomogenous, PixelOut);
		return bValid;
	}

	void ConvertPixelToUV(const FVector2D& PixelIn, double RelDepthIn, FVector2D& UVOut, const FSceneView& View)
	{
		FVector4 ScreenPoint = View.PixelToScreen(static_cast<float>(PixelIn.X), static_cast<float>(PixelIn.Y), static_cast<float>(RelDepthIn));
		FVector4 WorldPointHomogenous = View.ViewMatrices.GetClipToWorld().TransformFVector4(ScreenPoint);
		FVector WorldPoint(WorldPointHomogenous.X / WorldPointHomogenous.W,
			               WorldPointHomogenous.Y / WorldPointHomogenous.W,
			               WorldPointHomogenous.Z / WorldPointHomogenous.W);
		UVOut = (FVector2D)FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(WorldPoint);
	}

};


FUVEditor2DViewportClient::FUVEditor2DViewportClient(FEditorModeTools* InModeTools,
	FPreviewScene* InPreviewScene, const TWeakPtr<SEditorViewport>& InEditorViewportWidget,
	UUVToolViewportButtonsAPI* ViewportButtonsAPIIn,
	UUVTool2DViewportAPI* UVTool2DViewportAPIIn)
	: FEditorViewportClient(InModeTools, InPreviewScene, InEditorViewportWidget)
	, ViewportButtonsAPI(ViewportButtonsAPIIn), UVTool2DViewportAPI(UVTool2DViewportAPIIn)
{
	ShowWidget(false);

	// Don't draw the little XYZ drawing in the corner.
	bDrawAxes = false;

	// We want our near clip plane to be quite close so that we can zoom in further.
	OverrideNearClipPlane(KINDA_SMALL_NUMBER);

	// Set up viewport manipulation behaviors:
	// Note that this is only necessary because we use a perspective projection viewport 
	// instead of a proper ortho viewport. See comment in UVEditorToolkit.cpp concerning this choice.

	FEditorCameraController* CameraControllerPtr = GetCameraController();
	CameraController->GetConfig().MovementAccelerationRate = 0.0;
	CameraController->GetConfig().RotationAccelerationRate = 0.0;
	CameraController->GetConfig().FOVAccelerationRate = 0.0;

	BehaviorSet = NewObject<UInputBehaviorSet>();

	// We'll have the priority of our viewport manipulation behaviors be lower (i.e. higher
	// numerically) than both the gizmo default and the tool default.
	static constexpr int DEFAULT_VIEWPORT_BEHAVIOR_PRIORITY = 150;

	ScrollBehaviorTarget = MakeUnique<FEditor2DScrollBehaviorTarget>(this);
	UClickDragInputBehavior* ScrollBehavior = NewObject<UClickDragInputBehavior>();
	ScrollBehavior->Initialize(ScrollBehaviorTarget.Get());
	ScrollBehavior->SetDefaultPriority(DEFAULT_VIEWPORT_BEHAVIOR_PRIORITY);
	ScrollBehavior->SetUseRightMouseButton();
	BehaviorSet->Add(ScrollBehavior);

	ZoomBehaviorTarget = MakeUnique<FEditor2DMouseWheelZoomBehaviorTarget>(this);
	ZoomBehaviorTarget->SetCameraFarPlaneWorldZ(FUVEditorUXSettings::CameraFarPlaneWorldZ);
	ZoomBehaviorTarget->SetCameraNearPlaneProportionZ(FUVEditorUXSettings::CameraNearPlaneProportionZ);
	ZoomBehaviorTarget->SetZoomLimits(0.001, 100000);
	UMouseWheelInputBehavior* ZoomBehavior = NewObject<UMouseWheelInputBehavior>();
	ZoomBehavior->Initialize(ZoomBehaviorTarget.Get());
	ZoomBehavior->SetDefaultPriority(DEFAULT_VIEWPORT_BEHAVIOR_PRIORITY);	
	BehaviorSet->Add(ZoomBehavior);

	ModeTools->GetInteractiveToolsContext()->InputRouter->RegisterSource(this);

	static const FName CanvasName(TEXT("UVEditor2DCanvas"));
	CanvasObject = NewObject<UCanvas>(GetTransientPackage(), CanvasName);

	UVTool2DViewportAPI->OnDrawGridChange.AddLambda(
		[this](bool bDrawGridIn) {
			bDrawGrid = bDrawGridIn;
		});

	UVTool2DViewportAPI->OnDrawRulersChange.AddLambda(
		[this](bool bDrawRulersIn) {
			bDrawGridRulers = bDrawRulersIn;
		});

}


const UInputBehaviorSet* FUVEditor2DViewportClient::GetInputBehaviors() const
{
	return BehaviorSet;
}

// Collects UObjects that we don't want the garbage collecter to throw away under us
void FUVEditor2DViewportClient::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEditorViewportClient::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(BehaviorSet);
	Collector.AddReferencedObject(ViewportButtonsAPI);
	Collector.AddReferencedObject(CanvasObject);
}

bool FUVEditor2DViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	// We'll support disabling input like our base class, even if it does not end up being used.
	if (bDisableInput)
	{
		return true;
	}

	// Our viewport manipulation is placed in the input router that ModeTools manages
	return ModeTools->InputKey(this, EventArgs.Viewport, EventArgs.Key, EventArgs.Event);

}


// Note that this function gets called from the super class Draw(FViewport*, FCanvas*) overload to draw the scene.
// We don't override that top-level function so that it can do whatever view calculations it needs to do.
void FUVEditor2DViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (bDrawGrid)
	{
		DrawGrid(View, PDI);
	}

	// Calls ModeTools draw/render functions
	FEditorViewportClient::Draw(View, PDI);
}

void FUVEditor2DViewportClient::DrawGrid(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	// TODO: Drawing the grid is another location where we need to pay attention to the coordinate orienation of the viewport and camera. This
	// should probably be cleaned up at some point to make it more robust to these assumptions. 


	// Basic scaling amount
	const double UVScale = UUVEditorMode::GetUVMeshScalingFactor();
	
	// Determine important geometry of the viewport for creating grid lines
	FVector WorldCenterPoint( 0,0,0 );
	FVector4 WorldToScreenCenter = View->WorldToScreen(WorldCenterPoint);	
	double ZoomFactor = WorldToScreenCenter.W;
	FVector4 MaxScreen(1 * ZoomFactor, 1 * ZoomFactor, 0, 1);
	FVector4 MinScreen(-1 * ZoomFactor, -1 * ZoomFactor, 0, 1);
	FVector WorldBoundsMax = View->ScreenToWorld(MaxScreen);
	FVector WorldBoundsMin = View->ScreenToWorld(MinScreen);
	FVector ViewLoc = GetViewLocation();
	ViewLoc.Z = 0.0; // We are treating the scene like a 2D plane, so we'll clamp the Z position here to 
	               // 0 as a simple projection step just in case.

	// We wrap these inside an FAxisAlignedBox3d to ensure the Min coordinates < Max coordinates, which might not be the case coming directly out of the View.
	UE::Geometry::FAxisAlignedBox3d  WorldBounds;
	WorldBounds.Contain(WorldBoundsMax);
	WorldBounds.Contain(WorldBoundsMin);
	
	// Prevent grid from drawing if we are too close or too far, in order to avoid potential graphical issues.
	if (ZoomFactor < 100000 && ZoomFactor > 1)
	{
		// Setup and call grid calling function
		UE::Geometry::FFrame3d LocalFrame(ViewLoc);
		FTransform Transform;
		TArray<FColor> Colors;
		Colors.Push(FUVEditorUXSettings::GridMajorColor);
		Colors.Push(FUVEditorUXSettings::GridMinorColor);
		MeshDebugDraw::DrawHierarchicalGrid(UVScale, ZoomFactor / UVScale,
			500, // Maximum density of lines to draw per level before skipping the level
			WorldBounds.Max, WorldBounds.Min,
			FUVEditorUXSettings::GridLevels, // Number of levels to draw
			FUVEditorUXSettings::GridSubdivisionsPerLevel, // Number of subdivisions per level
			Colors,
			LocalFrame, FUVEditorUXSettings::GridMajorThickness, true,
			PDI, Transform);
	}

	// Draw colored axis lines
	PDI->DrawLine(FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(FVector2f(0,0)), 
		FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(FVector2f(1, 0)), 
		FUVEditorUXSettings::XAxisColor, SDPG_World, FUVEditorUXSettings::AxisThickness, 0, true);
	PDI->DrawLine(FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(FVector2f(0, 0)), 
		FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(FVector2f(0, 1)), 
		FUVEditorUXSettings::YAxisColor, SDPG_World, FUVEditorUXSettings::AxisThickness, 0, true);

	// TODO: Draw a little UV axis thing in the lower left, like the XYZ things that normal viewports have.
}

bool FUVEditor2DViewportClient::ShouldOrbitCamera() const
{
	return false; // The UV Editor's 2D viewport should never orbit.
}

void FUVEditor2DViewportClient::SetWidgetMode(UE::Widget::EWidgetMode NewMode)
{
	if (ViewportButtonsAPI)
	{
		switch (NewMode)
		{
		case UE::Widget::EWidgetMode::WM_None:
			ViewportButtonsAPI->SetGizmoMode(UUVToolViewportButtonsAPI::EGizmoMode::Select);
			break;
		case UE::Widget::EWidgetMode::WM_Translate:
			ViewportButtonsAPI->SetGizmoMode(UUVToolViewportButtonsAPI::EGizmoMode::Transform);
			break;
		default:
			// Do nothing
			break;
		}
	}
}

bool FUVEditor2DViewportClient::AreWidgetButtonsEnabled() const
{
	return ViewportButtonsAPI && ViewportButtonsAPI->AreGizmoButtonsEnabled();
}

void FUVEditor2DViewportClient::SetSnappingActive(bool bActive)
{
	if (ViewportButtonsAPI)
	{
		ViewportButtonsAPI->SetSnappingActive(bActive);	
	}
}

bool FUVEditor2DViewportClient::IsSnappingActive()
{
	return ViewportButtonsAPI && ViewportButtonsAPI->IsSnappingActive();
}

void FUVEditor2DViewportClient::SetLocationGridSnapEnabled(bool bEnabled)
{
	ViewportButtonsAPI->ToggleSnapEnabled(UUVToolViewportButtonsAPI::ESnapTypeFlag::Location);
}

bool FUVEditor2DViewportClient::GetLocationGridSnapEnabled()
{
	return ViewportButtonsAPI->GetSnapEnabled(UUVToolViewportButtonsAPI::ESnapTypeFlag::Location);
}

void FUVEditor2DViewportClient::SetLocationGridSnapValue(float SnapValue)
{
	ViewportButtonsAPI->SetSnapValue(UUVToolViewportButtonsAPI::ESnapTypeFlag::Location, SnapValue);
}

float FUVEditor2DViewportClient::GetLocationGridSnapValue()
{
	return 	ViewportButtonsAPI->GetSnapValue(UUVToolViewportButtonsAPI::ESnapTypeFlag::Location);
}

void FUVEditor2DViewportClient::SetRotationGridSnapEnabled(bool bEnabled)
{
	ViewportButtonsAPI->ToggleSnapEnabled(UUVToolViewportButtonsAPI::ESnapTypeFlag::Rotation);
}

bool FUVEditor2DViewportClient::GetRotationGridSnapEnabled()
{
	return ViewportButtonsAPI->GetSnapEnabled(UUVToolViewportButtonsAPI::ESnapTypeFlag::Rotation);
}

void FUVEditor2DViewportClient::SetRotationGridSnapValue(float SnapValue)
{
	ViewportButtonsAPI->SetSnapValue(UUVToolViewportButtonsAPI::ESnapTypeFlag::Rotation, SnapValue);
}

float FUVEditor2DViewportClient::GetRotationGridSnapValue()
{
	return ViewportButtonsAPI->GetSnapValue(UUVToolViewportButtonsAPI::ESnapTypeFlag::Rotation);
}

void FUVEditor2DViewportClient::SetScaleGridSnapEnabled(bool bEnabled)
{
	ViewportButtonsAPI->ToggleSnapEnabled(UUVToolViewportButtonsAPI::ESnapTypeFlag::Scale);
}

bool FUVEditor2DViewportClient::GetScaleGridSnapEnabled()
{
	return ViewportButtonsAPI->GetSnapEnabled(UUVToolViewportButtonsAPI::ESnapTypeFlag::Scale);
}

void FUVEditor2DViewportClient::SetScaleGridSnapValue(float SnapValue)
{
	ViewportButtonsAPI->SetSnapValue(UUVToolViewportButtonsAPI::ESnapTypeFlag::Scale, SnapValue);
}

float FUVEditor2DViewportClient::GetScaleGridSnapValue()
{
	return ViewportButtonsAPI->GetSnapValue(UUVToolViewportButtonsAPI::ESnapTypeFlag::Scale);
}

bool FUVEditor2DViewportClient::CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const
{
	if (!AreWidgetButtonsEnabled())
	{
		return false;
	}

	return NewMode == UE::Widget::EWidgetMode::WM_None 
		|| NewMode == UE::Widget::EWidgetMode::WM_Translate;
}

UE::Widget::EWidgetMode FUVEditor2DViewportClient::GetWidgetMode() const
{
	if (!AreWidgetButtonsEnabled())
	{
		return UE::Widget::EWidgetMode::WM_None;
	}

	switch (ViewportButtonsAPI->GetGizmoMode())
	{
	case UUVToolViewportButtonsAPI::EGizmoMode::Select:
		return UE::Widget::EWidgetMode::WM_None;
	case UUVToolViewportButtonsAPI::EGizmoMode::Transform:
		return UE::Widget::EWidgetMode::WM_Translate;
	default:
		return UE::Widget::EWidgetMode::WM_None;
	}
}

bool FUVEditor2DViewportClient::AreSelectionButtonsEnabled() const
{
	return ViewportButtonsAPI && ViewportButtonsAPI->AreSelectionButtonsEnabled();
}

void FUVEditor2DViewportClient::SetSelectionMode(UUVToolViewportButtonsAPI::ESelectionMode NewMode)
{
	if (ViewportButtonsAPI)
	{
		ViewportButtonsAPI->SetSelectionMode(NewMode);
	}
}

UUVToolViewportButtonsAPI::ESelectionMode FUVEditor2DViewportClient::GetSelectionMode() const
{
	if (!AreSelectionButtonsEnabled())
	{
		return UUVToolViewportButtonsAPI::ESelectionMode::None;
	}

	return ViewportButtonsAPI->GetSelectionMode();
}

void FUVEditor2DViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{

	if (CanvasObject)
	{
		CanvasObject->Init(InViewport.GetSizeXY().X, InViewport.GetSizeXY().Y, &View, &Canvas);
		if (bDrawGridRulers)
		{
			DrawGridRulers(InViewport, View, *CanvasObject);
		}

		bool bEnableUDIMSupport = (FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport.GetValueOnGameThread() > 0);
		if (bEnableUDIMSupport)
		{
			DrawUDIMLabels(InViewport, View, *CanvasObject);
		}
	}
	
	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);
}

void FUVEditor2DViewportClient::DrawGridRulers(FViewport& InViewport, FSceneView& View, UCanvas& Canvas)
{
	const double UVScale = FUVEditorUXSettings::UVMeshScalingFactor;
	const int32 Level = FUVEditorUXSettings::RulerSubdivisionLevel;
	const int32 Subdivisions = FUVEditorUXSettings::GridSubdivisionsPerLevel; // Number of subdivisions per level

	// Determine important geometry of the viewport for creating grid lines
	FVector WorldCenterPoint(0, 0, 0);
	FVector4 WorldToScreenCenter = View.WorldToScreen(WorldCenterPoint);
	double ZoomFactor = WorldToScreenCenter.W;
	double GridZoomFactor = ZoomFactor / UVScale;

	double LogZoom = FMath::LogX(Subdivisions, GridZoomFactor);
	double LogZoomDirection = FMath::Sign(LogZoom);
	LogZoom = FMath::Abs(LogZoom);
	LogZoom = FMath::Floor(LogZoom);
	LogZoom = LogZoomDirection * LogZoom;

	double GridScale = FMathd::Pow(Subdivisions, LogZoom - static_cast<double>(Level));

	FVector2D PixelMinBounds(0, Canvas.SizeY);
	FVector2D PixelMaxBounds(Canvas.SizeX, 0);
	FVector2D UVMinBounds, UVMaxBounds;
	FUVEditor2DViewportClientLocals::ConvertPixelToUV(PixelMinBounds, WorldToScreenCenter.Z / WorldToScreenCenter.W, UVMinBounds, View);
	FUVEditor2DViewportClientLocals::ConvertPixelToUV(PixelMaxBounds, WorldToScreenCenter.Z / WorldToScreenCenter.W, UVMaxBounds, View);

	double GridXOrigin = FMath::GridSnap(UVMinBounds.X, GridScale);
	double GridYOrigin = FMath::GridSnap(UVMinBounds.Y, GridScale);

	const UFont* Font = GEngine->GetTinyFont();

	double CurrentGridXPos = GridXOrigin;
	while (CurrentGridXPos < UVMinBounds.X)
    {
		CurrentGridXPos += GridScale;
	}

	do
    {
		FVector2D UVRulerPoint(CurrentGridXPos, 0.0);
		FVector2D PixelRulerPoint;
		FUVEditor2DViewportClientLocals::ConvertUVToPixel(UVRulerPoint, PixelRulerPoint, View);

		FCanvasTextItem TextItem = FUVEditor2DViewportClientLocals::CreateTextAnchored(PixelRulerPoint,
			FUVEditor2DViewportClientLocals::ConvertToExponentialNotation(CurrentGridXPos, 2),
			Font,
			FLinearColor(FUVEditorUXSettings::RulerXColor),
			Canvas,
			FUVEditor2DViewportClientLocals::ETextAnchorPosition::TopLeftCorner,
			{ 3,0,3,0 });

		Canvas.DrawItem(TextItem);
		CurrentGridXPos += GridScale;
	} while (CurrentGridXPos <= UVMaxBounds.X);

	double CurrentGridYPos = GridYOrigin;
	while (CurrentGridYPos < UVMinBounds.Y)
    {
		CurrentGridYPos += GridScale;
	}

	do
    {
		FVector2D UVRulerPoint(0.0, CurrentGridYPos);
		FVector2D PixelRulerPoint;
		FUVEditor2DViewportClientLocals::ConvertUVToPixel(UVRulerPoint, PixelRulerPoint, View);

		FCanvasTextItem TextItem = FUVEditor2DViewportClientLocals::CreateTextAnchored(PixelRulerPoint,
			FUVEditor2DViewportClientLocals::ConvertToExponentialNotation(CurrentGridYPos, 2),
			Font,
			FLinearColor(FUVEditorUXSettings::RulerYColor),
			Canvas,
			FUVEditor2DViewportClientLocals::ETextAnchorPosition::BottomRightCorner,
			{ 0,3,0,3 });

		Canvas.DrawItem(TextItem);
		CurrentGridYPos += GridScale;
	} while (CurrentGridYPos <= UVMaxBounds.Y);

}

void FUVEditor2DViewportClient::DrawUDIMLabels(FViewport& InViewport, FSceneView& View, UCanvas& Canvas)
{
	FVector2D Origin(0,0);
	FVector2D OneUDimOffet(1.0, 0);
	FVector2D OriginPixel, OneUDimOffetPixel;

	FUVEditor2DViewportClientLocals::ConvertUVToPixel(Origin, OriginPixel, View);
	FUVEditor2DViewportClientLocals::ConvertUVToPixel(OneUDimOffet, OneUDimOffetPixel, View);

	double MaxAllowedLabelWidth = FMath::Abs(OriginPixel.X - OneUDimOffetPixel.X);

	const UFont* Font = GEngine->GetLargeFont();
	FNumberFormattingOptions FormatOptions;
	FormatOptions.UseGrouping = false;

	for (const FUDIMBlock& Block : UVTool2DViewportAPI->GetUDIMBlocks())
	{
		FVector2D TextSize;
		FText UDIMLabel = FText::AsNumber(Block.UDIM, &FormatOptions);
		Canvas.TextSize(Font, UDIMLabel.ToString(), TextSize[0], TextSize[1]);
		if(TextSize[0] > MaxAllowedLabelWidth)
		{
			continue; // Don't draw if our label is bigger than the visual size of one UDIM "block" in the viewport
		}			

		FVector2D UDimUV(Block.BlockU() + 1.0, Block.BlockV() + 1.0);
		FVector2D UDimPixel;
		FUVEditor2DViewportClientLocals::ConvertUVToPixel(UDimUV, UDimPixel, View);

		FCanvasTextItem TextItem = FUVEditor2DViewportClientLocals::CreateTextAnchored(UDimPixel,
				UDIMLabel,
				Font,
				FLinearColor(FColor(255, 196, 196)),
				Canvas,
				FUVEditor2DViewportClientLocals::ETextAnchorPosition::TopRightCorner,
				{ 5,0,0,5 },
			    false /* Don't clamp to screen bounds */ );

		Canvas.DrawItem(TextItem);
	}
}
