// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditor2DViewportContext.h"

#include "SUVEditor2DViewport.h"
#include "UVEditorUXSettings.h"
#include "Settings/LevelEditorViewportSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditor2DViewportContext)

#define LOCTEXT_NAMESPACE "UVEditor2DViewportToolbar"

UUVEditor2DViewportContext::UUVEditor2DViewportContext()
{
	bShowSurfaceSnap = false;

	GridSnapSizes.Reserve(FUVEditorUXSettings::MaxLocationSnapValue());
	for (int32 Index = 0; Index < FUVEditorUXSettings::MaxLocationSnapValue(); ++Index)
	{
		GridSnapSizes.Add(FUVEditorUXSettings::LocationSnapValue(Index));
	}	
}

TSharedPtr<FUVEditor2DViewportClient> UUVEditor2DViewportContext::GetViewportClient() const
{
	if (TSharedPtr<SEditorViewport> EditorViewport = Viewport.Pin())
	{
		return StaticCastSharedPtr<FUVEditor2DViewportClient>(EditorViewport->GetViewportClient());
	}
	return nullptr;
}

FText UUVEditor2DViewportContext::GetGridSnapLabel() const
{
	if (TSharedPtr<FUVEditor2DViewportClient> ViewportClient = GetViewportClient())
	{
		return FText::AsNumber(ViewportClient->GetLocationGridSnapValue());
	}
	return FText();
}

TArray<float> UUVEditor2DViewportContext::GetGridSnapSizes() const
{
	return GridSnapSizes;
}

bool UUVEditor2DViewportContext::IsGridSnapSizeActive(int32 GridSizeIndex) const
{
	if (TSharedPtr<FUVEditor2DViewportClient> ViewportClient = GetViewportClient())
	{
		const float CurrGridSize = GridSnapSizes[GridSizeIndex];
		return FMath::IsNearlyEqual(ViewportClient->GetLocationGridSnapValue(), CurrGridSize);
	}
	return false;
}

void UUVEditor2DViewportContext::SetGridSnapSize(int32 GridSizeIndex)
{
	if (TSharedPtr<FUVEditor2DViewportClient> ViewportClient = GetViewportClient())
	{
		const float CurrGridSize = GridSnapSizes[GridSizeIndex];
		ViewportClient->SetLocationGridSnapValue(CurrGridSize);
	}
}

FText UUVEditor2DViewportContext::GetRotationSnapLabel() const
{
	if (TSharedPtr<FUVEditor2DViewportClient> ViewportClient = GetViewportClient())
	{
		return FText::Format(LOCTEXT("GridRotation - Number - DegreeSymbol", "{0}\u00b0"), FText::AsNumber(ViewportClient->GetRotationGridSnapValue()));
	}
	return FText();
}

bool UUVEditor2DViewportContext::IsRotationSnapActive(int32 RotationIndex, ERotationGridMode RotationMode) const
{
	if (TSharedPtr<FUVEditor2DViewportClient> ViewportClient = GetViewportClient())
	{
		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
		const TArray<float>& Sizes = RotationMode == GridMode_Common ? ViewportSettings->CommonRotGridSizes : ViewportSettings->DivisionsOf360RotGridSizes;
		const float CurrGridAngle = Sizes[RotationIndex];
		return FMath::IsNearlyEqual(ViewportClient->GetRotationGridSnapValue(), CurrGridAngle);
	}
	return false;
}

void UUVEditor2DViewportContext::SetRotationSnapSize(int32 RotationIndex, ERotationGridMode RotationMode)
{
	if (TSharedPtr<FUVEditor2DViewportClient> ViewportClient = GetViewportClient())
	{
		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
		const TArray<float>& Sizes = RotationMode == GridMode_Common ? ViewportSettings->CommonRotGridSizes : ViewportSettings->DivisionsOf360RotGridSizes;
		return ViewportClient->SetRotationGridSnapValue(Sizes[RotationIndex]);
	}
}

FText UUVEditor2DViewportContext::GetScaleSnapLabel() const
{
	if (TSharedPtr<FUVEditor2DViewportClient> ViewportClient = GetViewportClient())
	{
		FNumberFormattingOptions NumberFormattingOptions;
		NumberFormattingOptions.MaximumFractionalDigits = 5;

		const float CurGridAmount = ViewportClient->GetScaleGridSnapValue();
		return FText::AsNumber(CurGridAmount, &NumberFormattingOptions);
	}
	return FText();
}

bool UUVEditor2DViewportContext::IsScaleSnapActive(int32 ScaleIndex) const
{
	if (TSharedPtr<FUVEditor2DViewportClient> ViewportClient = GetViewportClient())
	{
		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
		const float CurrGridAngle = ViewportSettings->ScalingGridSizes[ScaleIndex];
		return FMath::IsNearlyEqual(ViewportClient->GetRotationGridSnapValue(), CurrGridAngle);
	}
	return false;
}

void UUVEditor2DViewportContext::SetScaleSnapSize(int32 ScaleIndex)
{
	if (TSharedPtr<FUVEditor2DViewportClient> ViewportClient = GetViewportClient())
	{
		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
		ViewportClient->SetScaleGridSnapValue(ViewportSettings->ScalingGridSizes[ScaleIndex]);
	}
}


#undef LOCTEXT_NAMESPACE
