// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditor3DViewportClient.h"

#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseWheelBehavior.h"
#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Drawing/MeshDebugDrawing.h"
#include "FrameTypes.h"
#include "UVEditorMode.h"
#include "ContextObjects/UVToolViewportButtonsAPI.h"

FUVEditor3DViewportClient::FUVEditor3DViewportClient(FEditorModeTools* InModeTools,
	FPreviewScene* InPreviewScene, const TWeakPtr<SEditorViewport>& InEditorViewportWidget,
	UUVToolViewportButtonsAPI* ViewportButtonsAPIIn)
	: FEditorViewportClient(InModeTools, InPreviewScene, InEditorViewportWidget), 
	ViewportButtonsAPI(ViewportButtonsAPIIn)
{
	// We want our near clip plane to be quite close so that we can zoom in further.
	OverrideNearClipPlane(KINDA_SMALL_NUMBER);
}

bool FUVEditor3DViewportClient::ShouldOrbitCamera() const
{
	// bIsTracking indicates that the viewport has captured the mouse for camera movement. If we don't have this
	//  check, things do mostly work (the camera won't orbit unnecessarily), but we will run into a tricky bug
	//  with tool drag captures: in EUVEditor3DViewportClientCameraMode::Orbit mode, the drag captures will hide
	//  the cursor because ShouldOrbitCamera would return true, and then the cursor will then be reinstated at the
	//  old position on capture end, leading to a broken-looking reset of the cursor position.
	if (!bIsTracking)
	{
		return false;
	}

	// Including some additional checks to prevent the orbit mode from being on all the time,
	// which ultimately causes weirdness in how the camera transform matrices are handled by
	// the viewport internally.
	const bool bLeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton) && !bLockFlightCamera;
	const bool bMiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton);
	const bool bRightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton);
	const bool bIsOnlyAltPressed = IsAltPressed() && !IsCtrlPressed() && !IsShiftPressed();

	switch (CameraMode) {
	case EUVEditor3DViewportClientCameraMode::Orbit:
		return bIsOnlyAltPressed || bLeftMouseButtonDown || bMiddleMouseButtonDown || bRightMouseButtonDown;
	case EUVEditor3DViewportClientCameraMode::Fly:
		return FEditorViewportClient::ShouldOrbitCamera();
	default:
		ensure(false);
		return FEditorViewportClient::ShouldOrbitCamera();
	}
}

void FUVEditor3DViewportClient::FocusCameraOnSelection()
{
	if (ViewportButtonsAPI) 
	{
		ViewportButtonsAPI->InitiateFocusCameraOnSelection();
	}
}
