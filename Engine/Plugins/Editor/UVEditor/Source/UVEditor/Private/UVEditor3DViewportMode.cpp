// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditor3DViewportMode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditor3DViewportMode)

#define LOCTEXT_NAMESPACE "UUVEditor3DViewportMode"

const FEditorModeID UUVEditor3DViewportMode::EM_ModeID = TEXT("EM_UVEditor3DViewportModeId");

UUVEditor3DViewportMode::UUVEditor3DViewportMode()
{
	Info = FEditorModeInfo(
		EM_ModeID,
		LOCTEXT("ModeName", "UV 3D Viewport Dummy Mode"),
		FSlateIcon(),
		false);
}

#undef LOCTEXT_NAMESPACE
