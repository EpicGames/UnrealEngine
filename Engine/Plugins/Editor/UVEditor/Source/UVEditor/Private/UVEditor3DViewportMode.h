// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UEdMode.h"

#include "UVEditor3DViewportMode.generated.h"

/**
 * The UV editor live preview has its own world, mode manager, and input router. It doesn't
 *  really need a mode, and we can't hook it up to the UV editor mode because we don't want
 *  input to be routed directly to it from this other world (it is instead accessed through
 *  a context API in tools). However, we do want hotkey events to be routed to our tools from
 *  the live preview. This is done by registering our command objects to a command list that
 *  the live preview routes input to, and to have that command list be at the expected place
 *  in the input routing, we need a mode that has a toolkit object (See FEditorModeTools::InputKey.
 *  The hotkey handling passes through the ProcessCommandBindings on the mode toolkit there).
 *  The default mode (UEdModeDefault) does not have a toolkit object, hence this dummy mode.
 */
UCLASS(Transient)
class UUVEditor3DViewportMode : public UEdMode
{
	GENERATED_BODY()

public:
	const static FEditorModeID EM_ModeID;

	UUVEditor3DViewportMode();
};
