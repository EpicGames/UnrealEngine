// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorBackgroundPreview.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Drawing/MeshWireframeComponent.h"
#include "Engine/Texture2D.h"
#include "Materials/Material.h"
#include "ToolSetupUtil.h"
#include "Async/Async.h"
#include "UVEditorUXSettings.h"
#include "UDIMUtilities.h"
#include "Materials/MaterialExpressionUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorBackgroundPreview)

using namespace UE::Geometry;

void UUVEditorBackgroundPreview::OnCreated()
{
	Settings = NewObject<UUVEditorBackgroundPreviewProperties>(this);
	Settings->WatchProperty(Settings->bVisible, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->SourceType, [this](EUVEditorBackgroundSourceType) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->SourceTexture, [this](UTexture2D*) {bSettingsModified = true; });
	Settings->WatchProperty(Settings->SourceMaterial, [this](UMaterial*) {bSettingsModified = true; });
	Settings->WatchProperty(Settings->UDIMBlocks, [this](const TArray<int32>&) {bSettingsModified = true; });

	bSettingsModified = false;

	BackgroundComponent = NewObject<UTriangleSetComponent>(GetActor());
	BackgroundComponent->SetupAttachment(GetActor()->GetRootComponent());
	BackgroundComponent->RegisterComponent();
}

void UUVEditorBackgroundPreview::OnTick(float DeltaTime)
{
	// Check if the CVAR has been updated behind the scenes
	bool bEnableUDIMSupport = (FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport.GetValueOnGameThread() > 0);
	if (Settings->bUDIMsEnabled != bEnableUDIMSupport)
	{
		Settings->bUDIMsEnabled = bEnableUDIMSupport;
		bSettingsModified = true;
	}

	if (bSettingsModified)
	{		
		UpdateBackground();
		UpdateVisibility();
		bSettingsModified = false;
		OnBackgroundMaterialChange.Broadcast(BackgroundMaterial);
	}
}

void UUVEditorBackgroundPreview::UpdateVisibility()
{
	if (Settings->bVisible == false)
	{
		BackgroundComponent->SetVisibility(false);
		return;
	}

	BackgroundComponent->SetVisibility(true);
	BackgroundComponent->MarkRenderStateDirty();
}

void UUVEditorBackgroundPreview::UpdateBackground()
{
	const FVector Normal(0, 0, 1);
	const FColor BackgroundColor = FColor::Blue;

	UMaterial* RegularMaterial = LoadObject<UMaterial>(nullptr, TEXT("/UVEditor/Materials/UVEditorBackground"));
	UMaterial* VirtualTextureMaterial = LoadObject<UMaterial>(nullptr, TEXT("/UVEditor/Materials/UVEditorVTBackground"));

	check(RegularMaterial);
	check(VirtualTextureMaterial);

	BackgroundMaterial = UMaterialInstanceDynamic::Create(RegularMaterial, this);
	switch (Settings->SourceType)
	{
		case EUVEditorBackgroundSourceType::Checkerboard:
		{
			// Do nothing, since the default material is already set up for a checkerboard.
		}
		break;

		case EUVEditorBackgroundSourceType::Material:
		{
			if (Settings->SourceMaterial)
			{
				BackgroundMaterial = UMaterialInstanceDynamic::Create(Settings->SourceMaterial.Get(), this);
			}
		}
		break;

		case EUVEditorBackgroundSourceType::Texture:
		{
			if (Settings->SourceTexture)
			{
				if (Settings->SourceTexture->IsCurrentlyVirtualTextured())
				{
					BackgroundMaterial = UMaterialInstanceDynamic::Create(VirtualTextureMaterial, this);
				}

				EMaterialSamplerType SamplerRequiredForTexture = MaterialExpressionUtils::GetSamplerTypeForTexture(Settings->SourceTexture.Get());

				switch (SamplerRequiredForTexture)
				{
				case EMaterialSamplerType::SAMPLERTYPE_Color:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_Color"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 0);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_Grayscale:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_Grayscale"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 1);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_Alpha:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_Alpha"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 2);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_Normal:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_Normal"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 3);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_Masks:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_Masks"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 4);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_DistanceFieldFont:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_DistanceFeildFont"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 5);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_LinearColor:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_LinearColor"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 6);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_LinearGrayscale:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_LinearGrayscale"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 7);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_Data:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_Data"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 8);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_External:
					// TODO: Enable support for External textures at a future date as needed.
					//BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_Data"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("TextureSampler"), 9);
					break;

				case EMaterialSamplerType::SAMPLERTYPE_VirtualColor:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_VTColor"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("VirtualTextureSampler"), 0);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_VirtualGrayscale:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_VTGrayscale"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("VirtualTextureSampler"), 1);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_VirtualAlpha:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_VTAlpha"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("VirtualTextureSampler"), 2);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_VirtualNormal:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_VTNormal"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("VirtualTextureSampler"), 3);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_VirtualMasks:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_VTMasks"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("VirtualTextureSampler"), 4);
					break;						
				case EMaterialSamplerType::SAMPLERTYPE_VirtualLinearColor:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_VTLinearColor"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("VirtualTextureSampler"), 6);
					break;
				case EMaterialSamplerType::SAMPLERTYPE_VirtualLinearGrayscale:
					BackgroundMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_VTLinearGrayscale"), Settings->SourceTexture);
					BackgroundMaterial->SetScalarParameterValue(TEXT("VirtualTextureSampler"), 7);
					break;
				}
			}
		}
		break;

		default:
			ensure(false);
	}

	BackgroundMaterial->SetScalarParameterValue(TEXT("BackgroundPixelDepthOffset"), FUVEditorUXSettings::BackgroundQuadDepthOffset);	
	BackgroundComponent->Clear();

	TArray<FIndex2i> UDimBlocksToRender;
	if (Settings->bUDIMsEnabled)
	{
		// TODO: Find a way to access the list of UDIMs from the context object instead? 
		for (int32 BlockIndex = 0; BlockIndex < Settings->UDIMBlocks.Num(); ++BlockIndex)
		{
			FIndex2i Block;
			UE::TextureUtilitiesCommon::ExtractUDIMCoordinates(Settings->UDIMBlocks[BlockIndex], Block.A, Block.B);
			UDimBlocksToRender.Push(Block);
		}
	}
	if (UDimBlocksToRender.Num() == 0)
	{
		UDimBlocksToRender.Push(FIndex2i(0, 0));
	}

	for (const FIndex2i& Block : UDimBlocksToRender)
	{
		auto MakeQuadVert = [&Block, &Normal, &BackgroundColor](int32 CornerX, int32 CornerY)
		{
			FVector2f ExternalUV(static_cast<float>(Block.A + CornerX), static_cast<float>(Block.B + CornerY));

			return FRenderableTriangleVertex(FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(ExternalUV),
				(FVector2D)FUVEditorUXSettings::ExternalUVToInternalUV(ExternalUV),
				Normal, BackgroundColor);
		};

		FRenderableTriangleVertex V00 = MakeQuadVert(0,0);
		FRenderableTriangleVertex V10 = MakeQuadVert(1, 0);
		FRenderableTriangleVertex V11 = MakeQuadVert(1, 1);
		FRenderableTriangleVertex V01 = MakeQuadVert(0,1);

		BackgroundComponent->AddTriangle(FRenderableTriangle(BackgroundMaterial, V00, V10, V11));
		BackgroundComponent->AddTriangle(FRenderableTriangle(BackgroundMaterial, V00, V11, V01));
	}
}

