// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorCommands.h"

#include "Styling/AppStyle.h"
#include "Framework/Commands/InputChord.h"
#include "UVEditorBrushSelectTool.h"
#include "UVEditorStyle.h"

#define LOCTEXT_NAMESPACE "FUVEditorCommands"
	
FUVEditorCommands::FUVEditorCommands()
	: TCommands<FUVEditorCommands>("UVEditor",
		LOCTEXT("ContextDescription", "UV Editor"), 
		NAME_None, // Parent
		FUVEditorStyle::Get().GetStyleSetName()
		)
{
}

void FUVEditorCommands::RegisterCommands()
{
	// These are part of the asset editor UI
	UI_COMMAND(OpenUVEditor, "UV Editor", "Open the UV Editor window.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ApplyChanges, "Apply", "Apply changes to original meshes", EUserInterfaceActionType::Button, FInputChord());

	// These get linked to various tool buttons.
	UI_COMMAND(BeginLayoutTool, "Layout", "Pack existing UVs", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginTransformTool, "Transform", "Transform existing UVs", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAlignTool, "Align", "Align existing UVs", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginDistributeTool, "Distribute", "Distribute existing UVs", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginTexelDensityTool, "Texel Density", "Modify UVs based on texel density", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginParameterizeMeshTool, "AutoUV", "Auto-unwrap and pack UVs", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginChannelEditTool, "Channels", "Modify UV channels", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginSeamTool, "Seam", "Edit UV seams", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginRecomputeUVsTool, "Unwrap", "Perform UV unwrapping", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginBrushSelectTool, "Brush", "Brush select triangles", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginUVSnapshotTool, "Snapshot", "Export a texture asset of a UV Layout", EUserInterfaceActionType::ToggleButton, FInputChord());

	// These get linked to one-off tool actions.
	UI_COMMAND(SewAction, "Sew", "Sew edges highlighted in red to edges highlighted in green", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SplitAction, "Split",
	           "Given an edge selection, split those edges. Given a vertex selection, split any selected bowtie vertices. Given a triangle selection, split along selection boundaries.",
	           EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MakeIslandAction, "Island", "Given a triangle selection, make the selection into a single separate UV Island.",
		EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UnsetUVsAction, "UnsetUVs", "Unset the UVs on the given triangle selection.",
		EUserInterfaceActionType::Button, FInputChord());

	// These allow us to link up to pressed keys
	UI_COMMAND(AcceptOrCompleteActiveTool, "Accept", "Accept the active tool", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter));
	UI_COMMAND(CancelOrCompleteActiveTool, "Cancel", "Cancel the active tool or clear current selection", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));

	// These get used in viewport buttons
	UI_COMMAND(VertexSelection, "Vertex Selection", "Select vertices", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::One));
	UI_COMMAND(EdgeSelection, "Edge Selection", "Select edges", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Two));
	UI_COMMAND(TriangleSelection, "Triangle Selection", "Select triangles", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Three));
	UI_COMMAND(IslandSelection, "Island Selection", "Select islands", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Four));
	UI_COMMAND(FullMeshSelection, "Mesh Selection", "Select meshes", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Five));
	UI_COMMAND(SelectAll, "Select All", "Select everything based on current selection mode", EUserInterfaceActionType::None, FInputChord(EKeys::A, EModifierKey::Control));

	UI_COMMAND(EnableOrbitCamera, "Orbit", "Enable orbit camera", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(EnableFlyCamera, "Fly", "Enable fly camera", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetFocusCamera, "Focus Camera", "Focus camera around the currently selected UVs", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Alt));

	UI_COMMAND(ToggleBackground, "Toggle Background", "Toggle background display", EUserInterfaceActionType::ToggleButton,
	           FInputChord(EModifierKey::Alt, EKeys::B));
}

//~ Modeled on ModelingToolsActions.cpp
UE::Geometry::FUVEditorToolActionCommands::FUVEditorToolActionCommands() :
	TInteractiveToolCommands<FUVEditorToolActionCommands>(
		"UVEditorHotkeys", // Context name for fast lookup
		LOCTEXT("HotkeysCategory", "UV Editor Hotkeys"), // Localized context name for displaying
		NAME_None, // Parent
		FUVEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
{
}
void UE::Geometry::FUVEditorToolActionCommands::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)
{
}
void UE::Geometry::FUVEditorToolActionCommands::RegisterAllToolActions()
{
	UE::Geometry::FUVEditorBrushSelectToolCommands::Register();
}
void UE::Geometry::FUVEditorToolActionCommands::UnregisterAllToolActions()
{
	UE::Geometry::FUVEditorBrushSelectToolCommands::Unregister();
}
void UE::Geometry::FUVEditorToolActionCommands::UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind)
{
#define UPDATE_BINDING(CommandsType)  if (!bUnbind) \
	CommandsType::Get().BindCommandsForCurrentTool(UICommandList, Tool); \
	else CommandsType::Get().UnbindActiveCommands(UICommandList);

	if (ExactCast<UUVEditorBrushSelectTool>(Tool))
	{
		UPDATE_BINDING(UE::Geometry::FUVEditorBrushSelectToolCommands);
	}
}

//~ Modeled on ModelingToolsActions.cpp
#define DEFINE_TOOL_ACTION_COMMANDS(CommandsClassName, ContextNameString, SettingsDialogString, ToolClassName ) \
UE::Geometry::CommandsClassName::CommandsClassName() : TInteractiveToolCommands<CommandsClassName>( \
ContextNameString, NSLOCTEXT("Contexts", ContextNameString, SettingsDialogString), NAME_None, FUVEditorStyle::Get().GetStyleSetName()) {} \
void UE::Geometry::CommandsClassName::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) \
{\
	ToolCDOs.Add(GetMutableDefault<ToolClassName>()); \
}

DEFINE_TOOL_ACTION_COMMANDS(FUVEditorBrushSelectToolCommands, "UVBrushSelect", "UV Editor - Brush Select", UUVEditorBrushSelectTool);


#undef LOCTEXT_NAMESPACE