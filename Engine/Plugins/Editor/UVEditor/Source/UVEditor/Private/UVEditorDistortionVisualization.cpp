// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorDistortionVisualization.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Drawing/MeshWireframeComponent.h"
#include "Engine/Texture2D.h"
#include "Materials/Material.h"
#include "ToolSetupUtil.h"
#include "Async/Async.h"
#include "UVEditorUXSettings.h"
#include "UDIMUtilities.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "MeshOpPreviewHelpers.h"
#include "UVEditorUXSettings.h"
#include "Math/UnitConversion.h"
#include "Parameterization/MeshUDIMClassifier.h"
#include "UDIMUtilities.h"
#include "Parameterization/UVMetrics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorDistortionVisualization)

using namespace UE::Geometry;

namespace DistortionMetrics
{
	double Triangle2DArea(FVector2f p1, FVector2f p2, FVector2f p3)
	{
		return FMath::Abs(((p2.X - p1.X) * (p3.Y - p1.Y) - (p3.X - p1.X) * (p2.Y - p1.Y)) / 2.0);
	}

	double Triangle3DArea(FVector3d q1, FVector3d q2, FVector3d q3)
	{
		FVector3d Q1 = FVector3d(q2.X - q1.X, q2.Y - q1.Y, q2.Z - q1.Z);
		FVector3d Q2 = FVector3d(q3.X - q1.X, q3.Y - q1.Y, q3.Z - q1.Z);

		FVector3d U = Q1.Cross(Q2);

		return U.Length() / 2.0;
	}

	double ReedBeta(const FDynamicMesh3& Mesh, int UVChannel, int Tid)
	{
		return 1 - FUVMetrics::ReedBeta(Mesh, UVChannel, Tid); // Flip the value, so green is 0, or non-distorted
	}

	double Sander(const FDynamicMesh3& Mesh, int UVChannel, int Tid, bool bUseL2)
	{	
			return 2.0 - FMath::Clamp(FUVMetrics::Sander(Mesh, UVChannel, Tid, bUseL2), 1.0, 2.0);
	}

	double TexelDensity(const FDynamicMesh3& Mesh, int UVChannel, int Tid, int32 MapSize, float TargetTexelDensity)
	{
		double Density = FUVMetrics::TexelDensity(Mesh, UVChannel, Tid, MapSize);

		// We're looking to map the density into a divergent map here, with values ranging from -0.5 to 0.5... Additionally we'd like the scaling factor to be equal in both direction.
		// So we're going to ensure that a 50% growth or reduction in size maps the same on the scale. This makes the math slightly more complex than just a relative difference.
		// This represents a mapping of [0.5*D, 1.5*D] where D is target density linearly to [-0.5,0.5]

		double DensityDifference = FMath::Abs(TargetTexelDensity - Density);
		double DensityRatio = DensityDifference < (TargetTexelDensity / 2.0f) ? DensityDifference / TargetTexelDensity : 0.5;
		return DensityRatio * FMath::Sign(TargetTexelDensity - Density);
	}
}



void UUVEditorDistortionVisualization::Initialize()
{
	Settings = NewObject<UUVEditorDistortionVisualizationProperties>(this);
	Settings->WatchProperty(Settings->bVisible, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->Metric, [this](EDistortionMetric) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->MapSize, [this](int32) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->TargetTexelDensity, [this](float) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bCompareToAverageDensity, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bRespectUDIMTextureResolutions, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->PerUDIMTextureResolution,
		                    [this](const TMap<int32, int32>&) { bSettingsModified = true; }, 
		                    [this](const TMap<int32, int32>& A, const TMap<int32, int32>& B) -> bool { return !A.OrderIndependentCompareEqual(B); });
	bSettingsModified = false;

	PerTargetAverageTexelDensity.SetNum(Targets.Num());
}

void UUVEditorDistortionVisualization::ComputeInitialMeshSurfaceAreas()
{
	double WorldSpaceSurfaceArea = 0.0;
	double UVSpaceSurfaceArea = 0.0;
	FVector3d q1, q2, q3;
	FVector2f p1, p2, p3;

	MaximumTileTextureResolution = 1000;
	if (Settings->bRespectUDIMTextureResolutions && !Settings->PerUDIMTextureResolution.IsEmpty())
	{
		MaximumTileTextureResolution = 0;
		for (TPair<int32, int32>& TileResolution : Settings->PerUDIMTextureResolution)
		{
			MaximumTileTextureResolution = FMath::Max(MaximumTileTextureResolution, TileResolution.Value);
		}
	}

	for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
	{
		TMap<int32, double> WorldSpaceSurfaceAreaPerTileResolution;
		TMap<int32, double> UVSpaceSurfaceAreaPerTileResolution;
		TMap<int32, double> AverageDensityPerTileResolution;
		TMap<int32, int32> CountPerTileResolution;

		TObjectPtr<UUVEditorToolMeshInput>& Target = Targets[TargetIndex];
		for (int32 Tid = 0; Tid < Target->AppliedCanonical->TriangleCount(); ++Tid)
		{
			const int32* TileTextureResolution = nullptr;
			if(Settings->bRespectUDIMTextureResolutions)
			{
				TArray<int32> TidsToClassify;
				TidsToClassify.Add(Tid);
				FVector2i UDIMCoords = FDynamicMeshUDIMClassifier::ClassifyTrianglesToUDIM(Target->AppliedCanonical->Attributes()->GetUVLayer(Target->UVLayerIndex), TidsToClassify);
				int32 UDIMBlockIndex = UE::TextureUtilitiesCommon::GetUDIMIndex(UDIMCoords.X, UDIMCoords.Y);
				TileTextureResolution = Settings->PerUDIMTextureResolution.Find(UDIMBlockIndex);
			}
			if (!TileTextureResolution)
			{
				TileTextureResolution = &MaximumTileTextureResolution;
			}

			
			Target->AppliedCanonical->GetTriVertices(Tid, q1, q2, q3);
			Target->AppliedCanonical->Attributes()->GetUVLayer(Target->UVLayerIndex)->GetTriElements(Tid, p1, p2, p3);

			WorldSpaceSurfaceAreaPerTileResolution.FindOrAdd(*TileTextureResolution) += DistortionMetrics::Triangle3DArea(q1,q2,q3);
			UVSpaceSurfaceAreaPerTileResolution.FindOrAdd(*TileTextureResolution) += DistortionMetrics::Triangle2DArea(p1, p2, p3);
			CountPerTileResolution.FindOrAdd(*TileTextureResolution)++;			

			AverageDensityPerTileResolution.FindOrAdd(*TileTextureResolution) += (DistortionMetrics::Triangle2DArea(p1, p2, p3) / DistortionMetrics::Triangle3DArea(q1, q2, q3));			
		}
		TArray<int32> Resolutions;
		WorldSpaceSurfaceAreaPerTileResolution.GenerateKeyArray(Resolutions);
		PerTargetAverageTexelDensity[TargetIndex] = 0.0;
		for (int32 Resolution : Resolutions)
		{
			PerTargetAverageTexelDensity[TargetIndex] += Resolution * FMath::Sqrt(AverageDensityPerTileResolution[Resolution] / CountPerTileResolution[Resolution]);
		}
		PerTargetAverageTexelDensity[TargetIndex] /= Resolutions.Num();
	}
}

void UUVEditorDistortionVisualization::Shutdown()
{
}

void UUVEditorDistortionVisualization::OnTick(float DeltaTime)
{

	if (bSettingsModified)
	{		
		UpdateVisibility();
		bSettingsModified = false;
	}
}

void UUVEditorDistortionVisualization::UpdateVisibility()
{
	if (Settings->bVisible && Settings->Metric == EDistortionMetric::TexelDensity)
	{
		ComputeInitialMeshSurfaceAreas();
	}

	for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
	{
		TObjectPtr<UUVEditorToolMeshInput>& Target = Targets[TargetIndex];

		if (Settings->bVisible)
		{
			ConfigureMeshColorsForTarget(TargetIndex);
		}
		else
		{
			Target->UnwrapPreview->PreviewMesh->ClearTriangleColorFunction(UPreviewMesh::ERenderUpdateMode::FastUpdate);
		}
	}
}

void UUVEditorDistortionVisualization::ConfigureMeshColorsForTarget(int32 TargetIndex)
{
	TObjectPtr<UUVEditorToolMeshInput>& Target = Targets[TargetIndex];

	Target->UnwrapPreview->PreviewMesh->SetTriangleColorFunction([this, &Target, TargetIndex](const FDynamicMesh3* Mesh, int Tid)
	{
		FColor TidDistortionColor;
		Target->AppliedPreview->PreviewMesh->ProcessMesh([this, &TidDistortionColor, Tid, &Target, TargetIndex](const FDynamicMesh3& AppliedMesh)
			{
				TidDistortionColor = GetDistortionColorForTriangle(AppliedMesh, Target->UVLayerIndex, Tid, PerTargetAverageTexelDensity[TargetIndex]);
			});
		return TidDistortionColor;
	}, UPreviewMesh::ERenderUpdateMode::FastUpdate);
}

FColor UUVEditorDistortionVisualization::GetDistortionColorForTriangle(const FDynamicMesh3& Mesh, int UVChannel, int Tid, double TargetAverageTexelDensity)
{
	double MetricValue = 0;
	FColor BackgroundColor = FColor::Black;

	switch (Settings->Metric)
	{
	case EDistortionMetric::ReedBeta:
		MetricValue = DistortionMetrics::ReedBeta(Mesh, UVChannel, Tid);
		BackgroundColor = FUVEditorUXSettings::MakeCividisColorFromScalar(MetricValue);
		break;
	case EDistortionMetric::Sander_L2:
		MetricValue = DistortionMetrics::Sander(Mesh, UVChannel, Tid, true);
		BackgroundColor = FUVEditorUXSettings::MakeCividisColorFromScalar(MetricValue);
		break;
	case EDistortionMetric::Sander_LInf:
		MetricValue = DistortionMetrics::Sander(Mesh, UVChannel, Tid, false);
		BackgroundColor = FUVEditorUXSettings::MakeCividisColorFromScalar(MetricValue);
		break;
	case EDistortionMetric::TexelDensity:
	{
		int32* TileTextureResolution = nullptr;
		if (Settings->bRespectUDIMTextureResolutions)
		{
			TArray<int32> TidsToClassify;
			TidsToClassify.Add(Tid);
			FVector2i UDIMCoords = FDynamicMeshUDIMClassifier::ClassifyTrianglesToUDIM(Mesh.Attributes()->GetUVLayer(UVChannel), TidsToClassify);
			int32 UDIMBlockIndex = UE::TextureUtilitiesCommon::GetUDIMIndex(UDIMCoords.X, UDIMCoords.Y);
			TileTextureResolution = Settings->PerUDIMTextureResolution.Find(UDIMBlockIndex);
		}

		if (Settings->bCompareToAverageDensity)
		{
			MetricValue = DistortionMetrics::TexelDensity(Mesh, UVChannel, Tid, TileTextureResolution ? *TileTextureResolution : MaximumTileTextureResolution /* Use some artifical "pixel density" here to avoid small numbers */, TargetAverageTexelDensity);
		}
		else
		{
			MetricValue = DistortionMetrics::TexelDensity(Mesh, UVChannel, Tid, TileTextureResolution ? *TileTextureResolution : Settings->MapSize, Settings->TargetTexelDensity);
		}
		BackgroundColor = FUVEditorUXSettings::MakeTurboColorFromScalar(MetricValue);
	}
		break;
	default:
		ensure(false);
	}

	
	return BackgroundColor;

}


