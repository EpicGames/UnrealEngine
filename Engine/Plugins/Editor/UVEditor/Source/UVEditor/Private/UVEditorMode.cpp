// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorMode.h"

#include "Algo/AnyOf.h"
#include "Actions/UnsetUVsAction.h"
#include "Actions/UVMakeIslandAction.h"
#include "Actions/UVSplitAction.h"
#include "Actions/UVSeamSewAction.h"
#include "Actions/UVToolAction.h"
#include "AssetEditorModeManager.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "ContextObjects/UVToolViewportButtonsAPI.h"
#include "ContextObjects/UVToolAssetInputsContext.h"
#include "ContextObjectStore.h"
#include "Selection/UVToolSelectionAPI.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "Editor.h"
#include "EditorModelingObjectsCreationAPI.h"
#include "EditorModes.h"
#include "EditorViewportClient.h"
#include "Tools/EdModeInteractiveToolsContext.h" //ToolsContext, EditorInteractiveToolsContext
#include "EngineAnalytics.h"
#include "Framework/Commands/UICommandList.h"
#include "InteractiveTool.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "ModelingToolTargetUtil.h"
#include "PreviewMesh.h"
#include "PreviewScene.h"
#include "TargetInterfaces/AssetBackedTarget.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "ToolSetupUtil.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "ToolTargetManager.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "UVEditor3DViewportMode.h"
#include "UVEditorToolBase.h"
#include "UVEditorBrushSelectTool.h"
#include "UVEditorCommands.h"
#include "UVEditorLayoutTool.h"
#include "UVEditorTransformTool.h"
#include "UVEditorParameterizeMeshTool.h"
#include "UVEditorLayerEditTool.h"
#include "UVEditorSeamTool.h"
#include "UVEditorRecomputeUVsTool.h"
#include "UVEditorUVSnapshotTool.h"
#include "UVSelectTool.h"
#include "UVEditorTexelDensityTool.h"
#include "UVEditorInitializationContext.h"
#include "UVEditorModeToolkit.h"
#include "UVEditorSubsystem.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "UVEditorBackgroundPreview.h"
#include "UVEditorDistortionVisualization.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "UVEditorUXSettings.h"
#include "UDIMUtilities.h"
#include "EditorSupportDelegates.h"
#include "Parameterization/MeshUDIMClassifier.h"
#include "UVEditorLogging.h"
#include "UObject/ObjectSaveContext.h"
#include "Materials/Material.h"
#include "UVEditorUXPropertySets.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorMode)

#define LOCTEXT_NAMESPACE "UUVEditorMode"

using namespace UE::Geometry;

const FEditorModeID UUVEditorMode::EM_UVEditorModeId = TEXT("EM_UVEditorMode");

FDateTime UUVEditorMode::AnalyticsLastStartTimestamp;

namespace UVEditorModeLocals
{
	// The layer we open when we first open the UV editor
	const int32 DefaultUVLayerIndex = 0;

	const FText UVLayerChangeTransactionName = LOCTEXT("UVLayerChangeTransactionName", "Change UV Layer");

	void GetCameraState(const FEditorViewportClient& ViewportClientIn, FViewCameraState& CameraStateOut)
	{
		FViewportCameraTransform ViewTransform = ViewportClientIn.GetViewTransform();
		CameraStateOut.bIsOrthographic = false;
		CameraStateOut.bIsVR = false;
		CameraStateOut.Position = ViewTransform.GetLocation();
		CameraStateOut.HorizontalFOVDegrees = ViewportClientIn.ViewFOV;
		CameraStateOut.AspectRatio = ViewportClientIn.AspectRatio;

		// if using Orbit camera, the rotation in the ViewTransform is not the current camera rotation, it
		// is set to a different rotation based on the Orbit. So we have to convert back to camera rotation.
		FRotator ViewRotation = (ViewportClientIn.bUsingOrbitCamera) ?
			ViewTransform.ComputeOrbitMatrix().InverseFast().Rotator() : ViewTransform.GetRotation();
		CameraStateOut.Orientation = ViewRotation.Quaternion();
	}

	/**
	 * Support for undoing a tool start in such a way that we go back to the mode's default
	 * tool on undo.
	 */
	class FUVEditorBeginToolChange : public FToolCommandChange
	{
	public:
		virtual void Apply(UObject* Object) override
		{
			// Do nothing, since we don't allow a re-do back into a tool
		}

		virtual void Revert(UObject* Object) override
		{
			UUVEditorMode* Mode = Cast<UUVEditorMode>(Object);
			// Don't really need the check for default tool since we theoretically shouldn't
			// be issuing this transaction for starting the default tool, but still...
			if (Mode && !Mode->IsDefaultToolActive())
			{
				Mode->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel);
				Mode->ActivateDefaultTool();
			}
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			// To not be expired, we must be active and in some non-default tool.
			UUVEditorMode* Mode = Cast<UUVEditorMode>(Object);
			return !(Mode && Mode->IsActive() && !Mode->IsDefaultToolActive());
		}

		virtual FString ToString() const override
		{
			return TEXT("UVEditorModeLocals::FUVEditorBeginToolChange");
		}
	};

	/** 
	 * Change for undoing/redoing displayed layer changes.
	 */
	class FInputObjectUVLayerChange : public FToolCommandChange
	{
	public:
		FInputObjectUVLayerChange(int32 AssetIDIn, int32 OldUVLayerIndexIn, int32 NewUVLayerIndexIn)
			: AssetID(AssetIDIn)
			, OldUVLayerIndex(OldUVLayerIndexIn)
			, NewUVLayerIndex(NewUVLayerIndexIn)
		{
		}

		virtual void Apply(UObject* Object) override
		{
			UUVEditorMode* UVEditorMode = Cast<UUVEditorMode>(Object);
			UVEditorMode->ChangeInputObjectLayer(AssetID, NewUVLayerIndex);
		}

		virtual void Revert(UObject* Object) override
		{
			UUVEditorMode* UVEditorMode = Cast<UUVEditorMode>(Object);
			UVEditorMode->ChangeInputObjectLayer(AssetID, OldUVLayerIndex);
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			UUVEditorMode* UVEditorMode = Cast<UUVEditorMode>(Object);
			return !(UVEditorMode && UVEditorMode->IsActive());
		}

		virtual FString ToString() const override
		{
			return TEXT("UVEditorModeLocals::FInputObjectUVLayerChange");
		}

	protected:
		int32 AssetID;
		int32 OldUVLayerIndex;
		int32 NewUVLayerIndex;
	};

	void InitializeAssetNames(const TArray<TObjectPtr<UToolTarget>>& ToolTargets, TArray<FString>& AssetNamesOut)
	{
		AssetNamesOut.Reset(ToolTargets.Num());

		for (const TObjectPtr<UToolTarget>& ToolTarget : ToolTargets)
		{
			AssetNamesOut.Add(UE::ToolTarget::GetHumanReadableName(ToolTarget));
		}
	}
}


namespace UVEditorModeChange
{
	/**
	 * Change for undoing/redoing "Apply" of UV editor changes.
	 */
	class FApplyChangesChange : public FToolCommandChange
	{
	public:
		FApplyChangesChange(TSet<int32> OriginalModifiedAssetIDsIn)
			: OriginalModifiedAssetIDs(OriginalModifiedAssetIDsIn)
		{
		}

		virtual void Apply(UObject* Object) override
		{
			UUVEditorMode* UVEditorMode = Cast<UUVEditorMode>(Object);
			UVEditorMode->ModifiedAssetIDs.Reset();
		}

		virtual void Revert(UObject* Object) override
		{
			UUVEditorMode* UVEditorMode = Cast<UUVEditorMode>(Object);
			UVEditorMode->ModifiedAssetIDs = OriginalModifiedAssetIDs;
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			UUVEditorMode* UVEditorMode = Cast<UUVEditorMode>(Object);
			return !(UVEditorMode && UVEditorMode->IsActive());
		}

		virtual FString ToString() const override
		{
			return TEXT("UVEditorModeLocals::FApplyChangesChange");
		}

	protected:
		TSet<int32> OriginalModifiedAssetIDs;
	};
}

const FToolTargetTypeRequirements& UUVEditorMode::GetToolTargetRequirements()
{
	static const FToolTargetTypeRequirements ToolTargetRequirements =
		FToolTargetTypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshCommitter::StaticClass(),
			UDynamicMeshProvider::StaticClass()
			});
	return ToolTargetRequirements;
}

void UUVEditorUDIMProperties::PostAction(EUVEditorModeActions Action)
{
	if (Action == EUVEditorModeActions::ConfigureUDIMsFromTexture)
	{
		UpdateActiveUDIMsFromTexture();
	}
	if (Action == EUVEditorModeActions::ConfigureUDIMsFromAsset)
	{
		UpdateActiveUDIMsFromAsset();
	}
}

const TArray<FString>& UUVEditorUDIMProperties::GetAssetNames()
{
	return UVAssetNames;
}

int32 UUVEditorUDIMProperties::AssetByIndex() const
{
	return UVAssetNames.Find(UDIMSourceAsset);
}

void UUVEditorUDIMProperties::InitializeAssets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
{
	UVAssetNames.Reset(TargetsIn.Num());

	for (int i = 0; i < TargetsIn.Num(); ++i)
	{
		UVAssetNames.Add(UE::ToolTarget::GetHumanReadableName(TargetsIn[i]->SourceTarget));
	}
}

void UUVEditorUDIMProperties::UpdateActiveUDIMsFromTexture()
{
	ActiveUDIMs.Empty();
	if (UDIMSourceTexture && UDIMSourceTexture->IsCurrentlyVirtualTextured() && UDIMSourceTexture->Source.GetNumBlocks() > 1)
	{
		for (int32 Block = 0; Block < UDIMSourceTexture->Source.GetNumBlocks(); ++Block)
		{
			FTextureSourceBlock SourceBlock;
			UDIMSourceTexture->Source.GetBlock(Block, SourceBlock);

			ActiveUDIMs.Add(FUDIMSpecifier({ UE::TextureUtilitiesCommon::GetUDIMIndex(SourceBlock.BlockX, SourceBlock.BlockY),
				                             SourceBlock.BlockX, SourceBlock.BlockY, FMath::Min(SourceBlock.SizeX, SourceBlock.SizeY) }));
		}
	}
	CheckAndUpdateWatched();

	// Make sure we're updating the viewport immediately.
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

void UUVEditorUDIMProperties::UpdateActiveUDIMsFromAsset()
{
	ActiveUDIMs.Empty();
	int32 AssetID = AssetByIndex();
	ParentMode->PopulateUDIMsByAsset(AssetID, ActiveUDIMs);
	CheckAndUpdateWatched();

	// Make sure we're updating the viewport immediately.
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

UUVEditorMode::UUVEditorMode()
{
	Info = FEditorModeInfo(
		EM_UVEditorModeId,
		LOCTEXT("UVEditorModeName", "UV"),
		FSlateIcon(),
		false);
}

double UUVEditorMode::GetUVMeshScalingFactor() {
	return FUVEditorUXSettings::UVMeshScalingFactor;
}

void UUVEditorMode::Enter()
{
	Super::Enter();

	bPIEModeActive = false;
	if (GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor)
	{
		bPIEModeActive = true;
		SetSimulationWarning(true);
	}

	BeginPIEDelegateHandle = FEditorDelegates::PreBeginPIE.AddLambda([this](bool bSimulating)
	{
		bPIEModeActive = true;
        SetSimulationWarning(true);
	});

	EndPIEDelegateHandle = FEditorDelegates::EndPIE.AddLambda([this](bool bSimulating)
	{
		bPIEModeActive = false;
		ActivateDefaultTool();
		SetSimulationWarning(false);
	});

	CancelPIEDelegateHandle = FEditorDelegates::CancelPIE.AddLambda([this]()
	{
		bPIEModeActive = false;
		ActivateDefaultTool();
		SetSimulationWarning(false);
	});

	PostSaveWorldDelegateHandle = FEditorDelegates::PostSaveWorldWithContext.AddLambda([this](UWorld*, FObjectPostSaveContext)
	{
		ActivateDefaultTool();
	});

	// Our mode needs to get Render and DrawHUD calls, but doesn't implement the legacy interface that
	// causes those to be called. Instead, we'll hook into the tools context's Render and DrawHUD calls.
	GetInteractiveToolsContext()->OnRender.AddUObject(this, &UUVEditorMode::Render);
	GetInteractiveToolsContext()->OnDrawHUD.AddUObject(this, &UUVEditorMode::DrawHUD);

	// We also want the Render and DrawHUD calls from the 3D viewport
	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	UUVEditorInitializationContext* InitContext = ContextStore->FindContext<UUVEditorInitializationContext>();
	UUVToolLivePreviewAPI* LivePreviewAPI = ContextStore->FindContext<UUVToolLivePreviewAPI>();
	if (ensure(InitContext && InitContext->LivePreviewITC.IsValid() && LivePreviewAPI))
	{
		LivePreviewITC = InitContext->LivePreviewITC;
		LivePreviewITC->OnRender.AddWeakLambda(this, [this, LivePreviewAPI](IToolsContextRenderAPI* RenderAPI) 
		{ 
			if (LivePreviewAPI->IsValidLowLevel())
			{
				LivePreviewAPI->OnRender.Broadcast(RenderAPI);
			}

			// This could have been attached to the LivePreviewAPI delegate the way that tools might do it,
			// but might as well do the call ourselves. Same for DrawHUD.
			if (SelectionAPI)
			{
				SelectionAPI->LivePreviewRender(RenderAPI);
			}
		});
		LivePreviewITC->OnDrawHUD.AddWeakLambda(this, [this, LivePreviewAPI](FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
		{
			if (LivePreviewAPI->IsValidLowLevel())
			{
				LivePreviewAPI->OnDrawHUD.Broadcast(Canvas, RenderAPI);
			}

			if (SelectionAPI)
			{
				SelectionAPI->LivePreviewDrawHUD(Canvas, RenderAPI);
			}
		});

		LivePreviewToolkitCommands = InitContext->LivePreviewToolkitCommands;
	}

	InitializeModeContexts();
	InitializeTargets();

	// By default combined transform gizmos now hide some of their elements based on what gizmo mode is chosen in
	// the viewport. The UV editor doesn't currently have that selector, so we don't want to try to use it. If we
	// add that selector, we'll need to remove this call.
	GetInteractiveToolsContext()->SetForceCombinedGizmoMode(true);

	BackgroundVisualization = NewObject<UUVEditorBackgroundPreview>(this);
	BackgroundVisualization->CreateInWorld(GetWorld(), FTransform::Identity);

	BackgroundVisualization->Settings->WatchProperty(BackgroundVisualization->Settings->bVisible, 
		[this](bool IsVisible) {
			UpdateTriangleMaterialBasedOnDisplaySettings();			
		});

	BackgroundVisualization->OnBackgroundMaterialChange.AddWeakLambda(this,
		[this](TObjectPtr<UMaterialInstanceDynamic> MaterialInstance) {
			UpdatePreviewMaterialBasedOnBackground();
		});

	PropertyObjectsToTick.Add(BackgroundVisualization->Settings);

	DistortionVisualization = NewObject<UUVEditorDistortionVisualization>(this);
	DistortionVisualization->Targets = ToolInputObjects;
	DistortionVisualization->Initialize();


	DistortionVisualization->Settings->WatchProperty(DistortionVisualization->Settings->bVisible,
		[this](bool IsVisible) {
			UpdateTriangleMaterialBasedOnDisplaySettings();			
		});

	PropertyObjectsToTick.Add(DistortionVisualization->Settings);

	UVEditorGridProperties = NewObject <UUVEditorGridProperties>(this);

	UVEditorGridProperties->WatchProperty(UVEditorGridProperties->bDrawGrid,
		[this](bool bDrawGrid) {
			UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
			UUVTool2DViewportAPI* UVTool2DViewportAPI = ContextStore->FindContext<UUVTool2DViewportAPI>();
			if (UVTool2DViewportAPI)
			{
				UVTool2DViewportAPI->SetDrawGrid(bDrawGrid, true);
			}		
		});

	UVEditorGridProperties->WatchProperty(UVEditorGridProperties->bDrawRulers,
		[this](bool bDrawRulers) {
			UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
			UUVTool2DViewportAPI* UVTool2DViewportAPI = ContextStore->FindContext<UUVTool2DViewportAPI>();
			if (UVTool2DViewportAPI)
			{
				UVTool2DViewportAPI->SetDrawRulers(bDrawRulers, true);
			}
		});

	PropertyObjectsToTick.Add(UVEditorGridProperties);
	
	UVEditorUnwrappedUXProperties = NewObject<UUVEditorUnwrappedUXProperties>(this);
	
	UVEditorUnwrappedUXProperties->BoundaryLineColors.Reserve(ToolInputObjects.Num());
	// initialize display menu properties
	for (int ObjIndex = 0; ObjIndex < ToolInputObjects.Num(); ObjIndex++)
	{
		UVEditorUnwrappedUXProperties->BoundaryLineColors.Add(ToolInputObjects[ObjIndex]->WireframeDisplay->Settings->BoundaryEdgeColor);
	}
	UVEditorUnwrappedUXProperties->BoundaryLineThickness = FUVEditorUXSettings::BoundaryEdgeThickness;
	UVEditorUnwrappedUXProperties->WireframeThickness = FUVEditorUXSettings::WireframeThickness;

	UVEditorUnwrappedUXProperties->WatchProperty(UVEditorUnwrappedUXProperties->BoundaryLineColors,
		[this](const TArray<FColor>& EdgeColors)
		{
			for (int32 AssetID = 0; AssetID < ToolInputObjects.Num(); ++AssetID)
			{
				ToolInputObjects[AssetID]->WireframeDisplay->Settings->BoundaryEdgeColor = EdgeColors[AssetID];
			}
		});
	UVEditorUnwrappedUXProperties->WatchProperty(UVEditorUnwrappedUXProperties->BoundaryLineThickness,
		[this](const float Thickness)
		{
			for (int32 AssetID = 0; AssetID < ToolInputObjects.Num(); ++AssetID)
			{
				ToolInputObjects[AssetID]->WireframeDisplay->WireframeComponent->BoundaryEdgeThickness = Thickness;
				ToolInputObjects[AssetID]->WireframeDisplay->WireframeComponent->UpdateWireframe();
			}
		});
	UVEditorUnwrappedUXProperties->WatchProperty(UVEditorUnwrappedUXProperties->WireframeThickness,
		[this](const float WireframeThickness)
		{
			for (int32 AssetID = 0; AssetID < ToolInputObjects.Num(); ++AssetID)
			{
				ToolInputObjects[AssetID]->WireframeDisplay->WireframeComponent->WireframeThickness = WireframeThickness;
				ToolInputObjects[AssetID]->WireframeDisplay->WireframeComponent->UpdateWireframe();
			}
		});
	PropertyObjectsToTick.Add(UVEditorUnwrappedUXProperties);

	UVEditorLivePreviewUXProperties = NewObject<UUVEditorLivePreviewUXProperties>(this);

	// initialize display menu properties
	UVEditorLivePreviewUXProperties->SelectionColor = FUVEditorUXSettings::SelectionTriangleWireframeColor;
	UVEditorLivePreviewUXProperties->SelectionLineThickness = FUVEditorUXSettings::LivePreviewHighlightThickness;
	UVEditorLivePreviewUXProperties->SelectionPointSize = FUVEditorUXSettings::LivePreviewHighlightPointSize;
	
	UVEditorLivePreviewUXProperties->WatchProperty(UVEditorLivePreviewUXProperties->SelectionColor,
		[this](const FColor InSelectionLineColor)
		{
			UUVToolSelectionAPI::FLivePreviewSelectionUXSettings Settings = UUVToolSelectionAPI::FLivePreviewSelectionUXSettings();
			Settings.SelectionColor = InSelectionLineColor;

			SelectionAPI->SetLivePreviewSelectionUXSettings(Settings);
			SelectionAPI->RebuildAppliedPreviewHighlight();
		});
	UVEditorLivePreviewUXProperties->WatchProperty(UVEditorLivePreviewUXProperties->SelectionLineThickness,
		[this](const float InLineThickness)
		{
			UUVToolSelectionAPI::FLivePreviewSelectionUXSettings Settings = UUVToolSelectionAPI::FLivePreviewSelectionUXSettings();
			Settings.LineThickness = InLineThickness;
			
			SelectionAPI->SetLivePreviewSelectionUXSettings(Settings);
			SelectionAPI->RebuildAppliedPreviewHighlight();
		});
	UVEditorLivePreviewUXProperties->WatchProperty(UVEditorLivePreviewUXProperties->SelectionPointSize,
		[this](const float InPointSize)
		{
			UUVToolSelectionAPI::FLivePreviewSelectionUXSettings Settings = UUVToolSelectionAPI::FLivePreviewSelectionUXSettings();
			Settings.PointSize = InPointSize;
			
			SelectionAPI->SetLivePreviewSelectionUXSettings(Settings);
			SelectionAPI->RebuildAppliedPreviewHighlight();
		});

	PropertyObjectsToTick.Add(UVEditorLivePreviewUXProperties);

	UVEditorUDIMProperties = NewObject< UUVEditorUDIMProperties >(this);
	UVEditorUDIMProperties->Initialize(this);
	UVEditorUDIMProperties->InitializeAssets(ToolInputObjects);
	UDIMsChangedWatcherId = UVEditorUDIMProperties->WatchProperty(UVEditorUDIMProperties->ActiveUDIMs,
		[this](const TArray<FUDIMSpecifier>& ActiveUDIMs) {
			UpdateActiveUDIMs();
		},
		[](const TArray<FUDIMSpecifier>& ActiveUDIMsOld, const TArray<FUDIMSpecifier>& ActiveUDIMsNew) {
			TSet<FUDIMSpecifier> NewCopy(ActiveUDIMsNew);
			TSet<FUDIMSpecifier> OldCopy(ActiveUDIMsOld);

			if (OldCopy.Num() != NewCopy.Num())
			{
				return true;
			}
			TSet<FUDIMSpecifier> Test = OldCopy.Union(NewCopy);
			if (Test.Num() != OldCopy.Num())
			{
				return true;
			}
			return false;
		});
	PropertyObjectsToTick.Add(UVEditorUDIMProperties);

	RegisterTools();
	RegisterActions();
	ActivateDefaultTool();

	// do any toolkit UI initialization that depends on the mode setup above
	if (Toolkit.IsValid())
	{
		FUVEditorModeToolkit* UVModeToolkit = (FUVEditorModeToolkit*)Toolkit.Get();
		UVModeToolkit->InitializeAfterModeSetup();
	}

	if (FEngineAnalytics::IsAvailable())
	{
		AnalyticsLastStartTimestamp = FDateTime::UtcNow();

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), AnalyticsLastStartTimestamp.ToString()));

		FEngineAnalytics::GetProvider().RecordEvent(UVEditorAnalytics::UVEditorAnalyticsEventName(TEXT("Enter")), Attributes);
	}

	bIsActive = true;
}

void UUVEditorMode::RegisterTools()
{
	const FUVEditorCommands& CommandInfos = FUVEditorCommands::Get();

	// The Select Tool is a bit different because it runs between other tools. It doesn't have a button and is not
	// hotkeyable, hence it doesn't have a command item. Instead we register it directly with the tool manager.
	UUVSelectToolBuilder* UVSelectToolBuilder = NewObject<UUVSelectToolBuilder>();
	UVSelectToolBuilder->Targets = &ToolInputObjects;
	DefaultToolIdentifier = TEXT("BeginSelectTool");
	GetToolManager()->RegisterToolType(DefaultToolIdentifier, UVSelectToolBuilder);
	ToolsThatAllowActions.Add(DefaultToolIdentifier);

	// Note that the identifiers below need to match the command names so that the tool icons can 
	// be easily retrieved from the active tool name in UVEditorModeToolkit::OnToolStarted. Otherwise
	// we would need to keep some other mapping from tool identifier to tool icon.

	UUVEditorLayoutToolBuilder* UVEditorLayoutToolBuilder = NewObject<UUVEditorLayoutToolBuilder>();
	UVEditorLayoutToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginLayoutTool, TEXT("BeginLayoutTool"), UVEditorLayoutToolBuilder);

	UUVEditorTransformToolBuilder* UVEditorTransformToolBuilder = NewObject<UUVEditorTransformToolBuilder>();
	UVEditorTransformToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginTransformTool, TEXT("BeginTransformTool"), UVEditorTransformToolBuilder);

	UUVEditorAlignToolBuilder* UVEditorAlignToolBuilder = NewObject<UUVEditorAlignToolBuilder>();
	UVEditorAlignToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginAlignTool, TEXT("BeginAlignTool"), UVEditorAlignToolBuilder);

	UUVEditorDistributeToolBuilder* UVEditorDistributeToolBuilder = NewObject<UUVEditorDistributeToolBuilder>();
	UVEditorDistributeToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginDistributeTool, TEXT("BeginDistributeTool"), UVEditorDistributeToolBuilder);

	UUVEditorTexelDensityToolBuilder* UVEditorTexelDensityToolBuilder = NewObject<UUVEditorTexelDensityToolBuilder>();
	UVEditorTexelDensityToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginTexelDensityTool, TEXT("BeginTexelDensityTool"), UVEditorTexelDensityToolBuilder);

	UUVEditorParameterizeMeshToolBuilder* UVEditorParameterizeMeshToolBuilder = NewObject<UUVEditorParameterizeMeshToolBuilder>();
	UVEditorParameterizeMeshToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginParameterizeMeshTool, TEXT("BeginParameterizeMeshTool"), UVEditorParameterizeMeshToolBuilder);

	UUVEditorChannelEditToolBuilder* UVEditorChannelEditToolBuilder = NewObject<UUVEditorChannelEditToolBuilder>();
	UVEditorChannelEditToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginChannelEditTool, TEXT("BeginChannelEditTool"), UVEditorChannelEditToolBuilder);

	UUVEditorSeamToolBuilder* UVEditorSeamToolBuilder = NewObject<UUVEditorSeamToolBuilder>();
	UVEditorSeamToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginSeamTool, TEXT("BeginSeamTool"), UVEditorSeamToolBuilder);

	UUVEditorRecomputeUVsToolBuilder* UVEditorRecomputeUVsToolBuilder = NewObject<UUVEditorRecomputeUVsToolBuilder>();
	UVEditorRecomputeUVsToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginRecomputeUVsTool, TEXT("BeginRecomputeUVsTool"), UVEditorRecomputeUVsToolBuilder);

	FString BrushToolIdentifier = TEXT("BeginBrushSelectTool");
	UGenericUVEditorToolBuilder* BrushToolBuilder = NewObject<UGenericUVEditorToolBuilder>();
	BrushToolBuilder->Initialize(ToolInputObjects, UUVEditorBrushSelectTool::StaticClass());
	RegisterTool(CommandInfos.BeginBrushSelectTool, BrushToolIdentifier, BrushToolBuilder);
	ToolsThatAllowActions.Add(BrushToolIdentifier);
	UUVEditorUVSnapshotToolBuilder* UVEditorUVSnapshotToolBuilder = NewObject<UUVEditorUVSnapshotToolBuilder>();
	UVEditorUVSnapshotToolBuilder->Targets = &ToolInputObjects;
	RegisterTool(CommandInfos.BeginUVSnapshotTool, TEXT("BeginUVSnapshotTool"), UVEditorUVSnapshotToolBuilder);
}

void UUVEditorMode::RegisterActions()
{
	using namespace UVEditorModeLocals;

	const FUVEditorCommands& CommandInfos = FUVEditorCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	auto PrepAction = [this, CommandList](TSharedPtr<const FUICommandInfo> CommandInfo, UUVToolAction* Action)
	{
		Action->Setup(GetToolManager());
		CommandList->MapAction(CommandInfo,
			FExecuteAction::CreateWeakLambda(Action, [this, Action]() 
			{
				// If we're activating from some other selection tool, go ahead and switch to the regular one
				if (!IsDefaultToolActive())
				{
					ActivateDefaultTool();
				}

				Action->ExecuteAction();
				for (const FUVToolSelection& Selection : SelectionAPI->GetSelections())
				{
					// The actions are expected to have left the selection in a valid state. If this is not
					// the case, we'll clear the selection here, which is still not a proper solution because
					// undoing the transaction will still put things into an invalid state (but our Select Tool
					// should be robust to that in its OnSelectionChanged).
					if (!ensure(Selection.Target.IsValid() 
						&& Selection.AreElementsPresentInMesh(*Selection.Target->UnwrapCanonical)))
					{
						SelectionAPI->ClearSelections(true, true);
						SelectionAPI->ClearHighlight();
						break;
					}
				}
			}),
			FCanExecuteAction::CreateWeakLambda(Action, [Action, this]() 
			{
				// We could make some generic system for determining which tools are safe to call one-off actions from,
				//  but it's unclear that it's worth it while we have so few.
				auto DoesCurrentToolAllowActions = [this]()
				{
					if (UInteractiveToolsContext* ToolsContext = GetInteractiveToolsContext())
					{
						return ToolsThatAllowActions.Contains(ToolsContext->GetActiveToolName(EToolSide::Mouse));
					}
					return false;
				};
				return DoesCurrentToolAllowActions() && Action->CanExecuteAction();
			}));
		RegisteredActions.Add(Action);
	};
	PrepAction(CommandInfos.SewAction, NewObject<UUVSeamSewAction>());
	PrepAction(CommandInfos.SplitAction, NewObject<UUVSplitAction>());
	PrepAction(CommandInfos.MakeIslandAction, NewObject<UUVMakeIslandAction>());
	PrepAction(CommandInfos.UnsetUVsAction, NewObject<UUnsetUVsAction>());
}

bool UUVEditorMode::ShouldToolStartBeAllowed(const FString& ToolIdentifier) const
{
	if (bPIEModeActive)
	{
		return false;
	}

	// For now we've decided to disallow switch-away on accept/cancel tools in the UV editor.
	if (GetInteractiveToolsContext()->ActiveToolHasAccept())
	{
		return false;
	}
	
	return Super::ShouldToolStartBeAllowed(ToolIdentifier);
}

void UUVEditorMode::CreateToolkit()
{
	Toolkit = MakeShared<FUVEditorModeToolkit>();
}

void UUVEditorMode::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	using namespace UVEditorModeLocals;

	FUVEditorToolActionCommands::UpdateToolCommandBinding(Tool, Toolkit->GetToolkitCommands(), false);
	if (LivePreviewToolkitCommands.IsValid())
	{
		FUVEditorToolActionCommands::UpdateToolCommandBinding(Tool, LivePreviewToolkitCommands.Pin(), false);
	}

	FText TransactionName = LOCTEXT("ActivateTool", "Activate Tool");
	
	if (!IsDefaultToolActive())
	{
		GetInteractiveToolsContext()->GetTransactionAPI()->BeginUndoTransaction(TransactionName);
	
		// If a tool doesn't support selection, we can't be certain that it won't put the meshes
		// in a state where the selection refers to invalid elements.
		IUVToolSupportsSelection* SelectionTool = Cast<IUVToolSupportsSelection>(Tool);
		if (!SelectionTool)
		{
			SelectionAPI->BeginChange();
			SelectionAPI->ClearSelections(false, false); 
			SelectionAPI->ClearUnsetElementAppliedMeshSelections(false, false);
			SelectionAPI->EndChangeAndEmitIfModified(true); // broadcast, emit
			SelectionAPI->ClearHighlight();
		}
		else
		{
			if (!SelectionTool->SupportsUnsetElementAppliedMeshSelections())
			{
				SelectionAPI->BeginChange();				
				SelectionAPI->ClearUnsetElementAppliedMeshSelections(false, false);
				SelectionAPI->EndChangeAndEmitIfModified(true); // broadcast, emit
				SelectionAPI->ClearHighlight(false, true); // Clear and rebuild applied highlight to account for clearing unset selections
				SelectionAPI->RebuildAppliedPreviewHighlight();
			}
		}
	
		GetInteractiveToolsContext()->GetTransactionAPI()->AppendChange(
			this, MakeUnique<FUVEditorBeginToolChange>(), TransactionName);

		GetInteractiveToolsContext()->GetTransactionAPI()->EndUndoTransaction();
	}

	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	UToolsContextCursorAPI* ToolsContextCursorAPI = ContextStore->FindContext<UToolsContextCursorAPI>();
	if (ToolsContextCursorAPI)
	{
		ToolsContextCursorAPI->ClearCursorOverride();		
	}

}

void UUVEditorMode::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FUVEditorToolActionCommands::UpdateToolCommandBinding(Tool, Toolkit->GetToolkitCommands(), true);
	if (LivePreviewToolkitCommands.IsValid())
	{
		FUVEditorToolActionCommands::UpdateToolCommandBinding(Tool, LivePreviewToolkitCommands.Pin(), true);
	}

	for (TWeakObjectPtr<UUVToolContextObject> Context : ContextsToUpdateOnToolEnd)
	{
		if (ensure(Context.IsValid()))
		{
			Context->OnToolEnded(Tool);
		}
	}

	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	UToolsContextCursorAPI* ToolsContextCursorAPI = ContextStore->FindContext<UToolsContextCursorAPI>();
	if (ToolsContextCursorAPI)
	{
		ToolsContextCursorAPI->ClearCursorOverride();
	}

}

UObject* UUVEditorMode::GetBackgroundSettingsObject()
{
	if (BackgroundVisualization)
	{
		return BackgroundVisualization->Settings;
	}
	return nullptr;
}

UObject* UUVEditorMode::GetDistortionVisualsSettingsObject()
{
	if (DistortionVisualization)
	{
		return DistortionVisualization->Settings;
	}
	return nullptr;
}

UObject* UUVEditorMode::GetGridSettingsObject()
{
	if (UVEditorGridProperties)
	{
		return UVEditorGridProperties;
	}
	return nullptr;
}

UObject* UUVEditorMode::GetUnwrappedUXSettingsObject() const
{
	return UVEditorUnwrappedUXProperties.Get();
}

UObject* UUVEditorMode::GetLivePreviewUXSettingsObject() const
{
	return UVEditorLivePreviewUXProperties.Get();
}

UObject* UUVEditorMode::GetUDIMSettingsObject()
{
	if (UVEditorUDIMProperties)
	{
		return UVEditorUDIMProperties;
	}
	return nullptr;
}

UObject* UUVEditorMode::GetToolDisplaySettingsObject()
{
	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	UUVEditorToolPropertiesAPI* ToolPropertiesAPI = ContextStore->FindContext<UUVEditorToolPropertiesAPI>();

	if (ToolPropertiesAPI)
	{
		return ToolPropertiesAPI->GetToolDisplayProperties();
	}
	return nullptr;
}


void UUVEditorMode::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionAPI)
	{
		SelectionAPI->Render(RenderAPI);
	}
}

void UUVEditorMode::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionAPI)
	{
		SelectionAPI->DrawHUD(Canvas, RenderAPI);
	}
}

void UUVEditorMode::ActivateDefaultTool()
{
	if (!bPIEModeActive)
	{
		GetInteractiveToolsContext()->StartTool(DefaultToolIdentifier);
	}
}

bool UUVEditorMode::IsDefaultToolActive()
{
	return GetInteractiveToolsContext() && GetInteractiveToolsContext()->IsToolActive(EToolSide::Mouse, DefaultToolIdentifier);
}

void UUVEditorMode::BindCommands()
{
	const FUVEditorCommands& CommandInfos = FUVEditorCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	// Hookup Background toggle command
	CommandList->MapAction(CommandInfos.ToggleBackground, FExecuteAction::CreateLambda([this]() {
		BackgroundVisualization->Settings->bVisible = !BackgroundVisualization->Settings->bVisible;
	}));

	// Hook up to Enter/Esc key presses
	CommandList->MapAction(
		CommandInfos.AcceptOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() {
			// Give the tool a chance to take the nested accept first
			if (GetToolManager()->HasAnyActiveTool())
			{
				UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
				IInteractiveToolNestedAcceptCancelAPI* AcceptAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
				if (AcceptAPI && AcceptAPI->SupportsNestedAcceptCommand() && AcceptAPI->CanCurrentlyNestedAccept())
				{
					if (AcceptAPI->ExecuteNestedAcceptCommand())
					{
						return;
					}
				}
			}
			GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept); 
			ActivateDefaultTool();
			}),
		FCanExecuteAction::CreateLambda([this]() {
			if (GetInteractiveToolsContext()->CanAcceptActiveTool() || GetInteractiveToolsContext()->CanCompleteActiveTool())
			{
				return true;
			}
			// If we can't currently accept, may still be able to pass down to tool
			UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
			IInteractiveToolNestedAcceptCancelAPI* AcceptAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
			return AcceptAPI && AcceptAPI->SupportsNestedAcceptCommand() && AcceptAPI->CanCurrentlyNestedAccept(); 
			}),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() {return GetInteractiveToolsContext()->ActiveToolHasAccept(); }),
		EUIActionRepeatMode::RepeatDisabled
	);

	CommandList->MapAction(
		CommandInfos.CancelOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { 
			// Give the tool a chance to take the nested cancel first
			if (GetToolManager()->HasAnyActiveTool())
			{
				UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
				IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
				if (CancelAPI && CancelAPI->SupportsNestedCancelCommand() && CancelAPI->CanCurrentlyNestedCancel())
				{
					if (CancelAPI->ExecuteNestedCancelCommand())
					{
						return;
					}
				}
			}

			GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel);
			ActivateDefaultTool();
			}),
		FCanExecuteAction::CreateLambda([this]() { 
			if (GetInteractiveToolsContext()->CanCancelActiveTool() || GetInteractiveToolsContext()->CanCompleteActiveTool())
			{
				return true;
			}
			// If we can't currently cancel, may still be able to pass down to tool
			UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
			IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
			return CancelAPI && CancelAPI->SupportsNestedCancelCommand() && CancelAPI->CanCurrentlyNestedCancel();
			}),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() {return GetInteractiveToolsContext()->ActiveToolHasAccept(); }),
		EUIActionRepeatMode::RepeatDisabled
	);

	CommandList->MapAction(
		CommandInfos.SelectAll,
		FExecuteAction::CreateLambda([this]() {

			if (IsDefaultToolActive())
			{
				UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
				UUVSelectTool* SelectTool = Cast<UUVSelectTool>(Tool);
				check(SelectTool);
				SelectTool->SelectAll();
			}
		}),
		FCanExecuteAction::CreateLambda([this]() { 
			return IsDefaultToolActive();
		}));

	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	UUVToolViewportButtonsAPI* UVToolViewportButtonsAPI = ContextStore->FindContext<UUVToolViewportButtonsAPI>();
	if (UVToolViewportButtonsAPI)
	{
		UVToolViewportButtonsAPI->OnInitiateFocusCameraOnSelection.AddUObject(this, &UUVEditorMode::FocusLivePreviewCameraOnSelection);
	}
}

void UUVEditorMode::Exit()
{
	if (FEngineAnalytics::IsAvailable())
	{
		const FDateTime Now = FDateTime::UtcNow();
		const FTimespan ModeUsageDuration = Now - AnalyticsLastStartTimestamp;

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), Now.ToString()));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Duration.Seconds"), static_cast<float>(ModeUsageDuration.GetTotalSeconds())));

		FEngineAnalytics::GetProvider().RecordEvent(UVEditorAnalytics::UVEditorAnalyticsEventName(TEXT("Exit")), Attributes);
	}

	// ToolsContext->EndTool only shuts the tool on the next tick, and ToolsContext->DeactivateActiveTool is
	// inaccessible, so we end up having to do this to force the shutdown right now.
	GetToolManager()->DeactivateTool(EToolSide::Mouse, EToolShutdownType::Cancel);

	for (UUVToolAction* Action : RegisteredActions)
	{
		Action->Shutdown();
	}
	RegisteredActions.Reset();

	for (const TObjectPtr<UUVEditorToolMeshInput>& ToolInput : ToolInputObjects)
	{
		ToolInput->Shutdown();
	}
	ToolInputObjects.Reset();
	WireframesToTick.Reset();
	OriginalObjectsToEdit.Reset();
	
	for (const TObjectPtr<UMeshOpPreviewWithBackgroundCompute>& Preview : AppliedPreviews)
	{
		Preview->Shutdown();
	}
	AppliedPreviews.Reset();
	AppliedCanonicalMeshes.Reset();
	ToolTargets.Reset();

	if (BackgroundVisualization)
	{
		BackgroundVisualization->Disconnect();
		BackgroundVisualization = nullptr;
	}

	DistortionVisualization = nullptr;

	PropertyObjectsToTick.Empty();
	LivePreviewWorld = nullptr;

	bIsActive = false;

	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	for (TWeakObjectPtr<UUVToolContextObject> Context : ContextsToShutdown)
	{
		if (ensure(Context.IsValid()))
		{
			Context->Shutdown();
			ContextStore->RemoveContextObject(Context.Get());
		}
	}

	GetInteractiveToolsContext()->OnRender.RemoveAll(this);
	GetInteractiveToolsContext()->OnDrawHUD.RemoveAll(this);
	if (LivePreviewITC.IsValid())
	{
		LivePreviewITC->OnRender.RemoveAll(this);
		LivePreviewITC->OnDrawHUD.RemoveAll(this);
	}

	FEditorDelegates::PreBeginPIE.Remove(BeginPIEDelegateHandle);
	FEditorDelegates::EndPIE.Remove(EndPIEDelegateHandle);
	FEditorDelegates::CancelPIE.Remove(CancelPIEDelegateHandle);
	FEditorDelegates::PostSaveWorldWithContext.Remove(PostSaveWorldDelegateHandle);

	Super::Exit();
}

void UUVEditorMode::InitializeAssetEditorContexts(UContextObjectStore& ContextStore,
	const TArray<TObjectPtr<UObject>>& AssetsIn, const TArray<FTransform>& TransformsIn,
	FEditorViewportClient& LivePreviewViewportClient, FAssetEditorModeManager& LivePreviewModeManager,
	UUVToolViewportButtonsAPI& ViewportButtonsAPI, UUVTool2DViewportAPI& UVTool2DViewportAPI)
{
	using namespace UVEditorModeLocals;

	UUVToolAssetInputsContext* AssetInputsContext = ContextStore.FindContext<UUVToolAssetInputsContext>();
	if (!AssetInputsContext)
	{
		AssetInputsContext = NewObject<UUVToolAssetInputsContext>();
		AssetInputsContext->Initialize(AssetsIn, TransformsIn);
		ContextStore.AddContextObject(AssetInputsContext);
	}

	UEditorModelingObjectsCreationAPI* ModelingObjectsCreationAPI = ContextStore.FindContext<UEditorModelingObjectsCreationAPI>();
	if (!ModelingObjectsCreationAPI)
	{
		ModelingObjectsCreationAPI = NewObject<UEditorModelingObjectsCreationAPI>();
		ContextStore.AddContextObject(ModelingObjectsCreationAPI);
	}
	
	UToolsContextCursorAPI* LivePreviewToolsContextCursorAPI = LivePreviewModeManager.GetInteractiveToolsContext()->ContextObjectStore->FindContext<UToolsContextCursorAPI>();
	UUVToolLivePreviewAPI* LivePreviewAPI = ContextStore.FindContext<UUVToolLivePreviewAPI>();
	if (!LivePreviewAPI)
	{
		LivePreviewAPI = NewObject<UUVToolLivePreviewAPI>();
		LivePreviewAPI->Initialize(
			LivePreviewModeManager.GetPreviewScene()->GetWorld(),
			LivePreviewModeManager.GetInteractiveToolsContext()->InputRouter,
			[LivePreviewViewportClientPtr = &LivePreviewViewportClient](FViewCameraState& CameraStateOut) {
				GetCameraState(*LivePreviewViewportClientPtr, CameraStateOut);
			},
			[LivePreviewViewportClientPtr = &LivePreviewViewportClient](const FAxisAlignedBox3d& BoundingBox) {
				// We check for the Viewport here because it might not be open at the time this
				// method is called, e.g. during startup with an initially closed tab. And since
				// the FocusViewportOnBox method doesn't check internally that the Viewport is
				// available, this can crash.
				if (LivePreviewViewportClientPtr && LivePreviewViewportClientPtr->Viewport)
				{
					LivePreviewViewportClientPtr->FocusViewportOnBox((FBox)BoundingBox, true);
				}
			},
			[LivePreviewToolsContextCursorAPI](const EMouseCursor::Type Cursor, bool bEnableOverride)
			{
				if(bEnableOverride)
				{
					LivePreviewToolsContextCursorAPI->SetCursorOverride(Cursor);
				}
				else
				{
					LivePreviewToolsContextCursorAPI->ClearCursorOverride();
				}
			},
			LivePreviewModeManager.GetInteractiveToolsContext()->GizmoManager);
		ContextStore.AddContextObject(LivePreviewAPI);
	}

	// Prep the editor-only context that we use to pass things to the mode.
	if (!ContextStore.FindContext<UUVEditorInitializationContext>())
	{
		UUVEditorInitializationContext* InitContext = NewObject<UUVEditorInitializationContext>();
		InitContext->LivePreviewITC = Cast<UEditorInteractiveToolsContext>(LivePreviewModeManager.GetInteractiveToolsContext());

		LivePreviewModeManager.ActivateMode(UUVEditor3DViewportMode::EM_ModeID);
		UEdMode* LivePreviewDefaultMode = LivePreviewModeManager.GetActiveScriptableMode(UUVEditor3DViewportMode::EM_ModeID);
		if (ensure(LivePreviewDefaultMode))
		{
			TSharedPtr<FModeToolkit> Toolkit = LivePreviewDefaultMode->GetToolkit().Pin();
			if (ensure(Toolkit.IsValid()))
			{
				InitContext->LivePreviewToolkitCommands = Toolkit->GetToolkitCommands();
			}
		}

		ContextStore.AddContextObject(InitContext);
	}

	if (!ContextStore.FindContext<UUVToolViewportButtonsAPI>())
	{
		ContextStore.AddContextObject(&ViewportButtonsAPI);
	}

	if (!ContextStore.FindContext<UUVTool2DViewportAPI>())
	{
		ContextStore.AddContextObject(&UVTool2DViewportAPI);
	}
}

void UUVEditorMode::InitializeModeContexts()
{
	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();

	UUVToolAssetInputsContext* AssetInputsContext = ContextStore->FindContext<UUVToolAssetInputsContext>();
	check(AssetInputsContext);
	AssetInputsContext->GetAssetInputs(OriginalObjectsToEdit, Transforms);
	ContextsToUpdateOnToolEnd.Add(AssetInputsContext);

	UUVToolLivePreviewAPI* LivePreviewAPI = ContextStore->FindContext<UUVToolLivePreviewAPI>();
	check(LivePreviewAPI);
	LivePreviewWorld = LivePreviewAPI->GetLivePreviewWorld();
	ContextsToUpdateOnToolEnd.Add(LivePreviewAPI);

	UUVToolViewportButtonsAPI* ViewportButtonsAPI = ContextStore->FindContext<UUVToolViewportButtonsAPI>();
	check(ViewportButtonsAPI);
	ContextsToUpdateOnToolEnd.Add(ViewportButtonsAPI);

	UUVTool2DViewportAPI* UVTool2DViewportAPI = ContextStore->FindContext<UUVTool2DViewportAPI>();
	check(UVTool2DViewportAPI);
	ContextsToUpdateOnToolEnd.Add(UVTool2DViewportAPI);

	// Helper function for adding contexts that our mode creates itself, rather than getting from
	// the asset editor.
	auto AddContextObject = [this, ContextStore](UUVToolContextObject* Object)
	{
		if (ensure(ContextStore->AddContextObject(Object)))
		{
			ContextsToShutdown.Add(Object);
		}
		ContextsToUpdateOnToolEnd.Add(Object);
	};

	UUVToolEmitChangeAPI* EmitChangeAPI = NewObject<UUVToolEmitChangeAPI>();
	EmitChangeAPI = NewObject<UUVToolEmitChangeAPI>();
	EmitChangeAPI->Initialize(GetInteractiveToolsContext()->ToolManager);
	AddContextObject(EmitChangeAPI);

	UUVToolAssetAndChannelAPI* AssetAndLayerAPI = NewObject<UUVToolAssetAndChannelAPI>();
	AssetAndLayerAPI->RequestChannelVisibilityChangeFunc = [this](const TArray<int32>& LayerPerAsset, bool bEmitUndoTransaction) {
		SetDisplayedUVChannels(LayerPerAsset, bEmitUndoTransaction);
	};
	AssetAndLayerAPI->NotifyOfAssetChannelCountChangeFunc = [this](int32 AssetID) {
		// Don't currently need to do anything because the layer selection menu gets populated
		// from scratch each time that it's opened.
	};
	AssetAndLayerAPI->GetCurrentChannelVisibilityFunc = [this]() {
		TArray<int32> VisibleLayers;
		VisibleLayers.SetNum(ToolTargets.Num());
		for (int32 AssetID = 0; AssetID < ToolTargets.Num(); ++AssetID)
		{
			VisibleLayers[AssetID] = GetDisplayedChannel(AssetID);
		}
		return VisibleLayers;
	};
	AddContextObject(AssetAndLayerAPI);

	SelectionAPI = NewObject<UUVToolSelectionAPI>();
	SelectionAPI->Initialize(GetToolManager(), GetWorld(), 
		GetInteractiveToolsContext()->InputRouter, LivePreviewAPI, EmitChangeAPI);
	AddContextObject(SelectionAPI);

	UUVEditorToolPropertiesAPI* UVEditorToolPropertiesAPI = NewObject<UUVEditorToolPropertiesAPI>();
	AddContextObject(UVEditorToolPropertiesAPI);

	// Register gizmo helper
	UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(GetInteractiveToolsContext());
}

void UUVEditorMode::InitializeTargets()
{
	using namespace UVEditorModeLocals;

	// InitializeModeContexts needs to have been called first so that we have the 3d preview world ready.
	check(LivePreviewWorld);

	// Nathan - Todo: This is the start of a larger fix for the overall problem of what
	// to do if we're loading more than one object into the UV Editor that's backed by
	// the same asset. Ideally we should show as many copies of this object in the 3D preview
	// pane as we have... but this is a big change.
	// 
	// So for now, we are at least keeping track of, at this stage, which targets map to
	// which object(s) and providing correct transforms for at least one of them.
	typedef TArray<int32> FObjectToTargets;
	FObjectToTargets ObjectToTargetMap;

	// Build the tool targets that provide us with 3d dynamic meshes
	UUVEditorSubsystem* UVSubsystem = GEditor->GetEditorSubsystem<UUVEditorSubsystem>();
	UVSubsystem->BuildTargets(OriginalObjectsToEdit, GetToolTargetRequirements(), ToolTargets, &ObjectToTargetMap);

	// Collect the 3d dynamic meshes from targets. There will always be one for each asset, and the AssetID
	// of each asset will be the index into these arrays. Individual input objects (representing a specific
	// UV layer), will point to these existing 3d meshes.
	for (UToolTarget* Target : ToolTargets)
	{
		// The applied canonical mesh is the 3d mesh with all of the layer changes applied. If we switch
		// to a different layer, the changes persist in the applied canonical.
		TSharedPtr<FDynamicMesh3> AppliedCanonical = MakeShared<FDynamicMesh3>(UE::ToolTarget::GetDynamicMeshCopy(Target));
		AppliedCanonicalMeshes.Add(AppliedCanonical);

		// Make a preview version of the applied canonical to show. Tools can attach computes to this, though
		// they would have to take care if we ever allow multiple layers to be displayed for one asset, to
		// avoid trying to attach two computes to the same preview object (in which case one would be thrown out)
		UMeshOpPreviewWithBackgroundCompute* AppliedPreview = NewObject<UMeshOpPreviewWithBackgroundCompute>();
		AppliedPreview->Setup(LivePreviewWorld);
		AppliedPreview->PreviewMesh->UpdatePreview(AppliedCanonical.Get());

		FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
		AppliedPreview->ConfigureMaterials(MaterialSet.Materials,
			ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
		AppliedPreviews.Add(AppliedPreview);
	}

	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/UVEditor/Materials/UVEditor_UnwrapMaterial"));

	// When creating UV unwraps, these functions will determine the mapping between UV values and the
	// resulting unwrap mesh vertex positions. 
	// If we're looking down on the unwrapped mesh, with the Z axis towards us, we want U's to be right, and
	// V's to be up. In Unreal's left-handed coordinate system, this means that we map U's to world Y
	// and V's to world X.
	// Also, Unreal changes the V coordinates of imported meshes to 1-V internally, and we undo this
	// while displaying the UV's because the users likely expect to see the original UV's (it would
	// be particularly confusing for users working with UDIM assets, where internally stored V's 
	// frequently end up negative).
	// The ScaleFactor just scales the mesh up. Scaling the mesh up makes it easier to zoom in
	// further into the display before getting issues with the camera near plane distance.

	// Construct the full input objects that the tools actually operate on.
	for (int32 AssetID = 0; AssetID < ToolTargets.Num(); ++AssetID)
	{
		UUVEditorToolMeshInput* ToolInputObject = NewObject<UUVEditorToolMeshInput>();

		if (!ToolInputObject->InitializeMeshes(ToolTargets[AssetID], AppliedCanonicalMeshes[AssetID],
			AppliedPreviews[AssetID], AssetID, DefaultUVLayerIndex,
			GetWorld(), LivePreviewWorld, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()),
			FUVEditorUXSettings::UVToVertPosition, FUVEditorUXSettings::VertPositionToUV))
		{
			return;
		}

		if (Transforms.Num() == ToolTargets.Num())
		{
			ToolInputObject->AppliedPreview->PreviewMesh->SetTransform(Transforms[AssetID]);
		}
		else
		{			
			int32 TargetIndex = ObjectToTargetMap.IndexOfByKey(AssetID);
			if (ensure(TargetIndex != -1))
			{
				ToolInputObject->AppliedPreview->PreviewMesh->SetTransform(Transforms[TargetIndex]);
			}
		}

		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, GetToolManager());
		MatInstance->SetVectorParameterValue(TEXT("Color"), FUVEditorUXSettings::GetTriangleColorByTargetIndex(AssetID));
		MatInstance->SetScalarParameterValue(TEXT("DepthBias"), FUVEditorUXSettings::UnwrapTriangleDepthOffset);
		MatInstance->SetScalarParameterValue(TEXT("Opacity"), FUVEditorUXSettings::UnwrapTriangleOpacity);
		MatInstance->SetScalarParameterValue(TEXT("UseVertexColors"), false);

		ToolInputObject->bEnableTriangleVertexColors = false;
		ToolInputObject->UnwrapPreview->PreviewMesh->SetMaterial(
			0, MatInstance);

		// Set up the wireframe display of the unwrapped mesh.
		UMeshElementsVisualizer* WireframeDisplay = NewObject<UMeshElementsVisualizer>(this);
		WireframeDisplay->CreateInWorld(GetWorld(), FTransform::Identity);

		WireframeDisplay->Settings->DepthBias = FUVEditorUXSettings::WireframeDepthOffset;
		WireframeDisplay->Settings->bAdjustDepthBiasUsingMeshSize = false;
		WireframeDisplay->Settings->bShowWireframe = true;
		WireframeDisplay->Settings->bShowBorders = true;
		WireframeDisplay->Settings->WireframeColor = FUVEditorUXSettings::GetWireframeColorByTargetIndex(AssetID).ToFColorSRGB();
		WireframeDisplay->Settings->BoundaryEdgeColor = FUVEditorUXSettings::GetBoundaryColorByTargetIndex(AssetID).ToFColorSRGB(); ;
		WireframeDisplay->Settings->bShowUVSeams = false;
		WireframeDisplay->Settings->bShowNormalSeams = false;
		// These are not exposed at the visualizer level yet
		// TODO: Should they be?
		WireframeDisplay->WireframeComponent->WireframeThickness = FUVEditorUXSettings::WireframeThickness;
		WireframeDisplay->WireframeComponent->BoundaryEdgeThickness = FUVEditorUXSettings::BoundaryEdgeThickness;

		// The wireframe will track the unwrap preview mesh
		WireframeDisplay->SetMeshAccessFunction([ToolInputObject](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
			ToolInputObject->UnwrapPreview->ProcessCurrentMesh(ProcessFunc);
			});

		// The settings object and wireframe are not part of a tool, so they won't get ticked like they
		// are supposed to (to enable property watching), unless we add this here.
		PropertyObjectsToTick.Add(WireframeDisplay->Settings);
		WireframesToTick.Add(WireframeDisplay);

		// The tool input object will hold on to the wireframe for the purposes of updating it and cleaning it up
		ToolInputObject->WireframeDisplay = WireframeDisplay;

		// Bind to delegate so that we can detect changes
		ToolInputObject->OnCanonicalModified.AddWeakLambda(this, [this]
		(UUVEditorToolMeshInput* InputObject, const UUVEditorToolMeshInput::FCanonicalModifiedInfo&) {
				ModifiedAssetIDs.Add(InputObject->AssetID);
			});

		ToolInputObjects.Add(ToolInputObject);
	}

	// Prep things for layer/channel selection
	InitializeAssetNames(ToolTargets, AssetNames);
	PendingUVLayerIndex.SetNumZeroed(ToolTargets.Num());


	// Finish initializing the selection api
	SelectionAPI->SetTargets(ToolInputObjects);
}

void UUVEditorMode::EmitToolIndependentObjectChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description)
{
	GetInteractiveToolsContext()->GetTransactionAPI()->AppendChange(TargetObject, MoveTemp(Change), Description);
}

bool UUVEditorMode::HaveUnappliedChanges() const
{
	return ModifiedAssetIDs.Num() > 0;
}

bool UUVEditorMode::CanApplyChanges() const
{
	return !bPIEModeActive && HaveUnappliedChanges();
}


void UUVEditorMode::GetAssetsWithUnappliedChanges(TArray<TObjectPtr<UObject>> UnappliedAssetsOut)
{
	for (int32 AssetID : ModifiedAssetIDs)
	{
		// The asset ID corresponds to the index into OriginalObjectsToEdit
		UnappliedAssetsOut.Add(OriginalObjectsToEdit[AssetID]);
	}
}

void UUVEditorMode::ApplyChanges()
{
	using namespace UVEditorModeLocals;

	FText ApplyChangesText = LOCTEXT("UVEditorApplyChangesTransaction", "UV Editor Apply Changes");
	GetToolManager()->BeginUndoTransaction(ApplyChangesText);

	for (int32 AssetID : ModifiedAssetIDs)
	{
		// The asset ID corresponds to the index into ToolTargets and AppliedCanonicalMeshes
		UE::ToolTarget::CommitDynamicMeshUVUpdate(ToolTargets[AssetID], AppliedCanonicalMeshes[AssetID].Get());
	}

	GetInteractiveToolsContext()->GetTransactionAPI()->AppendChange(
		this, MakeUnique<UVEditorModeChange::FApplyChangesChange>(ModifiedAssetIDs), ApplyChangesText);
	ModifiedAssetIDs.Reset();
	GetInteractiveToolsContext()->GetTransactionAPI()->EndUndoTransaction();

	GetToolManager()->EndUndoTransaction();
}


void UUVEditorMode::UpdateActiveUDIMs()
{
	bool bEnableUDIMSupport = (FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport.GetValueOnGameThread() > 0);
	if (!bEnableUDIMSupport)
	{
		return;
	}

	if (!UVEditorUDIMProperties)
	{
		return;
	}

	TSet<FUDIMSpecifier> ActiveUDIMs(UVEditorUDIMProperties->ActiveUDIMs);
	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	UUVTool2DViewportAPI* UVTool2DViewportAPI = ContextStore->FindContext<UUVTool2DViewportAPI>();
	if (UVTool2DViewportAPI)
	{
		TArray<FUDIMBlock> Blocks;
		if (ActiveUDIMs.Num() > 0)
		{
			Blocks.Reserve(ActiveUDIMs.Num());
			for (FUDIMSpecifier& UDIMSpecifier : ActiveUDIMs)
			{
				Blocks.Add({ UDIMSpecifier.UDIM, UDIMSpecifier.TextureResolution });
				UDIMSpecifier.UCoord = Blocks.Last().BlockU();
				UDIMSpecifier.VCoord = Blocks.Last().BlockV();				
			}
		}
		UVTool2DViewportAPI->SetUDIMBlocks(Blocks, true);
	}
	UVEditorUDIMProperties->ActiveUDIMs = ActiveUDIMs.Array();
	UVEditorUDIMProperties->SilentUpdateWatcherAtIndex(UDIMsChangedWatcherId);

	if (BackgroundVisualization)
	{
		BackgroundVisualization->Settings->UDIMBlocks.Empty();
		BackgroundVisualization->Settings->UDIMBlocks.Reserve(ActiveUDIMs.Num());
		for (FUDIMSpecifier& UDIMSpecifier : ActiveUDIMs)
		{
			BackgroundVisualization->Settings->UDIMBlocks.Add(UDIMSpecifier.UDIM);
		}
		BackgroundVisualization->Settings->CheckAndUpdateWatched();
	}

	if (DistortionVisualization)
	{
		DistortionVisualization->Settings->PerUDIMTextureResolution.Empty();
		DistortionVisualization->Settings->PerUDIMTextureResolution.Reserve(ActiveUDIMs.Num());
		for (FUDIMSpecifier& UDIMSpecifier : ActiveUDIMs)
		{
			DistortionVisualization->Settings->PerUDIMTextureResolution.Add(UDIMSpecifier.UDIM, UDIMSpecifier.TextureResolution);
		}
		DistortionVisualization->Settings->CheckAndUpdateWatched();
	}
}

void UUVEditorMode::PopulateUDIMsByAsset(int32 AssetId, TArray<FUDIMSpecifier>& UDIMsOut) const
{
	UDIMsOut.Empty();
	if (AssetId < 0 || AssetId >= ToolInputObjects.Num())
	{
		return;
	}

	if (ToolInputObjects[AssetId]->AppliedCanonical->HasAttributes() && ToolInputObjects[AssetId]->UVLayerIndex >= 0)
	{
		FDynamicMeshUVOverlay* UVLayer = ToolInputObjects[AssetId]->AppliedCanonical->Attributes()->GetUVLayer(ToolInputObjects[AssetId]->UVLayerIndex);
		FDynamicMeshUDIMClassifier TileClassifier(UVLayer);
		TArray<FVector2i> Tiles = TileClassifier.ActiveTiles();
		for (FVector2i Tile : Tiles)
		{
			if (Tile.X >= 10 || Tile.X < 0 || Tile.Y < 0)
			{
				UE_LOGF(LogUVEditor, Warning, "Tile <%d,%d> is out of bounds of the UDIM10 convention, skipping...", Tile.X, Tile.Y);
			}
			else
			{
				FUDIMSpecifier UDIMSpecifier;
				UDIMSpecifier.UCoord = Tile.X;
				UDIMSpecifier.VCoord = Tile.Y;
				UDIMSpecifier.UDIM = UE::TextureUtilitiesCommon::GetUDIMIndex(Tile.X, Tile.Y);
				UDIMsOut.AddUnique(UDIMSpecifier);
			}
		}
	}
}

void UUVEditorMode::UpdateTriangleMaterialBasedOnBackground(bool IsBackgroundVisible)
{
	UpdateTriangleMaterialBasedOnDisplaySettings();
}

void UUVEditorMode::UpdateTriangleMaterialBasedOnDisplaySettings()
{
	float TriangleOpacity;

	// We adjust the mesh opacity depending on whether we're layered over the background or not.
	if (BackgroundVisualization && BackgroundVisualization->Settings && BackgroundVisualization->Settings->bVisible)
	{
		TriangleOpacity = FUVEditorUXSettings::UnwrapTriangleOpacityWithBackground;
	}
	else
	{
		TriangleOpacity = FUVEditorUXSettings::UnwrapTriangleOpacity;
	}

	bool bUseVertexColors = DistortionVisualization && DistortionVisualization->Settings && DistortionVisualization->Settings->bVisible;

	// Modify the material of the unwrapped mesh to account for the presence/absence of the background, 
	// changing the opacity as set just above.
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/UVEditor/Materials/UVEditor_UnwrapMaterial"));
	if (Material != nullptr)
	{
		for (int32 AssetID = 0; AssetID < ToolInputObjects.Num(); ++AssetID) {
			UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, GetToolManager());
			MatInstance->SetVectorParameterValue(TEXT("Color"), FUVEditorUXSettings::GetTriangleColorByTargetIndex(AssetID));
			MatInstance->SetScalarParameterValue(TEXT("DepthBias"), FUVEditorUXSettings::UnwrapTriangleDepthOffset);
			MatInstance->SetScalarParameterValue(TEXT("Opacity"), TriangleOpacity);
			MatInstance->SetScalarParameterValue(TEXT("UseVertexColors"), bUseVertexColors);

			ToolInputObjects[AssetID]->bEnableTriangleVertexColors = bUseVertexColors;
			ToolInputObjects[AssetID]->UnwrapPreview->PreviewMesh->SetMaterial(
				0, MatInstance);
		}
	}
}

void UUVEditorMode::UpdatePreviewMaterialBasedOnBackground()
{
	for (int32 AssetID = 0; AssetID < ToolInputObjects.Num(); ++AssetID) {
		ToolInputObjects[AssetID]->AppliedPreview->OverrideMaterial = nullptr;
	}
	if (BackgroundVisualization->Settings->bVisible)
	{
		for (int32 AssetID = 0; AssetID < ToolInputObjects.Num(); ++AssetID) {			
			TObjectPtr<UMaterialInstanceDynamic> BackgroundMaterialOverride = UMaterialInstanceDynamic::Create(BackgroundVisualization->BackgroundMaterial->Parent, this);
			BackgroundMaterialOverride->CopyInterpParameters(BackgroundVisualization->BackgroundMaterial);
			BackgroundMaterialOverride->SetScalarParameterValue(TEXT("UVChannel"), static_cast<float>(GetDisplayedChannel(AssetID)));

			ToolInputObjects[AssetID]->AppliedPreview->OverrideMaterial = BackgroundMaterialOverride;
		}
	}
}

void UUVEditorMode::FocusLivePreviewCameraOnSelection()
{
	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	UUVToolLivePreviewAPI* LivePreviewAPI = ContextStore->FindContext<UUVToolLivePreviewAPI>();
	if (!LivePreviewAPI)
	{
		return;
	}

	FAxisAlignedBox3d SelectionBoundingBox;

	const TArray<FUVToolSelection>& CurrentSelections = SelectionAPI->GetSelections();
	const TArray<FUVToolSelection>& CurrentUnsetSelections = SelectionAPI->GetUnsetElementAppliedMeshSelections();

	for (const FUVToolSelection& Selection : CurrentSelections)
	{
		FTransform3d Transform = Selection.Target->AppliedPreview->PreviewMesh->GetTransform();
		SelectionBoundingBox.Contain(Selection.GetConvertedSelectionForAppliedMesh().ToBoundingBox(*Selection.Target->AppliedCanonical, Transform));
	}
	for (const FUVToolSelection& Selection : CurrentUnsetSelections)
	{
		FTransform3d Transform = Selection.Target->AppliedPreview->PreviewMesh->GetTransform();
		SelectionBoundingBox.Contain(Selection.ToBoundingBox(*Selection.Target->AppliedCanonical, Transform));
	}
	if (CurrentSelections.Num() == 0 && CurrentUnsetSelections.Num() == 0)
	{
		for (int32 AssetID = 0; AssetID < ToolInputObjects.Num(); ++AssetID)
		{
			FTransform3d Transform = ToolInputObjects[AssetID]->AppliedPreview->PreviewMesh->GetTransform();
			FAxisAlignedBox3d ObjectBoundingBox = ToolInputObjects[AssetID]->AppliedCanonical->GetBounds();
			ObjectBoundingBox.Max = Transform.TransformPosition(ObjectBoundingBox.Max);
			ObjectBoundingBox.Min = Transform.TransformPosition(ObjectBoundingBox.Min);
			SelectionBoundingBox.Contain(ObjectBoundingBox);
		}
	}
	
	LivePreviewAPI->SetLivePreviewCameraToLookAtVolume(SelectionBoundingBox);
}

void UUVEditorMode::ModeTick(float DeltaTime)
{
	using namespace UVEditorModeLocals;

	Super::ModeTick(DeltaTime);

	bool bSwitchingLayers = false;
	for (int32 AssetID = 0; AssetID < ToolInputObjects.Num(); ++AssetID)
	{
		if (ToolInputObjects[AssetID]->UVLayerIndex != PendingUVLayerIndex[AssetID])
		{
			bSwitchingLayers = true;
			break;
		}
	}

	if (bSwitchingLayers)
	{
		// TODO: Perhaps we need our own interactive tools context that allows this kind of "end tool now"
		// call. We can't do the normal GetInteractiveToolsContext()->EndTool() call because we cannot defer
		// shutdown.
		GetInteractiveToolsContext()->ToolManager->DeactivateTool(EToolSide::Mouse, EToolShutdownType::Cancel);

		// We open the transaction here instead of before the tool close above because otherwise we seem
		// to get some kind of spurious items in the transaction that, while they don't seem to do anything,
		// cause the transaction to not be fully expired on editor close.
		GetToolManager()->BeginUndoTransaction(UVLayerChangeTransactionName);

		SetDisplayedUVChannels(PendingUVLayerIndex, true);

		ActivateDefaultTool();

		GetToolManager()->EndUndoTransaction();
	}
	
	for (TObjectPtr<UInteractiveToolPropertySet>& Propset : PropertyObjectsToTick)
	{
		if (Propset)
		{
			if (Propset->IsPropertySetEnabled())
			{
				Propset->CheckAndUpdateWatched();
			}
			else
			{
				Propset->SilentUpdateWatched();
			}
		}
	}

	for (TWeakObjectPtr<UMeshElementsVisualizer> WireframeDisplay : WireframesToTick)
	{
		if (WireframeDisplay.IsValid())
		{
			WireframeDisplay->OnTick(DeltaTime);
		}
	}

	if (BackgroundVisualization)
	{
		BackgroundVisualization->OnTick(DeltaTime);
	}

	if (DistortionVisualization)
	{
		DistortionVisualization->OnTick(DeltaTime);
	}

	for (int i = 0; i < ToolInputObjects.Num(); ++i)
	{
		TObjectPtr<UUVEditorToolMeshInput> ToolInput = ToolInputObjects[i];
		ToolInput->AppliedPreview->Tick(DeltaTime);
		ToolInput->UnwrapPreview->Tick(DeltaTime);
	}

	// If nothing is running at this point, restart the default tool, since it needs to be running to handle some toolbar UX
	if (GetInteractiveToolsContext() && !GetInteractiveToolsContext()->HasActiveTool())
	{
		ActivateDefaultTool();
	}

}

void UUVEditorMode::SetSimulationWarning(bool bEnabled)
{
	if (Toolkit.IsValid())
	{
		FUVEditorModeToolkit* UVEditorModeToolkit = StaticCast<FUVEditorModeToolkit*>(Toolkit.Get());
		UVEditorModeToolkit->EnableShowPIEWarning(bEnabled);
	}
}

int32 UUVEditorMode::GetNumUVChannels(int32 AssetID) const
{
	if (AssetID < 0 || AssetID >= AppliedCanonicalMeshes.Num())
	{
		return IndexConstants::InvalidID;
	}

	return AppliedCanonicalMeshes[AssetID]->HasAttributes() ?
		AppliedCanonicalMeshes[AssetID]->Attributes()->NumUVLayers() : 0;
}

int32 UUVEditorMode::GetDisplayedChannel(int32 AssetID) const
{
	if (!ensure(AssetID >= 0 && AssetID < ToolInputObjects.Num()))
	{
		return IndexConstants::InvalidID;
	}

	return ToolInputObjects[AssetID]->UVLayerIndex;;
}

void UUVEditorMode::RequestUVChannelChange(int32 AssetID, int32 Channel)
{
	if (!ensure(
		AssetID >= 0 && AssetID < ToolTargets.Num())
		&& Channel >= 0 && Channel < GetNumUVChannels(AssetID))
	{
		return;
	}

	PendingUVLayerIndex[AssetID] = Channel;
}

void UUVEditorMode::ChangeInputObjectLayer(int32 AssetID, int32 NewLayerIndex)
{
	using namespace UVEditorModeLocals;

	if (!ensure(AssetID >= 0 && AssetID < ToolInputObjects.Num() 
		&& ToolInputObjects[AssetID]->AssetID == AssetID))
	{
		return;
	}

	UUVEditorToolMeshInput* InputObject = ToolInputObjects[AssetID];
	if (InputObject->UVLayerIndex != NewLayerIndex)
	{
		InputObject->UVLayerIndex = NewLayerIndex;
		InputObject->UpdateAllFromAppliedCanonical();
	}

	PendingUVLayerIndex[AssetID] = InputObject->UVLayerIndex;
	UpdatePreviewMaterialBasedOnBackground();
}

void UUVEditorMode::SetDisplayedUVChannels(const TArray<int32>& LayerPerAsset, bool bEmitUndoTransaction)
{
	// Deal with any existing selections first.
	// TODO: We could consider keeping triangle selections, since these can be valid across different layers,
	// or converting the selections to elements in the applied mesh and back. The gain for this would probably
	// be minor, so for now we'll just clear selection here.
	if (SelectionAPI)
	{
		SelectionAPI->ClearSelections(true, bEmitUndoTransaction); // broadcast, maybe emit
	}

	// Now actually swap out the layers
	for (int32 AssetID = 0; AssetID < ToolInputObjects.Num(); ++AssetID)
	{
		if (ToolInputObjects[AssetID]->UVLayerIndex != LayerPerAsset[AssetID])
		{			
			if (bEmitUndoTransaction)
			{
				GetInteractiveToolsContext()->GetTransactionAPI()->AppendChange(this,
					MakeUnique<UVEditorModeLocals::FInputObjectUVLayerChange>(AssetID, ToolInputObjects[AssetID]->UVLayerIndex, LayerPerAsset[AssetID]),
					UVEditorModeLocals::UVLayerChangeTransactionName);
			}

			ChangeInputObjectLayer(AssetID, LayerPerAsset[AssetID]);
		}
	}
}

#undef LOCTEXT_NAMESPACE

