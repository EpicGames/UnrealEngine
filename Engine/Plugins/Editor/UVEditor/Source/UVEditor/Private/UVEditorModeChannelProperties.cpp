// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorModeChannelProperties.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargets/UVEditorToolMeshInput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorModeChannelProperties)

void UUVEditorUVChannelProperties::Initialize(
	TArray<TObjectPtr<UToolTarget>>& TargetObjects, 
	TArray<TSharedPtr<UE::Geometry::FDynamicMesh3>> AppliedCanonicalMeshes,
	bool bInitializeSelection)
{
	if (!ensure(TargetObjects.Num() == AppliedCanonicalMeshes.Num()))
	{
		return;
	}

	UVAssetNames.Reset(TargetObjects.Num());
	NumUVLayersPerAsset.Reset(TargetObjects.Num());

	for (int i = 0; i < TargetObjects.Num(); ++i)
	{
		UVAssetNames.Add(UE::ToolTarget::GetHumanReadableName(TargetObjects[i]));
		
		int32 NumUVChannels = AppliedCanonicalMeshes[i]->HasAttributes() ?
			AppliedCanonicalMeshes[i]->Attributes()->NumUVLayers() : 0;
		
		NumUVLayersPerAsset.Add(NumUVChannels);
	}

	UVChannelNames.Reset();
	if (bInitializeSelection)
	{
		Asset = UVAssetNames[0];
		GetUVChannelNames();
		UVChannel = UVChannelNames.Num() > 0 ? UVChannelNames[0] : TEXT("");
	}
}

const TArray<FString>& UUVEditorUVChannelProperties::GetAssetNames()
{
	return UVAssetNames;
}

const TArray<FString>& UUVEditorUVChannelProperties::GetUVChannelNames()
{
	int32 FoundAssetIndex = UVAssetNames.IndexOfByKey(Asset);

	if (FoundAssetIndex == INDEX_NONE)
	{
		UVChannelNames.Reset();
		return UVChannelNames;
	}

	if (UVChannelNames.Num() != NumUVLayersPerAsset[FoundAssetIndex])
	{
		UVChannelNames.Reset();
		for (int32 i = 0; i < NumUVLayersPerAsset[FoundAssetIndex]; ++i)
		{
			UVChannelNames.Add(FString::Printf(TEXT("UV%d"), i));
		}
	}

	return UVChannelNames;
}

bool UUVEditorUVChannelProperties::ValidateUVAssetSelection(bool bUpdateIfInvalid)
{
	int32 FoundIndex = UVAssetNames.IndexOfByKey(Asset);
	if (FoundIndex == INDEX_NONE)
	{
		if (bUpdateIfInvalid)
		{
			Asset = (UVAssetNames.Num() > 0) ? UVAssetNames[0] : TEXT("");
		}
		return false;
	}
	return true;
}

bool UUVEditorUVChannelProperties::ValidateUVChannelSelection(bool bUpdateIfInvalid)
{
	bool bValid = ValidateUVAssetSelection(bUpdateIfInvalid);

	int32 FoundAssetIndex = UVAssetNames.IndexOfByKey(Asset);
	if (FoundAssetIndex == INDEX_NONE)
	{
		if (bUpdateIfInvalid)
		{
			UVChannel = TEXT("");
		}
		return false;
	}

	int32 FoundIndex = UVChannelNames.IndexOfByKey(UVChannel);
	if (FoundIndex >= NumUVLayersPerAsset[FoundAssetIndex])
	{
		FoundIndex = INDEX_NONE;
	}

	if (FoundIndex == INDEX_NONE)
	{
		if (bUpdateIfInvalid)
		{
			UVChannel = (NumUVLayersPerAsset[FoundAssetIndex] > 0) ? UVChannelNames[0] : TEXT("");
		}
		return false;
	}

	return bValid;
}

int32 UUVEditorUVChannelProperties::GetSelectedAssetID()
{
	int32 FoundIndex = UVAssetNames.IndexOfByKey(Asset);
	if (FoundIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return FoundIndex;
}

int32 UUVEditorUVChannelProperties::GetSelectedChannelIndex(bool bForceToZeroOnFailure)
{
	int32 FoundAssetIndex = UVAssetNames.IndexOfByKey(Asset);
	if (FoundAssetIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	int32 FoundUVIndex = UVChannelNames.IndexOfByKey(UVChannel);
	if (!ensure(FoundUVIndex < NumUVLayersPerAsset[FoundAssetIndex]))
	{
		return INDEX_NONE;
	}
	return FoundUVIndex;
}
