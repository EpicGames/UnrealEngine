// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorModeToolkit.h"

#include "Styling/AppStyle.h" //FAppStyle
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "SPrimaryButton.h"
#include "STransformGizmoNumericalUIOverlay.h"
#include "Tools/UEdMode.h"
#include "UVEditorBackgroundPreview.h"
#include "UVEditorCommands.h"
#include "UVEditorMode.h"
#include "UVEditorUXSettings.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "EditorModeManager.h"
#include "UVEditorModeUILayer.h"
#include "UVEditorStyle.h"
#include "AssetEditorModeManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FUVEditorModeToolkit"

namespace UVEditorModeToolkitLocals
{
	TAutoConsoleVariable<bool> CVarAddDebugTools(
		TEXT("modeling.UVEditor.AddDebugTools"),
		false,
		TEXT("Enable UV tools that can be useful for debugging (currently just the \"Unset UVs\" tool."));
}

FUVEditorModeToolkit::FUVEditorModeToolkit()
{
	// Construct the panel that we will give in GetInlineContent().
	// This could probably be done in Init() instead, but constructor
	// makes it easy to guarantee that GetInlineContent() will always
	// be ready to work.

	SAssignNew(ToolkitWidget, SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	[
		SAssignNew(ToolWarningArea, STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f))) //TODO: This probably needs to not be hardcoded
		.Text(FText::GetEmpty())
		.Visibility(EVisibility::Collapsed)
	]
	+ SVerticalBox::Slot()
	.AutoHeight()
	[
		SAssignNew(ModeWarningArea, STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f))) //TODO: This probably needs to not be hardcoded
		.Text(FText::GetEmpty())
		.Visibility(EVisibility::Collapsed)
	]
	+ SVerticalBox::Slot()
	[
		SAssignNew(ToolDetailsContainer, SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
	];
}

FUVEditorModeToolkit::~FUVEditorModeToolkit()
{
	UEdMode* Mode = GetScriptableEditorMode().Get();
	if (ensure(Mode))
	{
		UEditorInteractiveToolsContext* Context = Mode->GetInteractiveToolsContext();
		if (ensure(Context))
		{
			Context->OnToolNotificationMessage.RemoveAll(this);
			Context->OnToolWarningMessage.RemoveAll(this);
		}
	}
}

void FUVEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FModeToolkit::Init(InitToolkitHost, InOwningMode);	

	UUVEditorMode* UVEditorMode = Cast<UUVEditorMode>(GetScriptableEditorMode());
	check(UVEditorMode);
	UEditorInteractiveToolsContext* InteractiveToolsContext = UVEditorMode->GetInteractiveToolsContext();
	check(InteractiveToolsContext);

	// Currently, there's no EToolChangeTrackingMode that reverts back to a default tool on undo (if we add that
	// support, the tool manager will need to be aware of the default tool). So, we instead opt to do our own management
	// of tool start transactions. See FUVEditorModeToolkit::OnToolStarted for how we issue the transactions.
	InteractiveToolsContext->ToolManager->ConfigureChangeTrackingMode(EToolChangeTrackingMode::NoChangeTracking);

	// Set up tool message areas
	ClearNotification();
	ClearWarning();
	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolNotificationMessage.AddSP(this, &FUVEditorModeToolkit::PostNotification);
	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolWarningMessage.AddSP(this, &FUVEditorModeToolkit::PostWarning);

	// Hook up the tool detail panel
	ToolDetailsContainer->SetContent(DetailsView.ToSharedRef());
	
	// Set up the overlay. Largely copied from ModelingToolsEditorModeToolkit.
	// TODO: We could put some of the shared code in some common place.
	SAssignNew(ViewportOverlayWidget, SHorizontalBox)

	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
		.Padding(8.f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this] () { return ActiveToolIcon; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text(this, &FUVEditorModeToolkit::GetActiveToolDisplayName)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("OverlayAccept", "Accept"))
				.ToolTipText(LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]"))
				.OnClicked_Lambda([this]() { 
					GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept);
					Cast<UUVEditorMode>(GetScriptableEditorMode())->ActivateDefaultTool();
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanAcceptActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Button")
				.TextStyle( FAppStyle::Get(), "DialogButtonText" )
				.Text(LOCTEXT("OverlayCancel", "Cancel"))
				.ToolTipText(LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { 
					GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel); 
					Cast<UUVEditorMode>(GetScriptableEditorMode())->ActivateDefaultTool();
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCancelActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("OverlayComplete", "Complete"))
				.ToolTipText(LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { 
					GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed);
					Cast<UUVEditorMode>(GetScriptableEditorMode())->ActivateDefaultTool();
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() {
					UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());
					return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCompleteActiveTool();
				})
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCompleteActiveTool() ? EVisibility::Visible : EVisibility::Collapsed; })
			]
		]	
	];

	// Set up the gizmo numerical UI
	GizmoNumericalUIOverlayWidget = SNew(STransformGizmoNumericalUIOverlay)
		.bPositionRelativeToBottom(true)
		.DefaultLeftPadding(15.0f)
		// Unlike the level editor, we don't have an axis indicator that we have to be above, so this value is different
		.DefaultVerticalPadding(15.0f)
		// Enable non-delta mode by giving a default reference transform
		.DefaultLocalReferenceTransform(FTransform())
		// Make our displays show 0-1 units instead of world units.
		.InternalToDisplayFunction([](const FVector3d& WorldPosition)
		{
			FVector2f UV = FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(WorldPosition);
			return FVector3d(UV.X, UV.Y, 0);
		})
		.DisplayToInternalFunction([](const FVector3d& UVPosition)
		{
			return FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(FVector2f(UVPosition.X, UVPosition.Y));
		})
		.TranslationScrubSensitivity(1 / FUVEditorUXSettings::UVMeshScalingFactor)
		;

	GetToolkitHost()->AddViewportOverlayWidget(GizmoNumericalUIOverlayWidget.ToSharedRef());
}

void FUVEditorModeToolkit::InitializeAfterModeSetup()
{
	if (bFirstInitializeAfterModeSetup)
	{
		if (ensure(GizmoNumericalUIOverlayWidget.IsValid()))
		{
			// This has to happen after the gizmo context object is registered
			GizmoNumericalUIOverlayWidget->BindToGizmoContextObject(GetScriptableEditorMode()->GetInteractiveToolsContext());
		}
		bFirstInitializeAfterModeSetup = false;
	}
	
}

FName FUVEditorModeToolkit::GetToolkitFName() const
{
	return FName("UVEditorMode");
}

FText FUVEditorModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("UVEditorModeToolkit", "DisplayName", "UVEditorMode");
}

TSharedRef<SWidget> FUVEditorModeToolkit::CreateChannelMenu()
{
	UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	// For each asset, create a submenu labeled with its name
	const TArray<FString>& AssetNames = Mode->GetAssetNames();
	for (int32 AssetID = 0; AssetID < AssetNames.Num(); ++AssetID)
	{
		MenuBuilder.AddSubMenu(
			FText::AsCultureInvariant(AssetNames[AssetID]), // label
			FText(), // tooltip
			FNewMenuDelegate::CreateWeakLambda(Mode, [this, Mode, AssetID](FMenuBuilder& SubMenuBuilder)
		{

			// Inside each submenu, create a button for each channel
			int32 NumChannels = Mode->GetNumUVChannels(AssetID);
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				SubMenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("ChannelLabel", "UV Channel {0}"), Channel), // label
					FText(), // tooltip
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateWeakLambda(Mode, [Mode, AssetID, Channel]()
						{
							Mode->RequestUVChannelChange(AssetID, Channel);

							// A bit of a hack to force the menu to close if the checkbox is clicked (which usually doesn't
							// close the menu)
							FSlateApplication::Get().DismissAllMenus();
						}), 
						FCanExecuteAction::CreateWeakLambda(Mode, []() { return true; }),
						FIsActionChecked::CreateWeakLambda(Mode, [Mode, AssetID, Channel]()
						{
							return Mode->GetDisplayedChannel(AssetID) == Channel;
						})),
					NAME_None,
					EUserInterfaceActionType::RadioButton);
			}
		}));
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FUVEditorModeToolkit::CreateBackgroundSettingsWidget()
{
	UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());
	return CreateDisplaySettingsWidget(Mode->GetBackgroundSettingsObject());
}

TSharedRef<SWidget> FUVEditorModeToolkit::CreateDistortionVisualsSettingsWidget()
{
	UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());
	return CreateDisplaySettingsWidget(Mode->GetDistortionVisualsSettingsObject());
}

TSharedRef<SWidget> FUVEditorModeToolkit::CreateGridSettingsWidget()
{
	UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());
	return CreateDisplaySettingsWidget(Mode->GetGridSettingsObject());
}

TSharedRef<SWidget> FUVEditorModeToolkit::CreateUnwrappedUXSettingsWidget()
{
	UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());
	return CreateDisplaySettingsWidget(Mode->GetUnwrappedUXSettingsObject());
}

TSharedRef<SWidget> FUVEditorModeToolkit::CreateLivePreviewUXSettingsWidget()
{
	UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());
	return CreateDisplaySettingsWidget(Mode->GetLivePreviewUXSettingsObject());
}

TSharedRef<SWidget> FUVEditorModeToolkit::CreateUDIMSettingsWidget()
{
	UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());
	return CreateDisplaySettingsWidget(Mode->GetUDIMSettingsObject());
}

TSharedRef<SWidget> FUVEditorModeToolkit::GetToolDisplaySettingsWidget()
{
	UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());
	UObject* SettingsObject = Mode->GetToolDisplaySettingsObject();
	if (SettingsObject)
	{
		return CreateDisplaySettingsWidget(SettingsObject);
	}
	else
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Padding(0);
	}
}

void FUVEditorModeToolkit::MakeGizmoNumericalUISubMenu(FMenuBuilder& MenuBuilder)
{
	if (GizmoNumericalUIOverlayWidget.IsValid())
	{
		GizmoNumericalUIOverlayWidget->MakeNumericalUISubMenu(MenuBuilder);
	}
}

TSharedRef<SWidget> FUVEditorModeToolkit::CreateDisplaySettingsWidget(UObject* SettingsObject) const
{
	TSharedRef<SBorder> GridDetailsContainer =
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"));

	TSharedRef<SWidget> Widget = SNew(SBorder)
		.HAlign(HAlign_Fill)
		.Padding(4)
		[
			SNew(SBox)
			.MinDesiredWidth(500)
		[
			GridDetailsContainer
		]
		];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs GridDetailsViewArgs;
	GridDetailsViewArgs.bAllowSearch = false;
	GridDetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	GridDetailsViewArgs.bHideSelectionTip = true;
	GridDetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	GridDetailsViewArgs.bShowOptions = false;
	GridDetailsViewArgs.bAllowMultipleTopLevelObjects = false;

	TSharedRef<IDetailsView> GridDetailsView = PropertyEditorModule.CreateDetailView(GridDetailsViewArgs);
	GridDetailsView->SetObject(SettingsObject);
	GridDetailsContainer->SetContent(GridDetailsView);

	return Widget;

}


void FUVEditorModeToolkit::UpdateActiveToolProperties()
{
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager()->GetActiveTool(EToolSide::Left);
	if (CurTool != nullptr)
	{
		DetailsView->SetObjects(CurTool->GetToolProperties(true));
	}
}

void FUVEditorModeToolkit::InvalidateCachedDetailPanelState(UObject* ChangedObject)
{
	DetailsView->InvalidateCachedState();
}

void FUVEditorModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FModeToolkit::OnToolStarted(Manager, Tool);

	UUVEditorMode* Mode = Cast<UUVEditorMode>(GetScriptableEditorMode());
	UInteractiveTool* CurTool = Mode->GetToolManager()->GetActiveTool(EToolSide::Left);
	CurTool->OnPropertySetsModified.AddSP(this, &FUVEditorModeToolkit::UpdateActiveToolProperties);
	CurTool->OnPropertyModifiedDirectlyByTool.AddSP(this, &FUVEditorModeToolkit::InvalidateCachedDetailPanelState);

	ActiveToolName = Tool->GetToolInfo().ToolDisplayName;

	FString ActiveToolIdentifier = GetScriptableEditorMode()->GetToolManager()->GetActiveToolName(EToolSide::Mouse);
	ActiveToolIdentifier.InsertAt(0, ".");
	FName ActiveToolIconName = ISlateStyle::Join(FUVEditorCommands::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifier));
	ActiveToolIcon = FUVEditorStyle::Get().GetOptionalBrush(ActiveToolIconName);

	if (!Mode->IsDefaultToolActive())
	{
		// Add the accept/cancel overlay only if the tool is not the default tool.
		GetToolkitHost()->AddViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}
}

void FUVEditorModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FModeToolkit::OnToolEnded(Manager, Tool);

	ActiveToolName = FText::GetEmpty();
	ClearNotification();
	ClearWarning();

	if (IsHosted())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}

	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager()->GetActiveTool(EToolSide::Left);
	if (CurTool)
	{
		CurTool->OnPropertySetsModified.RemoveAll(this);
		CurTool->OnPropertyModifiedDirectlyByTool.RemoveAll(this);
	}
}


// Place tool category names here, for creating the tool palette below
static const FName ToolsTabName(TEXT("Tools"));

const TArray<FName> FUVEditorModeToolkit::PaletteNames_Standard = { ToolsTabName };

void FUVEditorModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames = PaletteNames_Standard;
}

FText FUVEditorModeToolkit::GetToolPaletteDisplayName(FName Palette) const
{
	return FText::FromName(Palette);
}

void FUVEditorModeToolkit::BuildToolPalette(FName PaletteIndex, class FToolBarBuilder& ToolbarBuilder)
{
	using namespace UVEditorModeToolkitLocals;

	const FUVEditorCommands& Commands = FUVEditorCommands::Get();

	if (PaletteIndex == ToolsTabName)
	{
		ToolbarBuilder.AddToolBarButton(Commands.SewAction);
		ToolbarBuilder.AddToolBarButton(Commands.SplitAction);
		ToolbarBuilder.AddToolBarButton(Commands.MakeIslandAction);
		if (CVarAddDebugTools.GetValueOnGameThread())
		{
			ToolbarBuilder.AddToolBarButton(Commands.UnsetUVsAction);
		}

		ToolbarBuilder.AddToolBarButton(Commands.BeginLayoutTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginTransformTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginAlignTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginDistributeTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginTexelDensityTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginChannelEditTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginSeamTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginParameterizeMeshTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginRecomputeUVsTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginBrushSelectTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginUVSnapshotTool);
	}
}

void FUVEditorModeToolkit::PostNotification(const FText& Message)
{
	ClearNotification();

	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		ActiveToolMessageHandle = GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PushStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), Message);
	}
}

void FUVEditorModeToolkit::ClearNotification()
{
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessageHandle);
	}
	ActiveToolMessageHandle.Reset();
}

void FUVEditorModeToolkit::PostWarning(const FText& Message)
{
	if (Message.IsEmpty())
	{
		ClearWarning();
	}
	else
	{
		ToolWarningArea->SetText(Message);
		ToolWarningArea->SetVisibility(EVisibility::Visible);
	}
}

void FUVEditorModeToolkit::ClearWarning()
{
	ToolWarningArea->SetText(FText());
	ToolWarningArea->SetVisibility(EVisibility::Collapsed);
}

void FUVEditorModeToolkit::PostModeWarning(const FText& Message)
{
	if (Message.IsEmpty())
	{
		ClearModeWarning();
	}
	else
	{
		ModeWarningArea->SetText(Message);
		ModeWarningArea->SetVisibility(EVisibility::Visible);
	}
}

void FUVEditorModeToolkit::ClearModeWarning()
{
	ModeWarningArea->SetText(FText());
	ModeWarningArea->SetVisibility(EVisibility::Collapsed);
}

void FUVEditorModeToolkit::UpdatePIEWarnings()
{
	if (bShowPIEWarning)
	{
		PostModeWarning(LOCTEXT("ModelingModeToolkitPIEWarning", "UV Editor functionality is limited during Play in Editor sessions. End the current Play in Editor session to continue using the editor."));
	}
	else
	{
		ClearModeWarning();
	}
}

void FUVEditorModeToolkit::EnableShowPIEWarning(bool bEnable)
{
	if (bShowPIEWarning != bEnable)
	{
		bShowPIEWarning = bEnable;
		UpdatePIEWarnings();
	}
}

#undef LOCTEXT_NAMESPACE
