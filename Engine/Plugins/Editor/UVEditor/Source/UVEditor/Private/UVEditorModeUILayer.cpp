// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorModeUILayer.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Toolkits/IToolkit.h"
#include "UVEditorToolkit.h"
#include "UVEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorModeUILayer)

void UUVEditorUISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FUVEditorModule& UVEditorModule = FModuleManager::GetModuleChecked<FUVEditorModule>("UVEditor");
	UVEditorModule.OnRegisterLayoutExtensions().AddUObject(this, &UUVEditorUISubsystem::RegisterLayoutExtensions);
}
void UUVEditorUISubsystem::Deinitialize()
{
	FUVEditorModule& UVEditorModule = FModuleManager::GetModuleChecked<FUVEditorModule>("UVEditor");
	UVEditorModule.OnRegisterLayoutExtensions().RemoveAll(this);
}
void UUVEditorUISubsystem::RegisterLayoutExtensions(FLayoutExtender& Extender)
{
	FTabManager::FTab NewTab(FTabId(UAssetEditorUISubsystem::TopLeftTabID), ETabState::ClosedTab);
	Extender.ExtendStack("EditorSidePanelArea", ELayoutExtensionPosition::After, NewTab);
}

FUVEditorModeUILayer::FUVEditorModeUILayer(const IToolkitHost* InToolkitHost) :
	FAssetEditorModeUILayer(InToolkitHost)
{}

void FUVEditorModeUILayer::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	if (!Toolkit->IsAssetEditor())
	{
		FAssetEditorModeUILayer::OnToolkitHostingStarted(Toolkit);
		HostedToolkit = Toolkit;
		Toolkit->SetModeUILayer(SharedThis(this));
		Toolkit->RegisterTabSpawners(ToolkitHost->GetTabManager().ToSharedRef());
		RegisterModeTabSpawners();
		OnToolkitHostReadyForUI.ExecuteIfBound();
	}
}

void FUVEditorModeUILayer::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	if (HostedToolkit.IsValid() && HostedToolkit.Pin() == Toolkit)
	{
		FAssetEditorModeUILayer::OnToolkitHostingFinished(Toolkit);
	}
}

TSharedPtr<FWorkspaceItem> FUVEditorModeUILayer::GetModeMenuCategory() const
{
	check(UVEditorMenuCategory);
	return UVEditorMenuCategory;
}

void FUVEditorModeUILayer::SetModeMenuCategory(TSharedPtr<FWorkspaceItem> MenuCategoryIn)
{
	UVEditorMenuCategory = MenuCategoryIn;
}

