// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorModularFeature.h"

#include "CoreMinimal.h"
#include "UVEditorSubsystem.h"
#include "Editor.h"
#include "GameFramework/Actor.h"

namespace UE
{
namespace Geometry
{
	void FUVEditorModularFeature::LaunchUVEditor(const TArray<TObjectPtr<UObject>>& ObjectsIn)
	{
		UUVEditorSubsystem* UVSubsystem = GEditor->GetEditorSubsystem<UUVEditorSubsystem>();
		check(UVSubsystem);
		TArray<TObjectPtr<UObject>> ProcessedObjects;
		ConvertInputArgsToValidTargets(ObjectsIn, ProcessedObjects);
		UVSubsystem->StartUVEditor(ProcessedObjects);
	}

	bool FUVEditorModularFeature::CanLaunchUVEditor(const TArray<TObjectPtr<UObject>>& ObjectsIn)
	{
		UUVEditorSubsystem* UVSubsystem = GEditor->GetEditorSubsystem<UUVEditorSubsystem>();
		check(UVSubsystem);
		TArray<TObjectPtr<UObject>> ProcessedObjects;
		ConvertInputArgsToValidTargets(ObjectsIn, ProcessedObjects);
		return UVSubsystem->AreObjectsValidTargets(ProcessedObjects);
	}

	void FUVEditorModularFeature::ConvertInputArgsToValidTargets(const TArray<TObjectPtr<UObject>>& ObjectsIn, TArray<TObjectPtr<UObject>>& ObjectsOut) const
	{
		UUVEditorSubsystem* UVSubsystem = GEditor->GetEditorSubsystem<UUVEditorSubsystem>();
		if (!UVSubsystem)
		{
			return;
		}
		
		for (const TObjectPtr<UObject>& Object : ObjectsIn)
		{
			if (const AActor* Actor = Cast<const AActor>(Object))
			{
				// Need to transform actors to components here because that's what the UVEditor expects to have
				TInlineComponentArray<UActorComponent*> ActorComponents;
				Actor->GetComponents(ActorComponents);
				for (UActorComponent* ActorComponent : ActorComponents)
				{
					if (UVSubsystem->IsObjectValidTarget(ActorComponent))
					{
						ObjectsOut.AddUnique(ActorComponent);
					}
				}
			}
			else if (UVSubsystem->IsObjectValidTarget(Object))
			{
				ObjectsOut.AddUnique(Object);
			}
		}
	}
}
}
