// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorModule.h"

#include "Features/IModularFeatures.h"
#include "ContentBrowserMenuContexts.h"
#include "EditorModeRegistry.h"
#include "LevelEditorMenuContext.h"
#include "PropertyEditorModule.h"
#include "Selection.h"
#include "UVEditor.h"
#include "UVEditor3DViewportMode.h"
#include "UVEditorCommands.h"
#include "UVEditorMode.h"
#include "UVEditorStyle.h"
#include "UVEditorSubsystem.h"
#include "UVSelectTool.h"

#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "FUVEditorModule"

namespace UVEditorModuleLocals
{
	void BlockUntilAssetsReady(const TArray<TObjectPtr<UObject>>& Assets)
	{
		for (const TObjectPtr<UObject>& Asset : Assets)
		{
			if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
			{
				// This blocks until the lods are ready
				StaticMesh->GetNumSourceModels();
			}
		}
	}
}

void FUVEditorModule::StartupModule()
{
	UVEditorAssetEditor = MakeShared<UE::Geometry::FUVEditorModularFeature>();

	if (UVEditorAssetEditor.IsValid())
	{
		IModularFeatures::Get().RegisterModularFeature(IUVEditorModularFeature::GetModularFeatureName(), UVEditorAssetEditor.Get());
	}

	FUVEditorStyle::Get(); // Causes the constructor to be called
	FUVEditorCommands::Register();
	UE::Geometry::FUVEditorToolActionCommands::RegisterAllToolActions();

	// Menus need to be registered in a callback to make sure the system is ready for them.
	UToolMenus::RegisterStartupCallback(
		FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FUVEditorModule::RegisterMenus));

	// Register details view customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	ClassesToUnregisterOnShutdown.Reset();
	// Customizations get registered like this:
	//PropertyModule.RegisterCustomClassLayout(USelectToolActionPropertySet::StaticClass()->GetFName(), 
	//	FOnGetDetailCustomizationInstance::CreateStatic(&FUVSelectToolActionPropertySetDetails::MakeInstance));
	//ClassesToUnregisterOnShutdown.Add(USelectToolActionPropertySet::StaticClass()->GetFName());
}

void FUVEditorModule::ShutdownModule()
{
	if (UVEditorAssetEditor.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(IUVEditorModularFeature::GetModularFeatureName(), UVEditorAssetEditor.Get());
		UVEditorAssetEditor = nullptr;
	}

	// Clean up menu things
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	UE::Geometry::FUVEditorToolActionCommands::UnregisterAllToolActions();
	FUVEditorCommands::Unregister();

	FEditorModeRegistry::Get().UnregisterMode(UUVEditorMode::EM_UVEditorModeId);
	FEditorModeRegistry::Get().UnregisterMode(UUVEditor3DViewportMode::EM_ModeID);

	// Unregister customizations
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (FName ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}
	}
}

void FUVEditorModule::RegisterMenus()
{
	using namespace UVEditorModuleLocals;

	// Allows cleanup when module unloads.
	FToolMenuOwnerScoped OwnerScoped(this);

	// Extend the content browser context menu for static meshes and skeletal meshes
	auto AddToContextMenuSection = [this](FToolMenuSection& Section)
	{
		Section.AddDynamicEntry("OpenUVEditor", FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& Section)
			{
				// We'll need to get the target assets out of the context
				if (UContentBrowserAssetContextMenuContext* Context = Section.FindContext<UContentBrowserAssetContextMenuContext>())
				{
					UUVEditorSubsystem* UVSubsystem = GEditor->GetEditorSubsystem<UUVEditorSubsystem>();
					check(UVSubsystem);

					// We are deliberately not using Context->GetSelectedObjects() here to avoid triggering a load from right clicking
					// an asset in the content browser.
					if (UVSubsystem->AreAssetsValidTargets(Context->SelectedAssets))
					{
						TSharedPtr<class FUICommandList> CommandListToBind = MakeShared<FUICommandList>();
						CommandListToBind->MapAction(
							FUVEditorCommands::Get().OpenUVEditor,
							FExecuteAction::CreateWeakLambda(UVSubsystem, [Context, UVSubsystem]()
							{
								// When we actually do want to open the editor, trigger the load to get the objects
								TArray<TObjectPtr<UObject>> AssetsToEdit;
								AssetsToEdit.Append(Context->LoadSelectedObjects<UObject>());

								// The checks we do above are done on asset data, and the UVSubsystem->AreObjectsValidTargets call
								//  we do below is done on loaded objects. However, it would be possible for the former to pass while
								//  the latter fails if the target is a static mesh that is valid, but is currently being compiled 
								//  (see the early out in UStaticMeshReadOnlyToolTarget::HasNonGeneratedLOD).
								// What we want to do is let the static mesh finish compiling if needed, which is what this call does.
								BlockUntilAssetsReady(AssetsToEdit);

								// If we fail the ensure here, then there must be something that we're failing to check properly
								// in AreAssetsValidTargets that we would need to track down and check in the asset data.
								if (ensure(UVSubsystem->AreObjectsValidTargets(AssetsToEdit)))
								{
									UVSubsystem->StartUVEditor(AssetsToEdit);
								}
							}),
							FCanExecuteAction::CreateWeakLambda(Context, [Context]() { return Context->bCanBeModified; }));

						const TAttribute<FText> ToolTipOverride = Context->bCanBeModified ? TAttribute<FText>() : LOCTEXT("ReadOnlyAssetWarning", "The selected asset(s) are read-only and cannot be edited.");
						Section.AddMenuEntryWithCommandList(FUVEditorCommands::Get().OpenUVEditor, CommandListToBind, TAttribute<FText>(), ToolTipOverride, FSlateIcon(FUVEditorStyle::Get().GetStyleSetName(), "UVEditor.OpenUVEditor"));
					}
				}
			}));
	};
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.StaticMesh");
		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		AddToContextMenuSection(Section);
	}
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.SkeletalMesh");
		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		AddToContextMenuSection(Section);
	}

	// Extend the level editor menus

	// Helper struct to collect all necessary inputs for a menu entry.
	struct FMenuEntryParameters
	{
		TArray<TObjectPtr<UObject>> TargetObjects;
		UUVEditorSubsystem* UVSubsystem;
		bool bValidTargets;
	};

	// Sets up all parameters for a menu entry.
	auto SetupMenuEntryParameters = []
	{
		FMenuEntryParameters Result;
		Result.UVSubsystem = GEditor->GetEditorSubsystem<UUVEditorSubsystem>();
		Result.bValidTargets = false;
		if (!Result.UVSubsystem)
		{
			return Result;
		}

		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			const AActor* Actor = static_cast<AActor*>(*It);
			TInlineComponentArray<UActorComponent*> ActorComponents;
			Actor->GetComponents(ActorComponents);

			for (UActorComponent* ActorComponent : ActorComponents)
			{
				if (Result.UVSubsystem->IsObjectValidTarget(ActorComponent))
				{
					Result.TargetObjects.Add(ActorComponent);
				}
			}
		}

		Result.bValidTargets = Result.TargetObjects.Num() > 0;
		return Result;
	};

	// Adds a menu entry for the given parameters.
	auto AddMenuEntry = [](FToolMenuSection& Section, const FMenuEntryParameters& MenuEntryParameters)
	{
		const bool bValidTargets = MenuEntryParameters.bValidTargets;

		const TSharedPtr<class FUICommandList> CommandListToBind = MakeShared<FUICommandList>();
		CommandListToBind->MapAction(
			FUVEditorCommands::Get().OpenUVEditor,
			FExecuteAction::CreateUObject(MenuEntryParameters.UVSubsystem, &UUVEditorSubsystem::StartUVEditor, MenuEntryParameters.TargetObjects),
			FCanExecuteAction::CreateLambda([bValidTargets]() { return bValidTargets; }));

		Section.AddMenuEntryWithCommandList(FUVEditorCommands::Get().OpenUVEditor, CommandListToBind);
	};

	// Add UV Editor to actor context menu.
	{
		UToolMenu* ActorContextMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.ActorContextMenu");
		FToolMenuSection& ActorTypeToolsSection = ActorContextMenu->FindOrAddSection("ActorTypeTools");
		ActorTypeToolsSection.AddDynamicEntry("OpenUVEditor", FNewToolMenuSectionDelegate::CreateLambda(
			                                      [&SetupMenuEntryParameters, &AddMenuEntry](FToolMenuSection& Section)
			                                      {
				                                      const FMenuEntryParameters MenuEntryParameters = SetupMenuEntryParameters();
				                                      if (MenuEntryParameters.bValidTargets)
				                                      {
					                                      AddMenuEntry(Section, MenuEntryParameters);
				                                      }
			                                      }));
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FUVEditorModule, UVEditor)