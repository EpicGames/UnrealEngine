// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorSubsystem.h"

#include "CoreMinimal.h"

#include "ToolTargetManager.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"
#include "ToolTargets/StaticMeshToolTarget.h"
#include "ToolTargets/SkeletalMeshToolTarget.h"
#include "ToolTargets/StaticMeshComponentToolTarget.h"
#include "ToolTargets/SkeletalMeshComponentToolTarget.h"

#include "ModelingToolTargetUtil.h"
#include "UVEditor.h"
#include "UVEditorMode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorSubsystem)

using namespace UE::Geometry;

static TAutoConsoleVariable<int32> CVarEnableLivePreviewArrangement(
	TEXT("modeling.UVEditor.EnableLivePreviewArrangement"),
	1,
	TEXT("Enable auto arranging objects in the UV Editor's live viewport when multiple objects are loaded from the Content Browser."));

static TAutoConsoleVariable<int32> CVarLivePreviewArrangementMode(
	TEXT("modeling.UVEditor.LivePreviewArrangementMode"),
	0,
	TEXT("Pattern to use for UV Editor Live Preview auto arrangement: 0 - Line, 1 - Circular"));

namespace UVSubsystemLocals
{
	void ArrangeAsLine(TArray<FTransform3d>& ObjectTransforms, TArray<double>& ObjectBoundingRadii, FVector3d& ObjectsAverageCenter)
	{
		double ObjectsMaxRadius = FMath::Max(ObjectBoundingRadii);

		// Increase all radii by a fraction of the average radius for all objects, to provide geometry relative padding between objects.
		ObjectsMaxRadius = ObjectsMaxRadius * 1.2;

		for (int32 TargetIndex = 0; TargetIndex < ObjectBoundingRadii.Num(); ++TargetIndex)
		{
			ObjectTransforms[TargetIndex] = FTransform3d(FVector(0, TargetIndex * ObjectsMaxRadius * 2, 0));
			ObjectsAverageCenter += FVector(TargetIndex * ObjectsMaxRadius, 0, 0);
		}

		ObjectsAverageCenter = ObjectsAverageCenter / ObjectBoundingRadii.Num();
	}

	void ArrangeAsCircles(TArray<FTransform3d>& ObjectTransforms, TArray<double>& ObjectBoundingRadii, FVector3d& ObjectsAverageCenter)
	{
		// If we need to initialize the transforms from bounding boxes, we begin a algorithm to sequentially place them in a spiral around the first object.
		// The algorithm proceeds as follows: 
		// 1. Place first object at (0,0,0)
		// 2. Place next object at a minimal touching distance in some direction
		// 3. Check all previously placed objects for overlap. If there's overlap, push the object along it's placement direction until it's clear
		// 4. Rotate the placement angle for next iteration by the amount that creates zero overlap between the current placing object and the following one.
		// 5. Goto step 2, unless there are no remaining objects to place. 
		//
		// The algorthim should result in a pattern similar to a hexgonal circle packing with uniform sized spheres, or an approximation of one for variable sized objects.

		double ObjectsMaxRadius = FMath::Max(ObjectBoundingRadii);

		// Increase all radii by a fraction of the average radius for all objects, to provide geometry relative padding between objects.
		ObjectsMaxRadius = ObjectsMaxRadius * 1.2;

		// TODO: For now, we're using the max radius for all objects.
		// However, the algorithm below is written for variable radii.
		// If we decide we like this option, we can simplify the below
		// for use with a constant radius for all objects.
		for (int32 Index = 0; Index < ObjectBoundingRadii.Num(); ++Index)
		{
			ObjectBoundingRadii[Index] = ObjectsMaxRadius;
		}

		// "Insert" first object at (0,0,0)
		ObjectsAverageCenter = FVector3d::ZeroVector;
		ObjectTransforms[0] = FTransform3d::Identity;
		double NextPlacementAngle = 0.0;

		for (int32 TargetIndex = 1; TargetIndex < ObjectBoundingRadii.Num(); ++TargetIndex)
		{
			// Compute next object placement ray and translation based on next placement angle and naive radii values.
			FVector3d TestPlacementRay = FVector3d(FMath::Cos(NextPlacementAngle), FMath::Sin(NextPlacementAngle), 0.0);
			FVector3d TestTranslation = ((ObjectBoundingRadii[0] + ObjectBoundingRadii[TargetIndex]) * TestPlacementRay);

			// Save the current radius of the "ring" we want to use to compute the next angle for.
			double TestRadius = ObjectBoundingRadii[0] + ObjectBoundingRadii[TargetIndex];

			for (int32 PriorIndex = 0; PriorIndex < TargetIndex; ++PriorIndex)
			{
				// Test the collision with prior placed object, moving the translation and radius until no other objects collide
				double CenterDistance = FVector3d::DistSquaredXY(TestTranslation, ObjectTransforms[PriorIndex].GetTranslation());
				double TestCollisionPenetration = CenterDistance - FMath::Square(ObjectBoundingRadii[TargetIndex] + ObjectBoundingRadii[PriorIndex]);
				if (TestCollisionPenetration < 0.0 && !FMath::IsNearlyZero(TestCollisionPenetration))
				{
					TestTranslation += TestPlacementRay * 2.0 * ObjectBoundingRadii[PriorIndex];
					TestRadius += ObjectBoundingRadii[PriorIndex];
				}
			}

			ObjectTransforms[TargetIndex] = FTransform3d(TestTranslation);
			ObjectsAverageCenter += TestTranslation;

			// Finally if there's more objects to place, advance the angle by with a little chord math.
			// The next position to fit a circle will be a chord on the "ring" the current placement is at. 
			// It is possible this choice will result in a collision next pass, but the algorithm will automatically
			// adjust for this as it goes.
			if (TargetIndex + 1 < ObjectBoundingRadii.Num())
			{
				double NextChordDistance = ObjectBoundingRadii[TargetIndex] + ObjectBoundingRadii[TargetIndex + 1];
				NextPlacementAngle = NextPlacementAngle + (2 * FMath::FastAsin(NextChordDistance / (2 * TestRadius)));
			}
		}

		ObjectsAverageCenter = ObjectsAverageCenter / ObjectBoundingRadii.Num();
	}
}

void UUVEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	// The subsystem has its own tool target manager because it must exist before any UV editors exist,
	// to see if a UV editor can be started.
	ToolTargetManager = NewObject<UToolTargetManager>(this);
	ToolTargetManager->Initialize();

	ToolTargetManager->AddTargetFactory(NewObject<UStaticMeshToolTargetFactory>(ToolTargetManager));
	ToolTargetManager->AddTargetFactory(NewObject<USkeletalMeshToolTargetFactory>(ToolTargetManager));
	ToolTargetManager->AddTargetFactory(NewObject<UDynamicMeshComponentToolTargetFactory>(ToolTargetManager));
	ToolTargetManager->AddTargetFactory(NewObject<UStaticMeshComponentToolTargetFactory>(ToolTargetManager));
	ToolTargetManager->AddTargetFactory(NewObject<USkeletalMeshComponentToolTargetFactory>(ToolTargetManager));
}

void UUVEditorSubsystem::Deinitialize()
{
	ToolTargetManager->Shutdown();
	ToolTargetManager = nullptr;
}

bool UUVEditorSubsystem::IsObjectValidTarget(const TObjectPtr<UObject> InObject) const
{
	return InObject && ToolTargetManager->CanBuildTarget(InObject, UUVEditorMode::GetToolTargetRequirements());
}

bool UUVEditorSubsystem::AreObjectsValidTargets(const TArray<UObject*>& InObjects) const
{
	if (InObjects.IsEmpty())
	{
		return false;
	}

	for (UObject* Object : InObjects)
	{
		if (!IsObjectValidTarget(Object))
		{
			return false;
		}
	}

	return true;
}

bool UUVEditorSubsystem::IsAssetValidTarget(const FAssetData& InAsset) const
{
	// Currently our tool target factories don't evaluate FAssetData to figure out whether they can
	// build a tool target (they only work on UObjects directly), so for now we do corresponding checks
	// here ourselves.
	auto IsValidStaticMeshAsset = [](const FAssetData& AssetData)
	{
		// The static mesh tool target checks GetNumSourceModels, which we can't do directly, hence our check of the LODs tag
		int32 NumLODs = 0;
		return AssetData.IsInstanceOf<UStaticMesh>() && AssetData.GetTagValue<int32>("LODs", NumLODs) && NumLODs > 0;
	};
	auto IsValidSkeletalMeshAsset = [](const FAssetData& AssetData)
	{
		// The skeletal mesh tool targets don't seem to try to check the number of LODs, but the skeletal mesh tool target
		// uses an exact cast for the class, hence the '==' comparison here.
		return AssetData.GetClass() == USkeletalMesh::StaticClass();
	};

	return IsValidStaticMeshAsset(InAsset) || IsValidSkeletalMeshAsset(InAsset);
}

bool UUVEditorSubsystem::AreAssetsValidTargets(const TArray<FAssetData>& InAssets) const
{
	if (InAssets.IsEmpty())
	{
		return false;
	}

	for (const FAssetData& AssetData : InAssets)
	{
		if (!IsAssetValidTarget(AssetData))
		{
			return false;
		}
	}

	return true;
}

void UUVEditorSubsystem::BuildTargets(const TArray<TObjectPtr<UObject>>& ObjectsIn, 
	const FToolTargetTypeRequirements& TargetRequirements, TArray<TObjectPtr<UToolTarget>>& TargetsOut,
	TArray<int32>* ObjectToTargetMapOut)
{
	TargetsOut.Reset();
	if (ObjectToTargetMapOut)
	{
		ObjectToTargetMapOut->SetNum(ObjectsIn.Num());
	}

	// Map from Object to Index into Target Array
	TMap<UObject*, uint32> TargetAssets;

	for (int32 ObjectIndex = 0; ObjectIndex < ObjectsIn.Num(); ++ObjectIndex)
	{
		UObject* Object = ObjectsIn[ObjectIndex];

		if (UToolTarget* Target = ToolTargetManager->BuildTarget(Object, TargetRequirements))
		{
			// If this is an asset backed target, only add it if the asset is not already added to the list.
			if (IAssetBackedTarget* AssetTarget = Cast<IAssetBackedTarget>(Target))
			{
				UObject* SourceData = AssetTarget->GetSourceData();
				if (!TargetAssets.Contains(SourceData))
				{
					TargetsOut.Add(Target);
					TargetAssets.Add(SourceData, TargetsOut.Num()-1);
				}

				if (ObjectToTargetMapOut)
				{
					(*ObjectToTargetMapOut)[ObjectIndex] = TargetAssets[SourceData];
				}
			}
			else
			{
				TargetsOut.Add(Target);
				if (ObjectToTargetMapOut)
				{
					(*ObjectToTargetMapOut)[ObjectIndex] = (TargetsOut.Num() - 1);
				}
			}


		}
		else
		{
			if (ObjectToTargetMapOut)
			{
				(*ObjectToTargetMapOut)[ObjectIndex] = -1;
			}
		}
	}
}

void UUVEditorSubsystem::StartUVEditor(TArray<TObjectPtr<UObject>> ObjectsToEdit)
{
	// We don't allow opening a new instance if any of the objects are already opened
	// in an existing instance. Instead, we bring such an instance to the front.
	// Note that the asset editor subsystem takes care of this for "primary" asset editors, 
	// i.e., the editors that open when one double clicks an asset or selects "edit". Since
	// the UV editor is not a "primary" asset editor for any asset type, we do this management 
	// ourselves.
	for (TObjectPtr<UObject>& Object : ObjectsToEdit)
	{
		if (OpenedEditorInstances.Contains(Object))
		{
			OpenedEditorInstances[Object]->GetInstanceInterface()->FocusWindow(Object);
			return;
		}
	}

	// If we got here, there's not an instance already opened.

	// We should have done a check upstream to make sure that all of our targets are valid, but
	// we'll check again here.
	if (!ensure(AreObjectsValidTargets(ObjectsToEdit)))
	{
		return;
	}

	TArray<FTransform3d> ObjectTransforms;
	TArray<double> ObjectBoundingRadii;
	TArray<UToolTarget*> Targets;
	FVector3d ObjectsAverageCenter = FVector3d::ZeroVector;

	bool bInitializeTransformsFromBoundingBoxes = true;

	// TODO: The below logic needs to be moved out of the Subsystem and into the UV Editor proper when we
	// have a better understanding of how we want to integrate the 3D arrangement logic into the UX of the editor.

	for (UObject* Object : ObjectsToEdit)
	{
		UToolTarget* Target = ToolTargetManager->BuildTarget(Object, UUVEditorMode::GetToolTargetRequirements());
		Targets.Add(Target);


		// There are two options here: A) all targets are components (selected from the viewport) or B) all targets aren't components (selected from the content browser). 
		// TODO: Check if it's possible to have a mixture?
		// Depending on which scenario we're in, we'll either use the component transforms relative to each other, or build some transforms ourselves.
		UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);
		if (Component)
		{
			bInitializeTransformsFromBoundingBoxes = false;
			FTransform3d Transform = UE::ToolTarget::GetLocalToWorldTransform(Target);
			ObjectTransforms.Add(Transform);
			ObjectsAverageCenter += Transform.GetTranslation();
		}
		else
		{
			FDynamicMesh3 Mesh = UE::ToolTarget::GetDynamicMeshCopy(Target);
			double MaxXYRadius = FMath::Max(Mesh.GetBounds().Dimension(0), Mesh.GetBounds().Dimension(1)) * 0.5;
			ObjectBoundingRadii.Add(MaxXYRadius);
			ObjectTransforms.Add(FTransform3d::Identity);
		}
	}	


	bool bEnableAutoArrangement = (CVarEnableLivePreviewArrangement.GetValueOnGameThread() > 0);
	if (bEnableAutoArrangement && bInitializeTransformsFromBoundingBoxes && Targets.Num() > 0)
	{
		switch (CVarLivePreviewArrangementMode.GetValueOnGameThread())
		{
		case 0: 
			UVSubsystemLocals::ArrangeAsLine(ObjectTransforms, ObjectBoundingRadii, ObjectsAverageCenter);
			break;
		case 1:
			UVSubsystemLocals::ArrangeAsCircles(ObjectTransforms, ObjectBoundingRadii, ObjectsAverageCenter);
			break;
		default:
			ensureMsgf(false, TEXT("No valid arrangement CVAR selected for UV Editor auto-arrange. Defaulting to no arrangement (legacy behavior).") );
			break;
		}
	}
	else
	{
		ObjectsAverageCenter = ObjectsAverageCenter / ObjectsToEdit.Num();
	}
	
	
	for (FTransform3d& Transform: ObjectTransforms)
	{
		Transform.SetTranslation(Transform.GetTranslation() - ObjectsAverageCenter);
	}


	UUVEditor* UVEditor = NewObject<UUVEditor>(this);

	// Among other things, this call registers the UV editor with the asset editor subsystem,
	// which will prevent it from being garbage collected.
	UVEditor->Initialize(ObjectsToEdit, ObjectTransforms);

	for (TObjectPtr<UObject>& Object : ObjectsToEdit)
	{
		OpenedEditorInstances.Add(Object, UVEditor);
	}
}

void UUVEditorSubsystem::NotifyThatUVEditorClosed(TArray<TObjectPtr<UObject>> ObjectsItWasEditing)
{
	for (TObjectPtr<UObject>& Object : ObjectsItWasEditing)
	{
		OpenedEditorInstances.Remove(Object);
	}
}


