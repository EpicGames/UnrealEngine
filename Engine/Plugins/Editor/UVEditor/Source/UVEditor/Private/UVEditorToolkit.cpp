// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorToolkit.h"

#include "AdvancedPreviewScene.h"
#include "AssetEditorModeManager.h"
#include "ContextObjectStore.h"
#include "EditorViewportTabContent.h"
#include "Misc/MessageDialog.h"
#include "PreviewScene.h"
#include "SAssetEditorViewport.h"
#include "SUVEditor2DViewport.h"
#include "SUVEditor3DViewport.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ToolContextInterfaces.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "ToolMenus.h"
#include "UVEditor.h"
#include "UVEditor2DViewportClient.h"
#include "UVEditor3DViewportClient.h"
#include "UVEditorCommands.h"
#include "UVEditorMode.h"
#include "UVEditorModeToolkit.h"
#include "UVEditorSubsystem.h"
#include "UVEditorModule.h"
#include "UVEditorStyle.h"
#include "UVEditor3DViewportMode.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "UVEditorModeUILayer.h"
#include "Widgets/Docking/SDockTab.h"
#include "UVEditorUXSettings.h"

#include "SLevelViewport.h"

#include "Tools/EdModeInteractiveToolsContext.h"

#define LOCTEXT_NAMESPACE "UVEditorToolkit"

const FName FUVEditorToolkit::InteractiveToolsPanelTabID(TEXT("UVEditor_InteractiveToolsTab"));
const FName FUVEditorToolkit::LivePreviewTabID(TEXT("UVEditor_LivePreviewTab"));

FUVEditorToolkit::FUVEditorToolkit(UAssetEditor* InOwningAssetEditor)
	: FBaseAssetToolkit(InOwningAssetEditor)
{
	check(Cast<UUVEditor>(InOwningAssetEditor));

	// We will replace the StandaloneDefaultLayout that our parent class gave us with 
	// one where the properties detail panel is a vertical column on the left, and there
	// are two viewports on the right.
	// We define explicit ExtensionIds on the stacks to reference them later when the
    // UILayer provides layout extensions. 
	//
	// Note: Changes to the layout should include a increment to the layout's ID, i.e.
	// UVEditorLayout[X] -> UVEditorLayout[X+1]. Otherwise, layouts may be messed up
	// without a full reset to layout defaults inside the editor.
	StandaloneDefaultLayout = FTabManager::NewLayout(FName("UVEditorLayout2"))
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)				
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)					
					->SetExtensionId("EditorSidePanelArea")
					->SetHideTabWell(true)
				)				
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.4f)
					->AddTab(ViewportTabID, ETabState::OpenedTab)
					->SetExtensionId("Viewport2DArea")
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.4f)
					->AddTab(LivePreviewTabID, ETabState::OpenedTab)
					->SetExtensionId("Viewport3DArea")
					->SetHideTabWell(true)
				)
			)
		);
		
	// Add any extenders specified by the UStaticMeshEditorUISubsystem
    // The extenders provide defined locations for FModeToolkit to attach
    // tool palette tabs and detail panel tabs
	LayoutExtender = MakeShared<FLayoutExtender>();
	FUVEditorModule* UVEditorModule = &FModuleManager::LoadModuleChecked<FUVEditorModule>("UVEditor");
	UVEditorModule->OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
	StandaloneDefaultLayout->ProcessExtensions(*LayoutExtender);

	// This API object serves as a communication point between the viewport toolbars and the tools.
    // We create it here so that we can pass it both into the 2d & 3d viewports and when we initialize 
    // the mode.
	ViewportButtonsAPI = NewObject<UUVToolViewportButtonsAPI>();

	UVTool2DViewportAPI = NewObject<UUVTool2DViewportAPI>();

	// We could create the preview scenes in CreateEditorViewportClient() the way that FBaseAssetToolkit
	// does, but it seems more intuitive to create them right off the bat and pass it in later. 
	FPreviewScene::ConstructionValues PreviewSceneArgs;
	UnwrapScene = MakeUnique<FPreviewScene>(PreviewSceneArgs);
	LivePreviewScene = MakeUnique<FAdvancedPreviewScene>(PreviewSceneArgs);
	LivePreviewScene->SetFloorVisibility(false, true);

	LivePreviewEditorModeManager = MakeShared<FAssetEditorModeManager>();
	LivePreviewEditorModeManager->SetPreviewScene(LivePreviewScene.Get());
	LivePreviewEditorModeManager->SetDefaultMode(UUVEditor3DViewportMode::EM_ModeID);
	LivePreviewInputRouter = LivePreviewEditorModeManager->GetInteractiveToolsContext()->InputRouter;

	LivePreviewTabContent = MakeShareable(new FEditorViewportTabContent());
	LivePreviewViewportClient = MakeShared<FUVEditor3DViewportClient>(
		LivePreviewEditorModeManager.Get(), LivePreviewScene.Get(), nullptr, ViewportButtonsAPI);

	LivePreviewViewportDelegate = [this](FAssetEditorViewportConstructionArgs InArgs)
	{
		return SNew(SUVEditor3DViewport, InArgs)
			.EditorViewportClient(LivePreviewViewportClient);
	};


}

FUVEditorToolkit::~FUVEditorToolkit()
{
	// We need to force the uv editor mode deletion now because otherwise the preview and unwrap worlds
	// will end up getting destroyed before the mode's Exit() function gets to run, and we'll get some
	// warnings when we destroy any mode actors.
	EditorModeManager->DestroyMode(UUVEditorMode::EM_UVEditorModeId);

	// The UV subsystem is responsible for opening/focusing UV editor instances, so we should
	// notify it that this one is closing.
	UUVEditorSubsystem* UVSubsystem = GEditor->GetEditorSubsystem<UUVEditorSubsystem>();
	if (UVSubsystem)
	{
		TArray<TObjectPtr<UObject>> ObjectsWeWereEditing;
		OwningAssetEditor->GetObjectsToEdit(MutableView(ObjectsWeWereEditing));
		UVSubsystem->NotifyThatUVEditorClosed(ObjectsWeWereEditing);
	}
}

// This gets used to label the editor's tab in the window that opens.
FText FUVEditorToolkit::GetToolkitName() const
{
	const TArray<UObject*>* Objects = GetObjectsCurrentlyBeingEdited();
	if (Objects->Num() == 1)
	{
		return FText::Format(LOCTEXT("UVEditorTabNameWithObject", "UV: {0}"), 
			GetLabelForObject((*Objects)[0]));
	}
	return LOCTEXT("UVEditorMultipleTabName", "UV: Multiple");
}

// TODO: This may need changing once the editor team determines the proper course of action here.
// In other asset editor toolkits, this would usually always return the same string, such
// as "UVEditor". However, FAssetEditorToolkit::GetToolMenuToolbarName, which uses
// FAssetEditorToolkit::GetToolMenuAppName (both are non-virtual) falls through to GetToolkitFName
// for non-primary editors, rather than giving a unique name based on the edited UObject as it does
// for primary editors that edit a single asset. We need to be able to customize the toolbar differently
// for different instances of the UV editor to display different channel selection options, so we need 
// the toolbar name to be instance-dependent, hence the unique name in GetToolkitFName here.
FName FUVEditorToolkit::GetToolkitFName() const
{
	return FName(FString::Printf(TEXT("UVEditor%p"), this));
}

// TODO: What is this actually used for?
FText FUVEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("UVBaseToolkitName", "UV");
}

FText FUVEditorToolkit::GetToolkitToolTipText() const
{
	FString ToolTipString;
	ToolTipString += LOCTEXT("ToolTipAssetLabel", "Asset").ToString();
	ToolTipString += TEXT(": ");

	const TArray<UObject*>* Objects = GetObjectsCurrentlyBeingEdited();
	if (Objects && Objects->Num() > 0)
	{
		ToolTipString += GetLabelForObject((*Objects)[0]).ToString();
		for (int32 i = 1; i < Objects->Num(); ++i)
		{
			ToolTipString += TEXT(", ");
			ToolTipString += GetLabelForObject((*Objects)[i]).ToString();
		}
	}
	else
	{
		// This can occur if our targets have been deleted externally to the UV Editor.
		// It's a bad state, but one we can avoid crashing in by doing this.
		ToolTipString += TEXT("<NO OBJECT>");
	}
	return FText::FromString(ToolTipString);
}

void FUVEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// We bypass FBaseAssetToolkit::RegisterTabSpawners because it doesn't seem to provide us with
	// anything except tabs that we don't want.
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	UVEditorMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_UVEditor", "UV Editor"));

	// Here we set up the tabs we referenced in StandaloneDefaultLayout (in the constructor).
	// We don't deal with the toolbar palette here, since this is handled by existing
    // infrastructure in FModeToolkit. We only setup spawners for our custom tabs, namely
    // the 2D and 3D viewports.
	InTabManager->RegisterTabSpawner(ViewportTabID, FOnSpawnTab::CreateSP(this, &FUVEditorToolkit::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("2DViewportTabLabel", "2D Viewport"))
		.SetGroup(UVEditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(LivePreviewTabID, FOnSpawnTab::CreateSP(this, 
		&FUVEditorToolkit::SpawnTab_LivePreview))
		.SetDisplayName(LOCTEXT("3DViewportTabLabel", "3D Viewport"))
		.SetGroup(UVEditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));
}

bool FUVEditorToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	// Note: This needs a bit of adjusting, because currently OnRequestClose seems to be 
	// called multiple times when the editor itself is being closed. We can take the route 
	// of NiagaraScriptToolkit and remember when changes are discarded, but this can cause
	// issues if the editor close sequence is interrupted due to some other asset editor.

	UUVEditorMode* UVMode = Cast<UUVEditorMode>(EditorModeManager->GetActiveScriptableMode(UUVEditorMode::EM_UVEditorModeId));
	if (!UVMode) {
		// If we don't have a valid mode, because the OnRequestClose is currently being called multiple times,
		// simply return true because there's nothing left to do.
		return true; 
	}

	// If the asset is being force deleted, we don't want to apply changes
	if(InCloseReason == EAssetEditorCloseReason::AssetForceDeleted)
	{
		return true;
	}

	bool bHasUnappliedChanges = UVMode->HaveUnappliedChanges();
	bool bCanApplyChanges = UVMode->CanApplyChanges();

	// Warn the user if there are unapplied changes *but* we can't currently save them.
	if (bHasUnappliedChanges && !bCanApplyChanges)
	{
		EAppReturnType::Type YesNoReply = FMessageDialog::Open(EAppMsgType::YesNo,
			NSLOCTEXT("UVEditor", "Prompt_UVEditorCloseCannotSave", "At least one of the assets has unapplied changes, however the UV Editor cannot currently apply changes due to current editor conditions. Do you still want to exit the UV Editor? (Selecting 'Yes' will cause all yet unapplied changes to be lost!)"));

		switch (YesNoReply)
		{
		case EAppReturnType::Yes:
			// exit without applying changes
			break;

		case EAppReturnType::No:
			// don't exit
			return false;
		}
	}

	// Warn the user of any unapplied changes.
	if (bHasUnappliedChanges && bCanApplyChanges)
	{
		TArray<TObjectPtr<UObject>> UnappliedAssets;
		UVMode->GetAssetsWithUnappliedChanges(UnappliedAssets);

		EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(EAppMsgType::YesNoCancel,
			NSLOCTEXT("UVEditor", "Prompt_UVEditorClose", "At least one of the assets has unapplied changes. Would you like to apply them? (Selecting 'No' will cause all changes to be lost!)"));

		switch (YesNoCancelReply)
		{
		case EAppReturnType::Yes:
			UVMode->ApplyChanges();
			break;

		case EAppReturnType::No:
			// exit
			break;

		case EAppReturnType::Cancel:
			// don't exit
			return false;
		}
	}

	return FAssetEditorToolkit::OnRequestClose(InCloseReason);
}

void FUVEditorToolkit::OnClose()
{
	// Give any active modes a chance to shutdown while the toolkit host is still alive
	// This is super important to do, otherwise currently opened tabs won't be marked as "closed".
	// This results in tabs not being properly recycled upon reopening the editor and tab
	// duplication for each opening event.
	GetEditorModeManager().DeactivateAllModes();

	FAssetEditorToolkit::OnClose();
}

// These get called indirectly (via toolkit host) from the mode toolkit when the mode starts or ends a tool,
// in order to add or remove an accept/cancel overlay.
void FUVEditorToolkit::AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder) 
{
	TSharedPtr<SUVEditor2DViewport> ViewportWidget = StaticCastSharedPtr<SUVEditor2DViewport>(ViewportTabContent->GetFirstViewport());
	ViewportWidget->AddOverlayWidget(InViewportOverlayWidget, ZOrder);
}
void FUVEditorToolkit::RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget)
{
	TSharedPtr<SUVEditor2DViewport> ViewportWidget = StaticCastSharedPtr<SUVEditor2DViewport>(ViewportTabContent->GetFirstViewport());
	ViewportWidget->RemoveOverlayWidget(InViewportOverlayWidget);
}

// We override the "Save" button behavior slightly to apply our changes before saving the asset.
void FUVEditorToolkit::SaveAsset_Execute()
{
	UUVEditorMode* UVMode = Cast<UUVEditorMode>(EditorModeManager->GetActiveScriptableMode(UUVEditorMode::EM_UVEditorModeId));
	if (ensure(UVMode) && UVMode->HaveUnappliedChanges())
	{
		UVMode->ApplyChanges();
	}

	FAssetEditorToolkit::SaveAsset_Execute();
}

bool FUVEditorToolkit::CanSaveAsset() const 
{
	UUVEditorMode* UVMode = Cast<UUVEditorMode>(EditorModeManager->GetActiveScriptableMode(UUVEditorMode::EM_UVEditorModeId));
	if(ensure(UVMode))	
	{
		return UVMode->CanApplyChanges();
	}
	return false;	
}

bool FUVEditorToolkit::CanSaveAssetAs() const 
{
	return CanSaveAsset();
}


TSharedRef<SDockTab> FUVEditorToolkit::SpawnTab_LivePreview(const FSpawnTabArgs& Args)
{
	TSharedRef< SDockTab > DockableTab =
		SNew(SDockTab);

	const FString LayoutId = FString("UVEditorLivePreviewViewport");
	LivePreviewTabContent->Initialize(LivePreviewViewportDelegate, DockableTab, LayoutId);
	return DockableTab;
}

// This is bound in RegisterTabSpawners() to create the panel on the left. The panel is filled in by the mode.
TSharedRef<SDockTab> FUVEditorToolkit::SpawnTab_InteractiveToolsPanel(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> ToolsPanel = SNew(SDockTab);

	UUVEditorMode* UVMode = Cast<UUVEditorMode>(EditorModeManager->GetActiveScriptableMode(UUVEditorMode::EM_UVEditorModeId));
	if (!UVMode)
	{
		// This is where we will drop out on the first call to this callback, when the mode does not yet
		// exist. There is probably a place where we could safely intialize the mode to make sure that
		// it is around before the first call, but it seems cleaner to just do the mode initialization 
		// in PostInitAssetEditor and fill in the tab at that time.
		// Later calls to this callback will occur if the user closes and restores the tab, and they
		// will continue past this point to allow the tab to be refilled.
		return ToolsPanel;
	}
	TSharedPtr<FModeToolkit> UVModeToolkit = UVMode->GetToolkit().Pin();
	if (!UVModeToolkit.IsValid())
	{
		return ToolsPanel;
	}

	ToolsPanel->SetContent(UVModeToolkit->GetInlineContent().ToSharedRef());
	return ToolsPanel;
}

void FUVEditorToolkit::CreateWidgets()
{
	// This gets called during UAssetEditor::Init() after creation of the toolkit but before
	// calling InitAssetEditor on it. If we have custom mode-level toolbars we want to add,
	// they could potentially go here, but we still need to call the base CreateWidgets as well
	// because that calls things that make viewport client, etc.

	FBaseAssetToolkit::CreateWidgets();
}

// Called from FBaseAssetToolkit::CreateWidgets to populate ViewportClient, but otherwise only used 
// in our own viewport delegate.
TSharedPtr<FEditorViewportClient> FUVEditorToolkit::CreateEditorViewportClient() const
{
	// Note that we can't reliably adjust the viewport client here because we will be passing it
	// into the viewport created by the viewport delegate we get from GetViewportDelegate(), and
	// that delegate may (will) affect the settings based on FAssetEditorViewportConstructionArgs,
	// namely ViewportType.
	// Instead, we do viewport client adjustment in PostInitAssetEditor().
	check(EditorModeManager.IsValid());
	return MakeShared<FUVEditor2DViewportClient>(EditorModeManager.Get(), UnwrapScene.Get(), 
		UVEditor2DViewport, ViewportButtonsAPI, UVTool2DViewportAPI);
}

// Called from FBaseAssetToolkit::CreateWidgets. The delegate call path goes through FAssetEditorToolkit::InitAssetEditor
// and FBaseAssetToolkit::SpawnTab_Viewport.
AssetEditorViewportFactoryFunction FUVEditorToolkit::GetViewportDelegate()
{
	AssetEditorViewportFactoryFunction TempViewportDelegate = [this](FAssetEditorViewportConstructionArgs InArgs)
	{
		return SAssignNew(UVEditor2DViewport, SUVEditor2DViewport, InArgs)
			.EditorViewportClient(ViewportClient);
	};

	return TempViewportDelegate;
}

// Called from FBaseAssetToolkit::CreateWidgets.
void FUVEditorToolkit::CreateEditorModeManager()
{
	EditorModeManager = MakeShared<FAssetEditorModeManager>();

	// The mode manager is the authority on what the world is for the mode and the tools context,
	// and setting the preview scene here makes our GetWorld() function return the preview scene
	// world instead of the normal level editor one. Important because that is where we create
	// any preview meshes, gizmo actors, etc.
	StaticCastSharedPtr<FAssetEditorModeManager>(EditorModeManager)->SetPreviewScene(UnwrapScene.Get());
}

void FUVEditorToolkit::PostInitAssetEditor()
{

    // We setup the ModeUILayer connection here, since the InitAssetEditor is closed off to us. 
    // Other editors perform this step elsewhere, but this is our best location.
	TSharedPtr<class IToolkitHost> PinnedToolkitHost = ToolkitHost.Pin();
	check(PinnedToolkitHost.IsValid());
	ModeUILayer = MakeShareable(new FUVEditorModeUILayer(PinnedToolkitHost.Get()));
	ModeUILayer->SetModeMenuCategory( UVEditorMenuCategory );

	// Needed so that the live preview dummy mode is initialized enough to be able to route hotkeys
	LivePreviewEditorModeManager->SetToolkitHost(PinnedToolkitHost.ToSharedRef());

	TArray<TObjectPtr<UObject>> ObjectsToEdit;
	OwningAssetEditor->GetObjectsToEdit(MutableView(ObjectsToEdit));

	TArray<FTransform> ObjectTransforms;
	Cast<UUVEditor>(OwningAssetEditor)->GetWorldspaceRelativeTransforms(ObjectTransforms);

	// This static method call initializes the variety of contexts that UVEditorMode needs to be available in
	// the context store on Enter() to function properly.
	UUVEditorMode::InitializeAssetEditorContexts(*EditorModeManager->GetInteractiveToolsContext()->ContextObjectStore, 
		ObjectsToEdit, ObjectTransforms, *LivePreviewViewportClient, *LivePreviewEditorModeManager, 
		*ViewportButtonsAPI, *UVTool2DViewportAPI);

	// Currently, aside from setting up all the UI elements, the toolkit also kicks off the UV
	// editor mode, which is the mode that the editor always works in (things are packaged into
	// a mode so that they can be moved to another asset editor if necessary).
	// We need the UV mode to be active to create the toolbox on the left.
	check(EditorModeManager.IsValid());
	EditorModeManager->ActivateMode(UUVEditorMode::EM_UVEditorModeId);
	UUVEditorMode* UVMode = Cast<UUVEditorMode>(
		EditorModeManager->GetActiveScriptableMode(UUVEditorMode::EM_UVEditorModeId));
	check(UVMode);

	// Regardless of how the user has modified the layout, we're going to make sure that we have
	// a 2d viewport that will allow our mode to receive ticks.
    // We don't need to invoke the tool palette tab anymore, since this is handled by
    // underlying infrastructure.
	if (!TabManager->FindExistingLiveTab(ViewportTabID))
	{
		TabManager->TryInvokeTab(ViewportTabID);
	}

    // Note: We don't have to force the live viewport to be open, but if we don't, we need to
	// make sure that any future live preview api functionality does not crash when the viewport
	// is missing, because some viewport client functions are not robust to that case.For now,
	// we'll just force it open because it is safer and seems to be more convenient for the user,
	// since reopening the window can be unintuitive, whereas closing it is easy.
	if (!TabManager->FindExistingLiveTab(LivePreviewTabID))
	{
		TabManager->TryInvokeTab(LivePreviewTabID);
	}

	// Add the "Apply Changes" button. It should actually be safe to do this almost
	// any time, even before that toolbar's registration, but it's easier to put most
	// things into PostInitAssetEditor().

	// TODO: We may consider putting actions like these, which are tied to a mode, into
	// some list of mode actions, and then letting the mode supply them to the owning
	// asset editor on enter/exit. Revisit when/if this becomes easier to do.
	ToolkitCommands->MapAction(
		FUVEditorCommands::Get().ApplyChanges,
		FExecuteAction::CreateUObject(UVMode, &UUVEditorMode::ApplyChanges),
		FCanExecuteAction::CreateUObject(UVMode, &UUVEditorMode::CanApplyChanges));
	FName ParentToolbarName;
	const FName ToolBarName = GetToolMenuToolbarName(ParentToolbarName);
	UToolMenu* AssetToolbar = UToolMenus::Get()->ExtendMenu(ToolBarName);
	FToolMenuSection& Section = AssetToolbar->FindOrAddSection("Asset");
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(FUVEditorCommands::Get().ApplyChanges));
	
	// Add the channel selection button.
	check(UVMode->GetToolkit().Pin());
	FUVEditorModeToolkit* UVModeToolkit = static_cast<FUVEditorModeToolkit*>(UVMode->GetToolkit().Pin().Get());
	Section.AddEntry(FToolMenuEntry::InitComboButton(
		"UVEditorChannelMenu",
		FUIAction(),
		FOnGetContent::CreateLambda([UVModeToolkit]()
		{
			return UVModeToolkit->CreateChannelMenu();
		}),
		LOCTEXT("UVEditorChannelMenu_Label", "Channels"),
		LOCTEXT("UVEditorChannelMenu_ToolTip", "Select the current UV Channel for each mesh"),
		FSlateIcon(FUVEditorStyle::Get().GetStyleSetName(), "UVEditor.ChannelSettings")
	));

	// Add the background settings button.
	Section.AddEntry(FToolMenuEntry::InitComboButton(
		"UVEditorBackgroundSettings",
		FUIAction(),
		FOnGetContent::CreateLambda([UVModeToolkit]()
		{
			
			TSharedRef<SVerticalBox> Container = SNew(SVerticalBox);

			Container->AddSlot()
				.AutoHeight()
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(SBox)
					.MinDesiredWidth(500)
				[
					UVModeToolkit->CreateGridSettingsWidget()
				]
				];

			Container->AddSlot()
				.AutoHeight()
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(SBox)
					.MinDesiredWidth(500)
				[
					UVModeToolkit->GetToolDisplaySettingsWidget()
				]
				];

			bool bEnableUDIMSupport = (FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport.GetValueOnGameThread() > 0);
			if (bEnableUDIMSupport)
			{
				Container->AddSlot()
					.AutoHeight()
					.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
					[
						SNew(SBox)
						.MinDesiredWidth(500)
					[
						UVModeToolkit->CreateUDIMSettingsWidget()
					]
					];
			}

			Container->AddSlot()
				.AutoHeight()
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(SBox)
					.MinDesiredWidth(500)
				[
					UVModeToolkit->CreateBackgroundSettingsWidget()
				]
				];

			Container->AddSlot()
				.AutoHeight()
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(SBox)
					.MinDesiredWidth(500)
				[
					UVModeToolkit->CreateUnwrappedUXSettingsWidget()
				]
				];

			Container->AddSlot()
				.AutoHeight()
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(SBox)
					.MinDesiredWidth(500)
				[
					UVModeToolkit->CreateLivePreviewUXSettingsWidget()
				]
				];

			Container->AddSlot()
				.AutoHeight()
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(SBox)
					.MinDesiredWidth(500)
				[
					UVModeToolkit->CreateDistortionVisualsSettingsWidget()
				]
				];

			// TODO: We're moving this to the bottom because there's currently a bug with menu handling where mousing
			// over this entry will break other menus in the overall display menu. 
			// The bug causes premature menu closure, preventing settings from being applied to properties.

			FMenuBuilder MenuBuilder(true, TSharedPtr<FUICommandList>(), TSharedPtr<FExtender>(), true);
			MenuBuilder.BeginSection("Section_Gizmo", LOCTEXT("Section_Gizmo", "Gizmo"));

			MenuBuilder.AddSubMenu(
				LOCTEXT("GizmoTransformPanelSubMenu", "Gizmo Transform Panel"), LOCTEXT("GizmoTransformPanelSubMenu_ToolTip", "Configure the Gizmo Transform Panel."),
				FNewMenuDelegate::CreateLambda([UVModeToolkit](FMenuBuilder& SubMenuBuilder) {
					UVModeToolkit->MakeGizmoNumericalUISubMenu(SubMenuBuilder);
				}));

			MenuBuilder.EndSection();


			Container->AddSlot()
			.AutoHeight()
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SBox)
				.MinDesiredWidth(500)
				[
					MenuBuilder.MakeWidget()
				]
			];

			TSharedRef<SWidget> Widget = SNew(SBorder)
				.HAlign(HAlign_Fill)
				.Padding(4)
				[
					Container
				];

			return Widget;
		}),
		LOCTEXT("UVEditorBackgroundSettings_Label", "Display"),
		LOCTEXT("UVEditorBackgroundSettings_ToolTip", "Change the background display settings"),
		FSlateIcon(FUVEditorStyle::Get().GetStyleSetName(), "UVEditor.BackgroundSettings")
	));


	// Adjust our main (2D) viewport:
	auto SetCommonViewportClientOptions = [](FEditorViewportClient* Client)
	{
		// Normally the bIsRealtime flag is determined by whether the connection is remote, but our
		// tools require always being ticked.
		Client->SetRealtime(true);

		// Disable motion blur effects that cause our renders to "fade in" as things are moved
		Client->EngineShowFlags.SetTemporalAA(false);
		Client->EngineShowFlags.SetMotionBlur(false);

		// Disable the dithering of occluded portions of gizmos.
		Client->EngineShowFlags.SetOpaqueCompositeEditorPrimitives(true);

		// Disable hardware occlusion queries, which make it harder to use vertex shaders to pull materials
		// toward camera for z ordering because non-translucent materials start occluding themselves (once
		// the component bounds are behind the displaced geometry).
		Client->EngineShowFlags.SetDisableOcclusionQueries(true);
	};
	SetCommonViewportClientOptions(ViewportClient.Get());

	// Note about using perspective instead of ortho projection for a 2d viewport:
	// Originally, we had various issues with the materials that we were using in ortho mode. Most of
	// those issues seem to have been resolved since then, and an example of using a proper 2d viewport
	// can be seen in ClothEditorRestSpaceViewportClient.cpp and ClothEditorToolkit.cpp. The benefits of
	// doing so are viewport controls out of the box, and ability to switch the viewport type at will.
	// 
	// At this point, however, the UV editor would not get much benefit from converting to a true ortho
	// viewport, and the conversion carries with it some amount of risk. At time of last experimentation 
	// with the conversion (1/3/2023), the materials seemed to all work, but the gizmo failed to draw
	// (unknown reason), and the grid number labels needed fixing.
	// 
	// Until we run into issues, we will continue to use perspective projection to avoid the conversion
	// cost/risks and to keep the advantages of more trustworthy rendering, but if you use the editor as
	// a model, consider using an ortho viewport for the potential simplicity.
	ViewportClient->SetViewportType(ELevelViewportType::LVT_Perspective);

	// Lit gives us the most options in terms of the materials we can use.
	ViewportClient->SetViewMode(EViewModeIndex::VMI_Lit);

	// scale [0,1] to [0,ScaleFactor].
	// We set our camera to look downward, centered, far enough to be able to see the edges
	// with a 90 degree FOV
	FVector3d ViewLocation = FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(FVector2f(0.5, 0.5));
	ViewLocation.Z = FUVEditorUXSettings::UVMeshScalingFactor;
	ViewportClient->SetViewLocation(ViewLocation);
	
	// Note that the orientation at which we look at the world should match to the transformations
	// we set in FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition, etc.
	// Currently we choose to view the unwrap world with Z pointing at us, X pointing right, and 
	// Y pointing down, which matches the view given by LVT_OrthoXY, which we may someday decide to use.
	// Also, we want the U and V axes to lie along the X and Y axes, respectively, even if they are
	// flipped, because that makes the colors used in the gizmo and its numerical UI panel match up 
	// with what the user expects.
	ViewportClient->SetViewRotation(FRotator(-90, 0, -90));

	// If exposure isn't set to fixed, it will flash as we stare into the void
	ViewportClient->ExposureSettings.bFixed = true;

	// We need the viewport client to start out focused, or else it won't get ticked until
	// we click inside it.
	ViewportClient->ReceivedFocus(ViewportClient->Viewport);


	// Adjust our live preview (3D) viewport
	SetCommonViewportClientOptions(LivePreviewViewportClient.Get());
	LivePreviewViewportClient->ToggleOrbitCamera(true);

	// TODO: This should not be hardcoded
	LivePreviewViewportClient->SetViewLocation(FVector(-200, 100, 100));
	LivePreviewViewportClient->SetLookAtLocation(FVector(0, 0, 0));

	// Adjust camera view to focus on the scene
	UVMode->FocusLivePreviewCameraOnSelection();


	// Hook up the viewport command list to our toolkit command list so that hotkeys not
	// handled by our toolkit would be handled by the viewport (to allow us to use
	// whatever hotkeys the viewport registered after clicking in the detail panel or
	// elsewhere in the UV editor).
	// Note that the "Append" call for a command list should probably have been called
	// "AppendTo" because it adds the callee object as a child of the argument command list.
	// I.e. after looking in ToolkitCommands, we will look in the viewport command list.
	TSharedPtr<SUVEditor2DViewport> ViewportWidget = StaticCastSharedPtr<SUVEditor2DViewport>(ViewportTabContent->GetFirstViewport());
	ViewportWidget->GetCommandList()->Append(ToolkitCommands);
}

const FSlateBrush* FUVEditorToolkit::GetDefaultTabIcon() const
{
	return FUVEditorStyle::Get().GetBrush("UVEditor.OpenUVEditor");
}

FLinearColor FUVEditorToolkit::GetDefaultTabColor() const
{
	return FLinearColor::White;
}

void FUVEditorToolkit::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingStarted(Toolkit);
}

void FUVEditorToolkit::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingFinished(Toolkit);
}

#undef LOCTEXT_NAMESPACE
