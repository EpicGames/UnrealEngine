// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "InteractiveTool.h"
#include "UVEditorMode.h"

#include "UVEditorUXPropertySets.generated.h"

/**
* Visualization settings for the UUVEditorMode's Unwrapped Visualization
 */
UCLASS(MinimalAPI)
class UUVEditorUnwrappedUXProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category="Unwrapped Visualization", meta = (DisplayName = "Boundary Line Color"))
	TArray<FColor> BoundaryLineColors;

	UPROPERTY(EditAnywhere, Category="Unwrapped Visualization", meta = (UIMin = "0.0", UIMax = "10.0", ClampMin = "0.0", DisplayName = "Boundary Line Thickness"))
	float BoundaryLineThickness;

	// TODO: maybe opacity is the setting to change instead of/in addition to thickness? Could also have something to recognize line density and/or window size and adjust accordingly
	UPROPERTY(EditAnywhere, Category="Unwrapped Visualization", meta = (UIMin = "0.0", UIMax = "10.0", ClampMin = "0.0", DisplayName = "Wireframe Thickness"))
	float WireframeThickness;
};

/**
 * Visualization settings for the UUVEditorMode's properties in the Live Preview (3d) Viewport
 */
UCLASS(MinimalAPI)
class UUVEditorLivePreviewUXProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category="Live Preview Visualization", meta = (DisplayName = "Selection Line Color"))
	FColor SelectionColor;

	UPROPERTY(EditAnywhere, Category="Live Preview Visualization", meta = (UIMin = "0.0", UIMax = "10.0", ClampMin = "0.0", DisplayName = "Selection Line Thickness"))
	float SelectionLineThickness;

	UPROPERTY(EditAnywhere, Category="Live Preview Visualization", meta = (UIMin = "0.0", UIMax = "10.0", ClampMin = "0.0", DisplayName = "Selection Point Size"))
	float SelectionPointSize;
};
