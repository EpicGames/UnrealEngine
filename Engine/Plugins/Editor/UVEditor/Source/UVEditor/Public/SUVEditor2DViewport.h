// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SAssetEditorViewport.h"

#define UE_API UVEDITOR_API



class SUVEditor2DViewport : public SAssetEditorViewport
{
public:

	// SEditorViewport
	UE_API virtual void BindCommands() override;
	UE_API virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	UE_API bool IsWidgetModeActive(UE::Widget::EWidgetMode Mode) const override;
};

#undef UE_API
