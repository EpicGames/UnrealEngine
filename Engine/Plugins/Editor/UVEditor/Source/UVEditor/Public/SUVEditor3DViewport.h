// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SAssetEditorViewport.h"

#define UE_API UVEDITOR_API

/**
 * Viewport used for live preview in UV editor. Has a custom toolbar overlay at the top.
 */
class SUVEditor3DViewport : public SAssetEditorViewport 
{
public:

	// SAssetEditorViewport
	UE_API virtual void BindCommands() override;
	UE_API virtual TSharedPtr<SWidget> BuildViewportToolbar() override;

};

#undef UE_API
