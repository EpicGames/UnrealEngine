// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Tools/UAssetEditor.h"

#include "UVEditor.generated.h"

#define UE_API UVEDITOR_API


class FUICommandList;
class UEdModeInteractiveToolsContext;
class UInteractiveToolBuilder;
class UPreviewMesh;
class UUVEditorToolTargetManager;
class UToolTarget;
class UUVEditorMode;
class IUVUnwrapDynamicMesh;
class IAssetEditorInstance;

/** 
 * The actual asset editor class doesn't have that much in it, intentionally. 
 * 
 * Our current asset editor guidelines ask us to place as little business logic as possible
 * into the class, instead putting as much of the non-UI code into the subsystem as possible,
 * and the UI code into the toolkit (which this class owns).
 *
 * However, since we're using a mode and the Interactive Tools Framework, a lot of our business logic
 * ends up inside the mode and the tools, not the subsystem. The front-facing code is mostly in
 * the asset editor toolkit, though the mode toolkit has most of the things that deal with the toolbar
 * on the left.
 */
UCLASS(MinimalAPI)
class UUVEditor : public UAssetEditor
{
	GENERATED_BODY()

public:
	UE_API void Initialize(const TArray<TObjectPtr<UObject>>& InObjects);
	UE_API void Initialize(const TArray<TObjectPtr<UObject>>& InObjects, const TArray<FTransform3d>& InTransforms);

	UE_API void GetWorldspaceRelativeTransforms(TArray<FTransform3d>& OutTransforms);

	/** Returns the asset editor instance interface, so that its window can be focused, for example. */
	UE_API IAssetEditorInstance* GetInstanceInterface();

	// UAssetEditor overrides
	UE_API void GetObjectsToEdit(TArray<UObject*>& OutObjects) override;
	UE_API virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

	// Note that things like CloseWindow, FocusWindow, and IsPrimaryEditor seem to be called
	// on the toolkit, not the editor class, so they are overriden there.

protected:
	
	UPROPERTY()
	TArray<TObjectPtr<UObject>> OriginalObjectsToEdit;

	UPROPERTY()
	TArray<FTransform3d> ObjectWorldspaceOffsets;

	friend class UUVEditorSubsystem;
};

#undef UE_API
