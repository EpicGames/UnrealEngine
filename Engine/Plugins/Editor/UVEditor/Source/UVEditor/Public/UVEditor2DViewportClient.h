// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "EditorViewportClient.h"
#include "InputBehaviorSet.h"
#include "Behaviors/2DViewportBehaviorTargets.h" // FEditor2DScrollBehaviorTarget, FEditor2DMouseWheelZoomBehaviorTarget
#include "ContextObjects/UVToolViewportButtonsAPI.h" // UUVToolViewportButtonsAPI::ESelectionMode

#define UE_API UVEDITOR_API

class UCanvas;
class UUVTool2DViewportAPI;

/**
 * Client used to display a 2D view of the UV's, implemented by using a perspective viewport with a locked
 * camera.
 */
class FUVEditor2DViewportClient : public FEditorViewportClient, public IInputBehaviorSource
{
public:
	UE_API FUVEditor2DViewportClient(FEditorModeTools* InModeTools, FPreviewScene* InPreviewScene,
		const TWeakPtr<SEditorViewport>& InEditorViewportWidget, UUVToolViewportButtonsAPI* ViewportButtonsAPI, UUVTool2DViewportAPI* UVTool2DViewportAPI);

	virtual ~FUVEditor2DViewportClient() = default;

	UE_API bool AreSelectionButtonsEnabled() const;
	UE_API void SetSelectionMode(UUVToolViewportButtonsAPI::ESelectionMode NewMode);
	UE_API UUVToolViewportButtonsAPI::ESelectionMode GetSelectionMode() const;
	UE_API bool AreWidgetButtonsEnabled() const;

	UE_API void SetSnappingActive(bool bActive);
	UE_API bool IsSnappingActive();
	UE_API void SetLocationGridSnapEnabled(bool bEnabled);
	UE_API bool GetLocationGridSnapEnabled();
	UE_API void SetLocationGridSnapValue(float SnapValue);
	UE_API float GetLocationGridSnapValue();
	UE_API void SetRotationGridSnapEnabled(bool bEnabled);
	UE_API bool GetRotationGridSnapEnabled();
	UE_API void SetRotationGridSnapValue(float SnapValue);
	UE_API float GetRotationGridSnapValue();
	UE_API void SetScaleGridSnapEnabled(bool bEnabled);
	UE_API bool GetScaleGridSnapEnabled();
	UE_API void SetScaleGridSnapValue(float SnapValue);
	UE_API float GetScaleGridSnapValue();

	// FEditorViewportClient
	UE_API virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;

	UE_API virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual bool ShouldOrbitCamera() const override;
	UE_API bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override;
	UE_API void SetWidgetMode(UE::Widget::EWidgetMode NewMode) override;
	UE_API UE::Widget::EWidgetMode GetWidgetMode() const override;	
	UE_API void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;


	// Overriding base class visibility
	using FEditorViewportClient::OverrideNearClipPlane;

	// IInputBehaviorSource
	UE_API virtual const UInputBehaviorSet* GetInputBehaviors() const override;

	// FGCObject
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

protected:
	UE_API void DrawGrid(const FSceneView* View, FPrimitiveDrawInterface* PDI);
	UE_API void DrawGridRulers(FViewport& InViewport, FSceneView& View, UCanvas& Canvas);
	UE_API void DrawUDIMLabels(FViewport& InViewport, FSceneView& View, UCanvas& Canvas);

	// These get added in AddReferencedObjects for memory management
	TObjectPtr<UInputBehaviorSet> BehaviorSet;
	TObjectPtr<UUVToolViewportButtonsAPI> ViewportButtonsAPI;
	TObjectPtr<UUVTool2DViewportAPI> UVTool2DViewportAPI;

	bool bDrawGridRulers = true;
	bool bDrawGrid = true;
	TObjectPtr<UCanvas> CanvasObject;

	// Note that it's generally less hassle if the unique ptr types are complete here,
	// not forward declared, else we get compile errors if their destruction shows up
	// anywhere in the header.
	TUniquePtr<FEditor2DScrollBehaviorTarget> ScrollBehaviorTarget;
	TUniquePtr<FEditor2DMouseWheelZoomBehaviorTarget> ZoomBehaviorTarget;
};

#undef UE_API
