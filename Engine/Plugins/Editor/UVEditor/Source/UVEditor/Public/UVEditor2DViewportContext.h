// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UVEditor2DViewportClient.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"

#include "UVEditor2DViewportContext.generated.h"

UCLASS()
class UUVEditor2DViewportContext : public UUnrealEdViewportToolbarContext
{
	GENERATED_BODY()
	
public:

	UUVEditor2DViewportContext();

	virtual FText GetGridSnapLabel() const override;
	virtual TArray<float> GetGridSnapSizes() const override;
	virtual bool IsGridSnapSizeActive(int32 GridSizeIndex) const override;
	virtual void SetGridSnapSize(int32 GridSizeIndex) override;
	
	virtual FText GetRotationSnapLabel() const override;
	virtual bool IsRotationSnapActive(int32 RotationIndex, ERotationGridMode RotationMode) const override;
	virtual void SetRotationSnapSize(int32 RotationIndex, ERotationGridMode RotationMode) override;
	
	virtual FText GetScaleSnapLabel() const override;
	virtual bool IsScaleSnapActive(int32 ScaleIndex) const override;
	virtual void SetScaleSnapSize(int32 ScaleIndex) override;
	
protected:
	TSharedPtr<FUVEditor2DViewportClient> GetViewportClient() const;
	
	TArray<float> GridSnapSizes;
}; 
