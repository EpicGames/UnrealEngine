// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "EditorViewportClient.h"

#define UE_API UVEDITOR_API

class UUVToolViewportButtonsAPI;

// Types of camera motion for the UV Editor 3D viewport
enum EUVEditor3DViewportClientCameraMode {
	Orbit,
	Fly
};

/**
 * Viewport client for the 3d live preview in the UV editor. Currently same as editor viewport
 * client but doesn't allow editor gizmos/widgets, and alters orbit camera control.
 */
class FUVEditor3DViewportClient : public FEditorViewportClient
{
public:

	UE_API FUVEditor3DViewportClient(FEditorModeTools* InModeTools, FPreviewScene* InPreviewScene = nullptr,
		const TWeakPtr<SEditorViewport>& InEditorViewportWidget = nullptr, UUVToolViewportButtonsAPI* ViewportButtonsAPI = nullptr);

	virtual ~FUVEditor3DViewportClient() = default;

	// FEditorViewportClient
	UE_API virtual bool ShouldOrbitCamera() const override;
	bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override {	return false; }
	void SetWidgetMode(UE::Widget::EWidgetMode NewMode) override {}
	UE::Widget::EWidgetMode GetWidgetMode() const override { return UE::Widget::EWidgetMode::WM_None; }
	UE_API void FocusCameraOnSelection();
public:

	void SetCameraMode(EUVEditor3DViewportClientCameraMode CameraModeIn) { CameraMode = CameraModeIn; };
	EUVEditor3DViewportClientCameraMode GetCameraMode() const { return CameraMode; };

protected:

	// Enforce Orbit camera for UV editor live preview viewport. Use this instead of the base class orbit camera flag
	// to allow for expected behaviors of the base class when in fly camera mode.
	EUVEditor3DViewportClientCameraMode CameraMode = EUVEditor3DViewportClientCameraMode::Orbit;
	UUVToolViewportButtonsAPI* ViewportButtonsAPI;

};

#undef UE_API
