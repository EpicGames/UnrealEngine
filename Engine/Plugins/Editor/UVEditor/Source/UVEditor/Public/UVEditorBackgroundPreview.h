// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/TriangleSetComponent.h"
#include "InteractiveTool.h"
#include "GeometryBase.h"
#include "UVEditorBackgroundPreview.generated.h"

#define UE_API UVEDITOR_API


/**
 * Enum to control the background visualiztion mode
 */
UENUM()
enum class EUVEditorBackgroundSourceType
{
	Checkerboard,
	Texture,
	Material
};

/**
 * Visualization settings for the UUVEditorBackgroundPreview
 */
UCLASS(MinimalAPI)
class UUVEditorBackgroundPreviewProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Should the background be shown (Alt+B)*/
	UPROPERTY(EditAnywhere, Category = Background, meta = (DisplayName = "Display Background"))
	bool bVisible = false;

	/** Source of background visuals */
	UPROPERTY(EditAnywhere, Category = Background, meta = (EditCondition = "bVisible", DisplayName = "Background Source"))
	EUVEditorBackgroundSourceType SourceType = EUVEditorBackgroundSourceType::Checkerboard;

	/** Display a background based on the selected texture */
	UPROPERTY(EditAnywhere, Category = Background, meta = (EditCondition = "SourceType==EUVEditorBackgroundSourceType::Texture && bVisible", EditConditionHides = true))
	TObjectPtr<UTexture2D> SourceTexture;

	/** Display a background based on the selected material */
	UPROPERTY(EditAnywhere, Category = Background, meta = (EditCondition = "SourceType==EUVEditorBackgroundSourceType::Material && bVisible", EditConditionHides = true))
	TObjectPtr<UMaterial> SourceMaterial;

	UPROPERTY()
	TArray<int32> UDIMBlocks;

	UPROPERTY()
	bool bUDIMsEnabled = false;
};

/**
  Serves as a container for the background texture/material display in the UVEditor. This class is responsible for managing the quad
  drawn behind the grid, as well as maintaining the texture and material choices from the user to display.
 */
UCLASS(MinimalAPI, Transient)
class UUVEditorBackgroundPreview : public UPreviewGeometry
{
	GENERATED_BODY()

public:

	/**
	 * Client must call this every frame for changes to .Settings to be reflected in rendered result.
	 */
	UE_API void OnTick(float DeltaTime);
	
public:
	/** Visualization settings */
	UPROPERTY()
	TObjectPtr<UUVEditorBackgroundPreviewProperties> Settings;

	/** The component containing the quad visualization */
	UPROPERTY()
	TObjectPtr<UTriangleSetComponent> BackgroundComponent;

	/** The active material being displayed for the background */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> BackgroundMaterial;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBackgroundMaterialChange, TObjectPtr<UMaterialInstanceDynamic> MaterialInstance);
	FOnBackgroundMaterialChange OnBackgroundMaterialChange;

protected:
	UE_API virtual void OnCreated() override;

	bool bSettingsModified = false;

	UE_API void UpdateVisibility();
	UE_API void UpdateBackground();
};

#undef UE_API
