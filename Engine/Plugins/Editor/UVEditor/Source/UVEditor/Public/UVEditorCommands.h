// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Tools/InteractiveToolsCommands.h"

class UVEDITOR_API FUVEditorCommands : public TCommands<FUVEditorCommands>
{
public:

	FUVEditorCommands();

	// TCommands<> interface
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> OpenUVEditor;
	TSharedPtr<FUICommandInfo> ApplyChanges;

	TSharedPtr<FUICommandInfo> BeginLayoutTool;
	TSharedPtr<FUICommandInfo> BeginTransformTool;
	TSharedPtr<FUICommandInfo> BeginAlignTool;
	TSharedPtr<FUICommandInfo> BeginDistributeTool;
	TSharedPtr<FUICommandInfo> BeginTexelDensityTool;
	TSharedPtr<FUICommandInfo> BeginParameterizeMeshTool;
	TSharedPtr<FUICommandInfo> BeginChannelEditTool;
	TSharedPtr<FUICommandInfo> BeginSeamTool;
	TSharedPtr<FUICommandInfo> BeginRecomputeUVsTool;
	TSharedPtr<FUICommandInfo> BeginBrushSelectTool;
	TSharedPtr<FUICommandInfo> BeginUVSnapshotTool;

	TSharedPtr<FUICommandInfo> SewAction;
	TSharedPtr<FUICommandInfo> SplitAction;
	TSharedPtr<FUICommandInfo> MakeIslandAction;
	TSharedPtr<FUICommandInfo> UnsetUVsAction;

	TSharedPtr<FUICommandInfo> AcceptOrCompleteActiveTool;
	TSharedPtr<FUICommandInfo> CancelOrCompleteActiveTool;

	TSharedPtr<FUICommandInfo> VertexSelection;
	TSharedPtr<FUICommandInfo> EdgeSelection;
	TSharedPtr<FUICommandInfo> TriangleSelection;
	TSharedPtr<FUICommandInfo> IslandSelection;
	TSharedPtr<FUICommandInfo> FullMeshSelection;
	TSharedPtr<FUICommandInfo> SelectAll;

	TSharedPtr<FUICommandInfo> EnableOrbitCamera;
	TSharedPtr<FUICommandInfo> EnableFlyCamera;
	TSharedPtr<FUICommandInfo> SetFocusCamera;

	TSharedPtr<FUICommandInfo> ToggleBackground;
};

namespace UE::Geometry
{
//~ Modeled on ModelingToolsActions.h
class FUVEditorToolActionCommands : public TInteractiveToolCommands<FUVEditorToolActionCommands>
{
public:
	FUVEditorToolActionCommands();

	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;
	static void RegisterAllToolActions();
	static void UnregisterAllToolActions();

	/**
	 * Add or remove commands relevant to Tool to the given UICommandList.
	 * Call this when the active tool changes (eg on ToolManager.OnToolStarted / OnToolEnded)
	 * @param bUnbind if true, commands are removed, otherwise added
	 */
	static void UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind = false);
};
}

//~ Modeled on ModelingToolsActions.h
#define DECLARE_TOOL_ACTION_COMMANDS(CommandsClassName) \
class CommandsClassName : public TInteractiveToolCommands<CommandsClassName> \
{\
public:\
	CommandsClassName();\
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;\
};\

namespace UE::Geometry
{
	DECLARE_TOOL_ACTION_COMMANDS(FUVEditorBrushSelectToolCommands);
}