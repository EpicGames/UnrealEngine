// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/TriangleSetComponent.h"
#include "InteractiveTool.h"
#include "GeometryBase.h"
#include "ToolTargets/UVEditorToolMeshInput.h"

#include "UVEditorDistortionVisualization.generated.h"

#define UE_API UVEDITOR_API

UENUM()
enum class EDistortionMetric : uint8
{
	// Represents the shape deviation from the worldspace triangle element, sans scaling. 
	ReedBeta UMETA(DisplayName = "Elliptical Eccentricity"),
	// Represents the root-mean-square strech over all directions vs the worldspace triangle element
	Sander_L2 UMETA(DisplayName = "L2 Norm"),
	// Represents the worst-case, or greatest stretch in any particular direction, vs the worldspace triangle element
	Sander_LInf UMETA(DisplayName = "L Infinity Norm"),
	// Shows the current texile density ratio, compared against the average baseline of the entire mesh. 
	TexelDensity UMETA(DisplayName = "Texel Density")
};

/**
 * Visualization settings for the UUVEditorDistortionVisualization
 */
UCLASS(MinimalAPI)
class UUVEditorDistortionVisualizationProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Should the visualization be shown.*/
	UPROPERTY(EditAnywhere, Category = "Distortion Visualization", meta = (DisplayName = "Display UV Distortion Visualization"))
	bool bVisible = false;

	/** The specific metric used to classify and visualize distortion. */
	UPROPERTY(EditAnywhere, Category = "Distortion Visualization", meta = (DisplayName = "Metric"))
	EDistortionMetric Metric = EDistortionMetric::ReedBeta;

	//~ TODO - For now, we're disabling the user from changing this, as we're not going to support manual density targeting at the moment.
	/** When visualizing texel density, use the average texel density, assuming a constant arbitrary texture resolution, over the entire model as the baseline. Otherwise, specify exact densities for a specific target texture size.*/
	//~UPROPERTY(EditAnywhere, Category = "Distortion Visualization|Texel Density", meta = (DisplayName = "Use Average Baseline", EditCondition = "Metric == EDistortionMetric::TexelDensity"))

	UPROPERTY()
	bool bCompareToAverageDensity = true;

	/** When computing average texel density, take into account specific UDIM tile resolutions for a more accurate representation */
	UPROPERTY(EditAnywhere, Category = "Distortion Visualization|Texel Density", meta = (DisplayName = "Use UDIM Resolutions", EditCondition = "Metric == EDistortionMetric::TexelDensity"))
	bool bRespectUDIMTextureResolutions = false;

	//~ TODO - Enable support for manual density specificaitons at a future date
	//~UPROPERTY(EditAnywhere, Category = "Distortion Visualization|Texel Density", meta = (DisplayName = "Texture Map Dimension", EditCondition = "!bCompareToAverageDensity && !bRespectUDIMTextureResolutions && Metric == EDistortionMetric::TexelDensity"))

	UPROPERTY()
	int32 MapSize;

	//~ TODO - Enable support for manual density specificaitons at a future date	
	//~UPROPERTY(EditAnywhere, Category = "Distortion Visualization|Texel Density", meta = (DisplayName = "Target Texel Density", EditCondition = "!bCompareToAverageDensity && Metric == EDistortionMetric::TexelDensity", ClampMin = 0.0, UIMin = 0.0))

	UPROPERTY()
	float TargetTexelDensity;

	UPROPERTY()
	TMap<int32, int32> PerUDIMTextureResolution;
};

/**
  This class contains the logic for handling distortion visualization within the UV Editor. It manages the setting
  of triangle color and computing the potential metrics used to describe the amount of distortion present in the UVs.
 */
UCLASS(MinimalAPI, Transient)
class UUVEditorDistortionVisualization : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Client must call this after construction and parameter configuration to complete setup.
	 */
	UE_API void Initialize();

	/**
	 * Client must call this before destruction to clean up state.
	 */
	UE_API void Shutdown();

	/**
	 * Client must call this every frame for changes to .Settings to be reflected in rendered result.
	 */
	UE_API void OnTick(float DeltaTime);


	
public:

	UPROPERTY()
	TObjectPtr<UUVEditorDistortionVisualizationProperties> Settings;

	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

protected:
	bool bSettingsModified = false;

	TArray<double> PerTargetAverageTexelDensity;
	int32 MaximumTileTextureResolution;

	UE_API void UpdateVisibility();
	UE_API void ConfigureMeshColorsForTarget(int32 TargetIndex);
	UE_API FColor GetDistortionColorForTriangle(const UE::Geometry::FDynamicMesh3& Mesh, int UVChannel, int Tid, double TargetAverageTexelDensity);
	UE_API void ComputeInitialMeshSurfaceAreas();

};

#undef UE_API
