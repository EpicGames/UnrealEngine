// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ContextObjects/UVToolContextObjects.h"

#include "UVEditorInitializationContext.generated.h"

class FUICommandList;
class UEditorInteractiveToolsContext;

/**
 * An editor-only context used specifically to pass data from the asset editor to the
 * mode when that data is not meant to be used by the tools. This allows a mode
 * to be prepped before its first Enter() call even when it doesn't exist yet.
 * 
 * Data/api's that are meant to be used by the tools would usually be placed into
 * other, non-editor-only context objects (UUVToolLivePreviewAPI, etc).
 */
UCLASS(MinimalAPI)
class UUVEditorInitializationContext : public UUVToolContextObject
{
	GENERATED_BODY()
public:
	TWeakObjectPtr<UEditorInteractiveToolsContext> LivePreviewITC;
	TWeakPtr<FUICommandList> LivePreviewToolkitCommands;
};
