// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Tools/UEdMode.h"
#include "ToolTargets/ToolTarget.h" // FToolTargetTypeRequirements
#include "GeometryBase.h"
#include "InteractiveTool.h"
#include "InteractiveToolsContext.h"

#include "UVEditorMode.generated.h"

#define UE_API UVEDITOR_API

PREDECLARE_GEOMETRY(class FDynamicMesh3);

class FAssetEditorModeManager;
class FUICommandList;
class UEditorInteractiveToolsContext;
class FEditorViewportClient;
class FToolCommandChange;
class UContextObjectStore;
class UInteractiveToolPropertySet;
class UMeshElementsVisualizer;
class UMeshOpPreviewWithBackgroundCompute;
class UPreviewMesh;
class UTexture2D;
class UUVEditorToolMeshInput;
class UUVEditorBackgroundPreview;
class UUVEditorDistortionVisualization;
class UUVToolAction;
class UUVToolContextObject;
class UUVToolSelectionAPI;
class UUVToolViewportButtonsAPI;
class UUVTool2DViewportAPI;
class UUVEditorMode;
class UWorld;
class UUVEditorUnwrappedUXProperties;
class UUVEditorLivePreviewUXProperties;

/**
 * Visualization settings for the UUVEditorMode's Grid
 */
UCLASS(MinimalAPI)
class UUVEditorGridProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Should the grid be shown?*/
	UPROPERTY(EditAnywhere, Category = "Grid & Guides", meta = (DisplayName = "Display Grid"))
	bool bDrawGrid = true;

	/** Should the grid rulers be shown?*/
	UPROPERTY(EditAnywhere, Category = "Grid & Guides", meta = (DisplayName = "Display Rulers"))
	bool bDrawRulers = true;
};

USTRUCT()
struct FUDIMSpecifier
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Transient, Category = UDIM, meta = (ClampMin = "1001", UIMin = "1001"))
	int32 UDIM = 1001;

	UPROPERTY(VisibleAnywhere, Transient, Category = UDIM, meta = (DisplayName = "Block U Offset"))
	int32 UCoord = 0;

	UPROPERTY(VisibleAnywhere, Transient, Category = UDIM, meta = (DisplayName = "Block V Offset"))
	int32 VCoord = 0;

	UPROPERTY(EditAnywhere, Transient, Category = UDIM, meta = (DisplayName = "Block Texture Resolution"))
	int32 TextureResolution = 512;

	friend bool operator==(const FUDIMSpecifier& A, const FUDIMSpecifier& B )
	{
		return A.UDIM == B.UDIM && A.TextureResolution == B.TextureResolution;
	}

	friend uint32 GetTypeHash(const FUDIMSpecifier& UDIMSpecifier)
	{
		// We're only hashing on the UDIM index, as we don't want to ever have more than one entry with the same UDIM position
		uint32 HashCode = GetTypeHash(UDIMSpecifier.UDIM);
		return HashCode;
	}
};

UENUM()
enum class EUVEditorModeActions
{
	NoAction,

	ConfigureUDIMsFromAsset,
	ConfigureUDIMsFromTexture
};

/**
 * Settings for UDIMs in the UVEditor
 */
UCLASS(MinimalAPI)
class UUVEditorUDIMProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	void Initialize(UUVEditorMode* ParentModeIn) { ParentMode = ParentModeIn; }
	UE_API void PostAction(EUVEditorModeActions Action);

	UPROPERTY(EditAnywhere, Transient, Category = "UDIM", meta = (DisplayName = "UDIM Source Mesh", GetOptions = GetAssetNames))
	FString UDIMSourceAsset;

	UFUNCTION()
	UE_API const TArray<FString>& GetAssetNames();

	UFUNCTION()
	UE_API int32 AssetByIndex() const;

	/** Set UDIM Layout from selected asset's UVs */
	UFUNCTION(CallInEditor, Category = UDIM)
	void SetUDIMsFromAsset() { PostAction(EUVEditorModeActions::ConfigureUDIMsFromAsset); }

	/** Texture asset to source UDIM information from */
	UPROPERTY(EditAnywhere, Transient, Category = UDIM)
	TObjectPtr<UTexture2D> UDIMSourceTexture;

	/** Set UDIM Layout from selected texture asset */
	UFUNCTION(CallInEditor, Category = UDIM)
	void SetUDIMsFromTexture() { PostAction(EUVEditorModeActions::ConfigureUDIMsFromTexture); };

	/** Currently active UDIM set */
	UPROPERTY(EditAnywhere, Transient, Category = UDIM, meta = (TitleProperty = "UDIM"))
	TArray<FUDIMSpecifier> ActiveUDIMs;

public:
	UE_API void InitializeAssets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn);

private:
	UE_API void UpdateActiveUDIMsFromTexture();
	UE_API void UpdateActiveUDIMsFromAsset();

	TWeakObjectPtr<UUVEditorMode> ParentMode;
	TArray<FString> UVAssetNames;
};

namespace UVEditorModeChange
{
	class FApplyChangesChange;
}

/**
 * The UV editor mode is the mode used in the UV asset editor. It holds most of the inter-tool state.
 * We put things in a mode instead of directly into the asset editor in case we want to someday use the mode
 * in multiple asset editors.
 */
UCLASS(Transient)
class UUVEditorMode : public UEdMode
{
	GENERATED_BODY()

	friend UVEditorModeChange::FApplyChangesChange;
public:
	const static FEditorModeID EM_UVEditorModeId;

	UUVEditorMode();

	/**
	 * Gets the tool target requirements for the mode. The resulting targets undergo further processing
	 * to turn them into the input objects that tools get (since these need preview meshes, etc).
	 */
	static const FToolTargetTypeRequirements& GetToolTargetRequirements();

	/**
	 * Gets the factor by which UV layer unwraps get scaled (scaling makes certain things easier, like zooming in, etc).
	 */
	static double GetUVMeshScalingFactor();

	/**
	 * Called by an asset editor so that a created instance of the mode has all the data it needs on Enter() to initialize itself.
	 */
	static void InitializeAssetEditorContexts(UContextObjectStore& ContextStore,
		const TArray<TObjectPtr<UObject>>& AssetsIn, const TArray<FTransform>& TransformsIn,
		FEditorViewportClient& LivePreviewViewportClient, FAssetEditorModeManager& LivePreviewModeManager,
		UUVToolViewportButtonsAPI& ViewportButtonsAPI, UUVTool2DViewportAPI& UVTool2DViewportAPI);

	// Public for use by undo/redo. Otherwise should use RequestUVChannelChange
	void ChangeInputObjectLayer(int32 AssetID, int32 NewLayerIndex);

	/** 
	 * Request a change of the displayed UV channel/layer. It will happen on the next tick, and 
	 * create an undo/redo event.
	 */
	void RequestUVChannelChange(int32 AssetID, int32 Channel);

	bool IsActive() { return bIsActive; }

	// TODO: We'll probably eventually want a function like this so we can figure out how big our 3d preview
	// scene is and how we should position the camera intially...
	//FBoxSphereBounds GetLivePreviewBoundingBox() const;

	// Unlike UInteractiveToolManager::EmitObjectChange, emitting an object change using this
	// function does not cause it to expire when the active tool doesn't match the emitting tool.
	// It is important that the emitted change deals properly with expiration itself, for instance
	// expiring itself when a tool input is invalid or a contained preview is disconnected.
	// TODO: Should this sort of option exist in UInteractiveToolManager?
	void EmitToolIndependentObjectChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description);

	// Asset management
	bool HaveUnappliedChanges() const;
	bool CanApplyChanges() const;
	void GetAssetsWithUnappliedChanges(TArray<TObjectPtr<UObject>> UnappliedAssetsOut);
	void ApplyChanges();

	/** @return List of asset names, indexed by AssetID */
	const TArray<FString>& GetAssetNames() const { return AssetNames; }

	/** @return Number of UV channels in the given asset, or IndexConstants::InvalidID if AssetID was invalid.  */
	int32 GetNumUVChannels(int32 AssetID) const;

	/** @return The index of the channel currently displayed for the given AssetID. */
	int32 GetDisplayedChannel(int32 AssetID) const;

	/** @return A settings object suitable for display in a details panel to control the background visualization. */
	UObject* GetBackgroundSettingsObject();

	/** @return A settings object suitable for display in a details panel to control the grid. */
	UObject* GetGridSettingsObject();

	/** @return A settings object suitable for display in a details panel to control the unwrapped viewport visualization. */
	UObject* GetUnwrappedUXSettingsObject() const;

	/** @return A settings object suitable for display in a details panel to control the visualization in the live preview (3d) viewport. */
	UObject* GetLivePreviewUXSettingsObject() const;

	/** @return A settings object suitable for display in a details panel to control UDIM configuration. */
	UObject* GetUDIMSettingsObject();

	/** @return A settings object suitable for display in a details panel to control the active tool's visualization settings. */
	UObject* GetToolDisplaySettingsObject();

	/** @return A settings object suitable for display in a details panel to control the UV distortion visualization settings. */
	UObject* GetDistortionVisualsSettingsObject();

	virtual void Render(IToolsContextRenderAPI* RenderAPI) ;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	// UEdMode overrides
	virtual void Enter() override;
	virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const override;
	virtual void Exit() override;
	virtual void ModeTick(float DeltaTime) override;
	// We're changing visibility of this one to public here so that we can call it from the toolkit
	// when clicking accept/cancel buttons. We don't want to friend the toolkit because we don't want
	// it to later get (accidentally) more entangled with mode internals. At the same time, we're not
	// sure whether we want to make the UEdMode one public. So this is the minimal-impact tweak.
	virtual void ActivateDefaultTool() override;

	// This is currently not part of the base class at all... Should it be?
	virtual bool IsDefaultToolActive();

	// We don't actually override MouseEnter, etc, because things get forwarded to the input
	// router via FEditorModeTools, and we don't have any additional input handling to do at the mode level.

	UPROPERTY()
	TObjectPtr<UUVEditorBackgroundPreview> BackgroundVisualization;

	UPROPERTY()
	TObjectPtr<UUVEditorDistortionVisualization> DistortionVisualization;

	// Hold a settings object to configure the grid
	UPROPERTY()
	TObjectPtr<UUVEditorGridProperties> UVEditorGridProperties = nullptr;

	// Hold a settings object to configure the UDIMs
	UPROPERTY()
	TObjectPtr<UUVEditorUDIMProperties> UVEditorUDIMProperties = nullptr;

	void PopulateUDIMsByAsset(int32 AssetId, TArray<FUDIMSpecifier>& UDIMsOut) const;

	void FocusLivePreviewCameraOnSelection();

protected:
	UPROPERTY()
	TArray<TObjectPtr<UUVToolAction>> RegisteredActions;

	void InitializeModeContexts();
	void InitializeTargets();
	void RegisterTools();
	void RegisterActions();
	void SetSimulationWarning(bool bEnabled);

	// UEdMode overrides
	virtual void CreateToolkit() override;
	// Not sure whether we need these yet
	virtual void BindCommands() override;
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	
	UE_DEPRECATED(5.3, "UpdateTriangleMaterialBasedOnBackground is deprecated, replaced by the more generic UpdateTriangleMaterialBasedOnDisplaySettings.")
	void UpdateTriangleMaterialBasedOnBackground(bool IsBackgroundVisible);
	void UpdatePreviewMaterialBasedOnBackground();
	void UpdateTriangleMaterialBasedOnDisplaySettings();

	void UpdateActiveUDIMs();
	int32 UDIMsChangedWatcherId;

	/**
	 * Stores original input objects, for instance UStaticMesh pointers. AssetIDs on tool input 
	 * objects are indices into this array (and ones that are 1:1 with it)
	 */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> OriginalObjectsToEdit;

	/**
	 * Tool targets created from OriginalObjectsToEdit (and 1:1 with that array) that provide
	 * us with dynamic meshes whose UV layers we unwrap.
	 */
	UPROPERTY()
	TArray<TObjectPtr<UToolTarget>> ToolTargets;

	/**
	 * Transforms that should be used for the 3d previews, 1:1 with OriginalObjectsToEdit
	 * and ToolTargets.
	 */
	TArray<FTransform> Transforms;

	/**
	 * 1:1 with the asset arrays. Hold dynamic mesh representations of the targets. These
	 * are authoritative versions of the combined UV layers that get baked back on apply.
	 */
	TArray<TSharedPtr<UE::Geometry::FDynamicMesh3>> AppliedCanonicalMeshes;

	/**
	 * 1:1 with AppliedCanonicalMeshes, the actual displayed 3d meshes that can be used
	 * by tools for background computations. However if doing so, keep in mind that while
	 * we currently do not display more than one layer at a time for each asset, if we
	 * someday do, tools would have to take care to disallow cases where the two layers
	 * of the same asset might try to use the same preview for a background compute.
	 */
	TArray<TObjectPtr<UMeshOpPreviewWithBackgroundCompute>> AppliedPreviews;

	/**
	 * Input objects we give to the tools, one per displayed UV layer. This includes pointers
	 * to the applied meshes, but also contains the unwrapped mesh and preview. These should
	 * not be assumed to be the same length as the asset arrays in case we someday do not
	 * display exactly a single layer per asset.
	 */
	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> ToolInputObjects;

	/**
	 * Wireframes have to get ticked to be able to respond to setting changes.
	 * This is 1:1 with ToolInputObjects.
	 */
	TArray<TWeakObjectPtr<UMeshElementsVisualizer>> WireframesToTick;


	// Authoritative list of targets that have changes that have not been baked back yet.
	TSet<int32> ModifiedAssetIDs;

	// 1:1 with ToolTargets, indexed by AssetID
	TArray<FString> AssetNames;

	// Used with the ToolAssetAndLayerAPI to process tool layer change requests
	void SetDisplayedUVChannels(const TArray<int32>& LayerPerAsset, bool bEmitUndoTransaction);

	TArray<int32> PendingUVLayerIndex;

	// Here largely for convenience to avoid having to pass it around functions.
	UPROPERTY()
	TObjectPtr<UWorld> LivePreviewWorld = nullptr;

	// Used to forward Render/DrawHUD calls in the live preview to the api object.
	TWeakObjectPtr<UEditorInteractiveToolsContext> LivePreviewITC;

	//~ Other members should be private as well, but too late for that. Keeping this one here
	//~  to be next to the other live preview members.
private: 
	// Used to make mode hotkeys be triggered properly from the live preview
	TWeakPtr<FUICommandList> LivePreviewToolkitCommands;
protected:

	UPROPERTY()
	TObjectPtr<UUVToolSelectionAPI> SelectionAPI = nullptr;

	/**
	 * Mode-level property objects (visible or not) that get ticked.
	 */
	UPROPERTY()
	TArray<TObjectPtr<UInteractiveToolPropertySet>> PropertyObjectsToTick;

	TArray<TWeakObjectPtr<UUVToolContextObject>> ContextsToUpdateOnToolEnd;
	TArray<TWeakObjectPtr<UUVToolContextObject>> ContextsToShutdown;

	bool bIsActive = false;
	FString DefaultToolIdentifier;

	static FDateTime AnalyticsLastStartTimestamp;

	// Holds references to PIE callbacks to handle logic when the PIE session starts & shuts down
	bool bPIEModeActive;
	FDelegateHandle BeginPIEDelegateHandle;
	FDelegateHandle EndPIEDelegateHandle;
	FDelegateHandle CancelPIEDelegateHandle;

	// Holds references to Save callbacks to handle logic when the autosave triggers and shuts down active tools. 
	// We need to recover from this, so we restart the select tool after the save is over.
	FDelegateHandle PostSaveWorldDelegateHandle;
private:
	// Hold a settings object to configure the unwrapped visualization
	UPROPERTY()
	TObjectPtr<UUVEditorUnwrappedUXProperties> UVEditorUnwrappedUXProperties = nullptr;

	// Hold a settings object to configure the visualization in the live preview (3d) viewport
	UPROPERTY()
	TObjectPtr<UUVEditorLivePreviewUXProperties> UVEditorLivePreviewUXProperties = nullptr;

	TArray<FString> ToolsThatAllowActions;
};

#undef UE_API
