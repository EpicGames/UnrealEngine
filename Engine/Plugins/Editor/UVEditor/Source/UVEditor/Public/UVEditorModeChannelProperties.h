// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h"
#include "GeometryBase.h"

#include "UVEditorModeChannelProperties.generated.h"

#define UE_API UVEDITOR_API

PREDECLARE_GEOMETRY(class FDynamicMesh3);

class UInteractiveToolPropertySet;
class UToolTarget;
class UUVEditorToolMeshInput;

/**
 * UV Layer Settings for the UVEditorMode
 */
UCLASS(MinimalAPI)
class UUVEditorUVChannelProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = UVChannel, meta = (DisplayName = "Asset", GetOptions = GetAssetNames))
	FString Asset;

	UFUNCTION()
	UE_API const TArray<FString>& GetAssetNames();

	UPROPERTY(EditAnywhere, Category = UVChannel, meta = (DisplayName = "UV Channel", GetOptions = GetUVChannelNames))
	FString UVChannel;

	UFUNCTION()
	UE_API const TArray<FString>& GetUVChannelNames();

	TArray<FString> UVChannelNames;
	TArray<FString> UVAssetNames;

	// 1:1 with UVAssetNames
	TArray<int32> NumUVLayersPerAsset;


public:
	UE_API void Initialize(
		TArray<TObjectPtr<UToolTarget>>& TargetObjects,
		TArray<TSharedPtr<UE::Geometry::FDynamicMesh3>> AppliedCanonicalMeshes,
		bool bInitializeSelection);

	/**
	 * Verify that the UV asset selection is valid
	 * @param bUpdateIfInvalid if selection is not valid, reset UVAsset to Asset0 or empty if no assets exist
	 * @return true if selection in UVAsset is an entry in UVAssetNames.
	 */
	UE_API bool ValidateUVAssetSelection(bool bUpdateIfInvalid);

	/**
	 * Verify that the UV channel selection is valid
	 * @param bUpdateIfInvalid if selection is not valid, UVChannel to UV0 or empty if no UV channels exist
	 * @return true if selection in UVChannel is an entry in UVChannelNamesList.
	 */
	UE_API bool ValidateUVChannelSelection(bool bUpdateIfInvalid);


	/**
	 * @return selected UV asset ID, or -1 if invalid selection
	 */
	UE_API int32 GetSelectedAssetID();

	/**
	 * @param bForceToZeroOnFailure if true, then instead of returning -1 we return 0 so calling code can fallback to default UV paths
	 * @return selected UV channel index, or -1 if invalid selection, or 0 if invalid selection and bool bForceToZeroOnFailure = true
	 */
	UE_API int32 GetSelectedChannelIndex(bool bForceToZeroOnFailure = false);
};

#undef UE_API
