// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h"
#include "Toolkits/BaseToolkit.h"
#include "StatusBarSubsystem.h"

struct FSlateBrush;
class SBorder;
class STextBlock;
class STransformGizmoNumericalUIOverlay;
class UInteractiveToolPropertySet;

/**
 * The UV editor mode toolkit is responsible for the panel on the side in the UV editor
 * that shows mode and tool properties. Tool buttons would go in Init().
 */
class FUVEditorModeToolkit : public FModeToolkit
{
public:
	FUVEditorModeToolkit();
	~FUVEditorModeToolkit();

	virtual void InitializeAfterModeSetup();

	/** Creates a menu where the displayed UV Channel can be changed for each asset */
	virtual TSharedRef<SWidget> CreateChannelMenu();
	
	/** Creates a widget where the background visualization can be changed. */
	virtual TSharedRef<SWidget> CreateBackgroundSettingsWidget();

	/** Creates a widget where the grid display can be changed. */
	virtual TSharedRef<SWidget> CreateGridSettingsWidget();

	/** Creates a widget where the visualization in the 2d/unwrapped view can be changed. */
	virtual TSharedRef<SWidget> CreateUnwrappedUXSettingsWidget();

	/** Creates a widget where the visualization properties in the 3d viewport can be changed. */
	virtual TSharedRef<SWidget> CreateLivePreviewUXSettingsWidget();
	
	/** Creates a widget where the visible UDIMs can be changed. */
	virtual TSharedRef<SWidget> CreateUDIMSettingsWidget();

	/** Creates a widget where the active tool's display settings can be changed. */
	virtual TSharedRef<SWidget> GetToolDisplaySettingsWidget();

	/** Creates submenu for manipulating the gizmo numerical UI */
	virtual void MakeGizmoNumericalUISubMenu(FMenuBuilder& MenuBuilder);

	/** Creates a widget where the distortion visualization settings can be changed. */
	virtual TSharedRef<SWidget> CreateDistortionVisualsSettingsWidget();

	/** Enables or diables the Play in Editor mode warning for the editor */
	virtual void EnableShowPIEWarning(bool bEnable);

	// FModeToolkit
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
	virtual FText GetActiveToolDisplayName() const override { return ActiveToolName; }
	/** Returns the Mode specific tabs in the mode toolbar **/
	virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const override;
    /** Returns human readable name for the specific palette tool category **/
	virtual FText GetToolPaletteDisplayName(FName PaletteName) const override;
    /** For a specific tool palette category, construct and fill ToolbarBuilder with the category's tools **/
	virtual void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder) override;
	/** Informs the built in FModeToolkit palette building routines about how we want the toolbars to be setup **/
	virtual bool HasIntegratedToolPalettes() const override { return true; }
	virtual bool HasExclusiveToolPalettes() const override { return false; }

	// IToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual TSharedPtr<class SWidget> GetInlineContent() const override { return ToolkitWidget; }

	// set/clear notification message area
	virtual void PostNotification(const FText& Message);
	virtual void ClearNotification();

	// set/clear warning message area
	virtual void PostWarning(const FText& Message);
	virtual void ClearWarning();

	// set/clear mode-level warning message area. These messages don't get cleared on tool shutdown.
	virtual void PostModeWarning(const FText& Message);
	virtual void ClearModeWarning();

protected:
	const static TArray<FName> PaletteNames_Standard;

	TSharedRef<SWidget> CreateDisplaySettingsWidget(UObject* SettingsObject) const;
	void UpdatePIEWarnings();

	// FModeToolkit
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;


	// The mode's entire toolbox, which gets returned by GetInlineContent()
	TSharedPtr<SWidget> ToolkitWidget;

	// The portion of the toolbox composed of buttons that activate tools
	TSharedPtr<SBorder> ToolButtonsContainer;

	// A place for tools to write out any warnings
	TSharedPtr<STextBlock> ToolWarningArea;

	// A place for the mode to write out any warnings
	TSharedPtr<STextBlock> ModeWarningArea;

	// A container for the tool settings that is populated by the DetailsView managed
	// in FModeToolkit
	TSharedPtr<SBorder> ToolDetailsContainer;

	// A container for the editor settings
	TSharedPtr<SBorder> EditorDetailsContainer;

	/** Contains the widget container for the Accept/Cancel buttons for tools. */
	TSharedPtr<SWidget> ViewportOverlayWidget;

	TSharedPtr<STransformGizmoNumericalUIOverlay> GizmoNumericalUIOverlayWidget;

	FText ActiveToolName;
	const FSlateBrush* ActiveToolIcon;
	FStatusBarMessageHandle ActiveToolMessageHandle;
	bool bShowPIEWarning;

	void UpdateActiveToolProperties();
	void InvalidateCachedDetailPanelState(UObject* ChangedObject);

	bool bFirstInitializeAfterModeSetup = true;
};