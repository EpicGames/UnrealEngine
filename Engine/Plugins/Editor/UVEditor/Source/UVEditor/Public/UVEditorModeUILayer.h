// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorModeUILayer.h"
#include "UVEditorModeUILayer.generated.h"

/** Interchange layer to manage built in tab locations within the editor's layout. **/
UCLASS()
class UUVEditorUISubsystem : public UAssetEditorUISubsystem
{
	GENERATED_BODY()
public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) override;
};

/** Handles the hosting of additional toolkits, such as the mode toolkit, within the UVEditor's toolkit. **/
class FUVEditorModeUILayer : public FAssetEditorModeUILayer
{
public:
	FUVEditorModeUILayer(const IToolkitHost* InToolkitHost);
	void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;

	void SetModeMenuCategory(TSharedPtr<FWorkspaceItem> MenuCategoryIn);
	TSharedPtr<FWorkspaceItem> GetModeMenuCategory() const override;

protected:
	TSharedPtr<FWorkspaceItem> UVEditorMenuCategory;

};
