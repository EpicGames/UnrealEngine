// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryProcessingInterfaces/IUVEditorModularFeature.h"

#define UE_API UVEDITOR_API

namespace UE
{
namespace Geometry
{

/** 
 * Connector class that allows other plugins to look for the UV Editor to see if the plugin is present,
 * (via IModularFeatures::Get().GetModularFeature and related methods), and then launch it if so.
 */
class FUVEditorModularFeature : public IUVEditorModularFeature
{
public:
	UE_API virtual void LaunchUVEditor(const TArray<TObjectPtr<UObject>>& Objects) override;
	UE_API virtual bool CanLaunchUVEditor(const TArray<TObjectPtr<UObject>>& Objects) override;

protected:
	UE_API void ConvertInputArgsToValidTargets(const TArray<TObjectPtr<UObject>>& ObjectsIn, TArray<TObjectPtr<UObject>>& ObjectsOut) const;

};


}
}

#undef UE_API
