// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#include "UVEditorModularFeature.h"

#define UE_API UVEDITOR_API

class FLayoutExtender;

/**
 * Besides the normal module things, the module class is also responsible for hooking the 
 * UV editor into existing menus.
 */
class FUVEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	DECLARE_EVENT_OneParam(FUVEditorModule, FOnRegisterLayoutExtensions, FLayoutExtender&);
	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() { return RegisterLayoutExtensions; }

protected:
	UE_API void RegisterMenus();

private:
	FOnRegisterLayoutExtensions	RegisterLayoutExtensions;

	/** StaticClass is not safe on shutdown, so we cache the name, and use this to unregister on shut down */
	TArray<FName> ClassesToUnregisterOnShutdown;

	TSharedPtr<UE::Geometry::FUVEditorModularFeature> UVEditorAssetEditor;
};

#undef UE_API
