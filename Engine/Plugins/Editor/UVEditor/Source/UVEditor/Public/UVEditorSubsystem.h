// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "EditorSubsystem.h"
#include "ToolTargets/ToolTarget.h" //FToolTargetTypeRequirements

#include "GeometryBase.h"

#include "UVEditorSubsystem.generated.h"

#define UE_API UVEDITOR_API

PREDECLARE_GEOMETRY(class FAssetDynamicMeshTargetComboInterface);

class UToolTargetManager;
class UUVEditor;
class UToolTarget;

/**
 * The subsystem is meant to hold any UV editor operations that are not UI related (those are
 * handled by the toolkit); however, in our case, most of our operations are wrapped up inside
 * tools and the UV mode. 
 * Instead, the subsystem deals with some global UV asset editor things- it manages existing
 * instances (we can't rely on the asset editor subsystem for this because the UV editor is
 * not the primary editor for meshes), and figures out what we can launch the editor on.
 */
UCLASS(MinimalAPI)
class UUVEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	/** Checks that the object is a valid target for a UV editor session. */
	bool IsObjectValidTarget(const TObjectPtr<UObject> InObject) const;

	/** Checks that all of the objects are valid targets for a UV editor session. */
	UE_API virtual bool AreObjectsValidTargets(const TArray<UObject*>& InObjects) const;

	/** Checks that the asset is a valid target for a UV Editor session. */
	bool IsAssetValidTarget(const FAssetData& InAsset) const;

	/**
	 * Checks that all of the assets are valid targets for an editor session. This
	 * is preferable over AreObjectsValidTargets when we have FAssetData because it
	 * allows us to avoid forcing a load of the underlying UObjects (for instance to
	 * avoid triggering a load when right clicking an asset in the content browser).
	 */
	UE_API virtual bool AreAssetsValidTargets(const TArray<FAssetData>& InAssets) const;

	/** 
	 * Tries to build the core targets that provide meshes for UV tools to work on.
	 * 
	 * ObjectToTargetMapOut provides a mapping from the input Object set to the output Target set,
	 * since these might not be 1:1 and we may need to know which objects are represented by a single
	 * source target
	 */
	UE_API virtual void BuildTargets(const TArray<TObjectPtr<UObject>>& ObjectsIn, 
		const FToolTargetTypeRequirements& TargetRequirements,
		TArray<TObjectPtr<UToolTarget>>& TargetsOut,
		TArray<int32>* ObjectToTargetMapOut = nullptr);

	/**
	 * Either brings to the front an existing UV editor instance that is editing one of
	 * these objects, if one exists, or starts up a new instance editing all of these 
	 * objects.
	 */
	UE_API virtual void StartUVEditor(TArray<TObjectPtr<UObject>> ObjectsToEdit);

	/** 
	 * Needs to be called when a UV editor instance is closed so that the subsystem knows
	 * to create a new one for these objects if they are opened again.
	 */
	UE_API virtual void NotifyThatUVEditorClosed(TArray<TObjectPtr<UObject>> ObjectsItWasEditing);

protected:

	/**
	 * Used to let the subsystem figure out whether targets are valid. New factories should be
	 * added here in Initialize() as they are developed.
	 */
	UPROPERTY()
	TObjectPtr<UToolTargetManager> ToolTargetManager = nullptr;

	TMap<TObjectPtr<UObject>, TObjectPtr<UUVEditor>> OpenedEditorInstances;
};

#undef UE_API
