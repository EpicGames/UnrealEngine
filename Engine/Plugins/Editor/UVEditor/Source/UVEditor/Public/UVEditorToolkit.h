// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Tools/BaseAssetToolkit.h"

#define UE_API UVEDITOR_API

class FAdvancedPreviewScene;
class FUVEditorModeUILayer;
class FUVEditorPreviewModeManager;
class IDetailsView;
class SDockTab;
class SBorder;
class SWidget;
class UInteractiveToolsContext;
class UInputRouter;
class UUVToolViewportButtonsAPI;
class UUVTool2DViewportAPI;

/**
 * The toolkit is supposed to act as the UI manager for the asset editor. It's responsible 
 * for setting up viewports and most toolbars, except for the internals of the mode panel.
 * However, because the toolkit also sets up the mode manager, and much of the important
 * state is held in the UUVEditorMode managed by the mode manager, the toolkit also ends up
 * initializing the UV mode.
 * Thus, the FUVEdiotrToolkit ends up being the central place for the UV Asset editor setup.
 */
class FUVEditorToolkit : public FBaseAssetToolkit
{
public:
	UE_API FUVEditorToolkit(UAssetEditor* InOwningAssetEditor);
	UE_API virtual ~FUVEditorToolkit();

	static UE_API const FName InteractiveToolsPanelTabID;
	static UE_API const FName LivePreviewTabID;

	FPreviewScene* GetPreviewScene() { return UnwrapScene.Get(); }

	// FBaseAssetToolkit
	UE_API virtual void CreateWidgets() override;

	// FAssetEditorToolkit
	UE_API virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder = INDEX_NONE) override;
	UE_API virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget) override;
	UE_API virtual void CreateEditorModeManager() override;
	UE_API virtual FText GetToolkitName() const override;
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FText GetToolkitToolTipText() const override;
	UE_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	UE_API virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	UE_API virtual void OnClose() override;
	UE_API virtual void SaveAsset_Execute() override;
	UE_API virtual bool CanSaveAsset() const override;
	UE_API virtual bool CanSaveAssetAs() const override;
	UE_API void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	UE_API void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;

	// IAssetEditorInstance
	// This is important because if this returns true, attempting to edit a static mesh that is
	// open in the UV editor may open the UV editor instead of opening the static mesh editor.
	virtual bool IsPrimaryEditor() const override { return false; };

protected:

	UE_API TSharedRef<SDockTab> SpawnTab_InteractiveToolsPanel(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_LivePreview(const FSpawnTabArgs& Args);

	// FBaseAssetToolkit
	UE_API virtual AssetEditorViewportFactoryFunction GetViewportDelegate() override;
	UE_API virtual TSharedPtr<FEditorViewportClient> CreateEditorViewportClient() const override;

	// FAssetEditorToolkit
	UE_API virtual void PostInitAssetEditor() override;
	UE_API virtual const FSlateBrush* GetDefaultTabIcon() const override;
	UE_API virtual FLinearColor GetDefaultTabColor() const override;

	/** Scene in which the 2D unwrapped uv meshes live. */
	TUniquePtr<FPreviewScene> UnwrapScene;

	/** Scene in which the 3D preview meshes of the assets live. */
	TUniquePtr<FAdvancedPreviewScene> LivePreviewScene;

	// These are related to the 3D "live preview" viewport. The 2d unwrap viewport things are
	// stored in FBaseAssetToolkit::ViewportTabContent, ViewportDelegate, ViewportClient
	TSharedPtr<class FEditorViewportTabContent> LivePreviewTabContent;
	AssetEditorViewportFactoryFunction LivePreviewViewportDelegate;
	TSharedPtr<FEditorViewportClient> LivePreviewViewportClient;
	
	TSharedPtr<FAssetEditorModeManager> LivePreviewEditorModeManager;
	TObjectPtr<UInputRouter> LivePreviewInputRouter = nullptr;

	TWeakPtr<SEditorViewport> UVEditor2DViewport;
	UUVToolViewportButtonsAPI* ViewportButtonsAPI = nullptr;

	UUVTool2DViewportAPI* UVTool2DViewportAPI = nullptr;

	TSharedPtr<FUVEditorModeUILayer> ModeUILayer;
	TSharedPtr<FWorkspaceItem> UVEditorMenuCategory;
};

#undef UE_API
