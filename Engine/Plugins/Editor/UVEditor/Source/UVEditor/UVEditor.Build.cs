// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UVEditor : ModuleRules
{
	public UVEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				
				"UnrealEd",
				"InputCore",
				"InteractiveToolsFramework",
				"MeshModelingToolsExp",
				"GeometryProcessingInterfaces" // For supporting launching the UVEditor directly from Modeling Tools or elsewhere
				
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				
				"AdvancedPreviewScene",
				"ContentBrowser",
				"DynamicMesh",
				"EditorFramework", // FEditorModeInfo
				"EditorInteractiveToolsFramework",			
				"EditorSubsystem",
				"GeometryCore",
				"LevelEditor",
				"ModelingComponentsEditorOnly", // Static/skeletal mesh tool targets
				"ModelingComponents",
				"ModelingEditorUI",
				"StatusBar",
				"ToolMenus",
				"ToolWidgets",
				"UVEditorTools",
				"UVEditorToolsEditorOnly",
				"WorkspaceMenuStructure",
				"TextureUtilitiesCommon",
				"GeometryAlgorithms"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
