// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/UVMakeIslandAction.h"

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h" // FDynamicMeshUVOverlay
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/UVUnwrapMeshUtil.h"
#include "Selection/UVToolSelection.h"
#include "Selection/UVToolSelectionAPI.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "UVEditorUXSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVMakeIslandAction)

#define LOCTEXT_NAMESPACE "UUVMakeIslandAction"

namespace UVMakeIslandActionLocals
{
	using namespace UE::Geometry;

	const FText TransactionName(LOCTEXT("TransactionName", "Make Island"));
}

bool UUVMakeIslandAction::CanExecuteAction() const
{
	return (SelectionAPI->HaveSelections()
		&& SelectionAPI->GetSelectionsType() == UE::Geometry::FUVToolSelection::EType::Triangle)
		|| SelectionAPI->HaveUnsetElementAppliedMeshSelections();
}
bool UUVMakeIslandAction::ExecuteAction()
{
	using namespace UVMakeIslandActionLocals;
	using namespace UE::Geometry;

	EmitChangeAPI->BeginUndoTransaction(TransactionName);
	
	bool bSuccess = true;

	TArray<FUVToolSelection> Selections = SelectionAPI->GetSelections();
	
	// Lump the unset element selections into the regular selections, because the "make island" operation will set them
	for (const FUVToolSelection& UnsetSelection : SelectionAPI->GetUnsetElementAppliedMeshSelections())
	{
		if (!ensure(UnsetSelection.Target.IsValid() && UnsetSelection.Target->AppliedCanonical
			&& !UnsetSelection.SelectedIDs.IsEmpty() && UnsetSelection.Type == FUVToolSelection::EType::Triangle))
		{
			continue;
		}

		if (FUVToolSelection* ExistingSelection = Selections.FindByPredicate(
			[&UnsetSelection](const FUVToolSelection& Selection) { return Selection.Target == UnsetSelection.Target; }))
		{
			if (ensure(ExistingSelection->Type == FUVToolSelection::EType::Triangle))
			{
				ExistingSelection->SelectedIDs.Append(UnsetSelection.SelectedIDs);
			}
		}
		else
		{
			Selections.Add(UnsetSelection);
		}
	}
	// Clear the unset selections now so that its in the proper place in the undo stack
	bool bHadUnsetSelections = SelectionAPI->HaveUnsetElementAppliedMeshSelections();
	SelectionAPI->ClearUnsetElementAppliedMeshSelections(/*bBroadcast*/ false, /*bEmit*/ true);

	// Now process all the regular selections
	for (FUVToolSelection& Selection : Selections)
	{
		if (!ensure(Selection.Target.IsValid() && Selection.Target->AppliedCanonical 
			&& !Selection.SelectedIDs.IsEmpty() && Selection.Type == FUVToolSelection::EType::Triangle))
		{
			continue;
		}

		FDynamicMesh3& Mesh = *Selection.Target->AppliedCanonical;
		FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->GetUVLayer(Selection.Target->UVLayerIndex);
		if (!ensure(UVOverlay))
		{
			continue;
		}

		TSet<int32> ChangedTids;
		UE::Geometry::FUVEditResult Result;
		FDynamicMeshUVEditor UVEditor(&Mesh, UVOverlay);
		bSuccess = UVEditor.MakeIsland(Selection.SelectedIDs, &Result, &ChangedTids) && bSuccess;
		ensure(bSuccess);

		FDynamicMeshChangeTracker ChangeTracker(Selection.Target->UnwrapCanonical.Get());
		ChangeTracker.BeginChange();
		ChangeTracker.SaveTriangles(ChangedTids, true);

		TArray<int32> ChangedTidsArray = ChangedTids.Array();
		Selection.Target->UpdateAllFromAppliedCanonical(&Result.NewUVElements, &ChangedTidsArray, &ChangedTidsArray);

		checkSlow(UVUnwrapMeshUtil::DoesUnwrapMatchOverlay(*UVOverlay, *Selection.Target->UnwrapCanonical, Selection.Target->UVToVertPosition, 0.01));

		EmitChangeAPI->EmitToolIndependentUnwrapCanonicalChange(Selection.Target.Get(),
			ChangeTracker.EndChange(), TransactionName);
	}//end for each target that has selections

	if (bHadUnsetSelections)
	{
		// If we had unset elements selected, they are now selected elements.
		SelectionAPI->SetSelections(Selections, true, true);
	}
	EmitChangeAPI->EndUndoTransaction();
	return bSuccess;
}

#undef LOCTEXT_NAMESPACE
