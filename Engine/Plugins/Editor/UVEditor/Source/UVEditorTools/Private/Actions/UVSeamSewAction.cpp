// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/UVSeamSewAction.h"

#include "MeshOpPreviewHelpers.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "Selection/UVToolSelection.h"
#include "Selection/UVToolSelectionAPI.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "UVEditorUXSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVSeamSewAction)


#define LOCTEXT_NAMESPACE "UUVSeamSewAction"

using namespace UE::Geometry;

int32 UUVSeamSewAction::FindSewEdgeOppositePairing(const FDynamicMesh3& UnwrapMesh, const FDynamicMesh3& AppliedMesh, 
	int32 UVLayerIndex, int32 UnwrapEid, bool& bWouldPreferReverseOut)
{
	// Given an edge id on the unwrap mesh, determine its opposite edge suitable for UV sewing elsewhere on the unwrap mesh

	if (!UnwrapMesh.IsBoundaryEdge(UnwrapEid))
	{
		return IndexConstants::InvalidID;
	}

	int32 StartUnwrapTid = UnwrapMesh.GetEdgeT(UnwrapEid).A;
	int StartEdgeSubIndex = IndexUtil::FindTriIndex(UnwrapEid, UnwrapMesh.GetTriEdges(StartUnwrapTid));

	int32 AppliedEid = AppliedMesh.GetTriEdge(StartUnwrapTid, StartEdgeSubIndex);
	FIndex2i AppliedTids = AppliedMesh.GetEdgeT(AppliedEid);
	int32 EndUnwrapTid = (AppliedTids.A == StartUnwrapTid) ? AppliedTids.B : AppliedTids.A;

	if (EndUnwrapTid == IndexConstants::InvalidID || !UnwrapMesh.IsTriangle(EndUnwrapTid))
	{
		// Either this edge is a boundary edge in the original mesh, or the other triangle in the original
		// mesh has unset UV's.
		return IndexConstants::InvalidID;
	}

	int32 EndEdgeSubIndex = IndexUtil::FindTriIndex(AppliedEid, AppliedMesh.GetTriEdges(EndUnwrapTid));
	int32 PairedEid = UnwrapMesh.GetTriEdge(EndUnwrapTid, EndEdgeSubIndex);

	bWouldPreferReverseOut = StartUnwrapTid > EndUnwrapTid;

	return PairedEid;
}

bool UUVSeamSewAction::CanExecuteAction() const
{
	return SelectionAPI->HaveSelections()
		&& SelectionAPI->GetSelectionsType() == FUVToolSelection::EType::Edge;
}

bool UUVSeamSewAction::ExecuteAction()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVSeamSewAction_ExecuteAction);
	
	const FText TransactionName(LOCTEXT("SewCompleteTransactionName", "Sew Edges"));

	// To properly undo, we need to emit the selection change before we emit the mesh change.
	// As long as we change at least a single mesh, we'll clear the entire selection, then set
	// it to the new selection at the end.
	bool bAtLeastOneValidCandidate = false;
	TArray<FUVToolSelection> NewSelections;
	TArray<FUVToolSelection> StartSelections = SelectionAPI->GetSelections();
	for (const FUVToolSelection& Selection : StartSelections)
	{
		if (!ensure(Selection.Target.IsValid() && Selection.Target->AreMeshesValid()))
		{
			continue;
		}

		UUVEditorToolMeshInput* Target = Selection.Target.Get();
		FDynamicMesh3& MeshToSew = *(Target->UnwrapCanonical);
		FUVToolSelection NewSelection;
		NewSelection.Target = Selection.Target;
		NewSelection.Type = Selection.Type;

		// Gather up all the edge pairs to sew.
		TArray<FIndex2i> EdgeSewPairs;
		for (int32 Eid : Selection.SelectedIDs)
		{
			bool bWouldPreferReverse = false;
			int32 PairedEid = FindSewEdgeOppositePairing(
				MeshToSew, *Target->AppliedCanonical,
				Target->UVLayerIndex, Eid, bWouldPreferReverse);

			if (PairedEid == IndexConstants::InvalidID)
			{
				NewSelection.SelectedIDs.Add(Eid); // unpairable edges should stay selected.
				continue;
			}

			// If the paired edge is also selected, use a consistent prefernce for which goes to which
			// (i.e. skip if we would prefer the welding to happen starting with the other edge).
			if (Selection.SelectedIDs.Contains(PairedEid) && bWouldPreferReverse)
			{
				continue;
			}

			EdgeSewPairs.Emplace(Eid, PairedEid);
		}

		if (EdgeSewPairs.Num() == 0)
		{
			continue;
		}

		if (!bAtLeastOneValidCandidate)
		{
			EmitChangeAPI->BeginUndoTransaction(TransactionName);
			SelectionAPI->ClearSelections(false, true); // Don't broadcast, do emit
			bAtLeastOneValidCandidate = true;
		}

		// Gather up all the triangles that may be changed. These are not just the triangles on either side
		// of the edge but triangles in one rings of the vertices.
		TSet<int32> ChangedTidSet;
		for (const FIndex2i& EidPair : EdgeSewPairs)
		{
			for (int PairIndex = 0; PairIndex < 2; ++PairIndex)
			{
				FIndex2i Vids = MeshToSew.GetEdgeV(EidPair[PairIndex]);
				for (int VidIndex = 0; VidIndex < 2; ++VidIndex)
				{
					FDynamicMesh3::FLocalIntArray Tids;
					MeshToSew.GetVtxTriangles(Vids[VidIndex], Tids);
					ChangedTidSet.Append(Tids);
				}
			}
		}

		TArray<int32> ChangedTids = ChangedTidSet.Array();

		FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
		ChangeTracker.BeginChange();
		ChangeTracker.SaveTriangles(ChangedTids, true);

		// Note that currently, we don't really need to gather up the kept verts and call them out for an
		// update, since no vert locations should have changed. But we'll do it anyway in case any of the
		// code changes in some way that moves the remaining verts (for instance, to some halfway point).
		TSet<int32> KeptVidsSet;

		// Perform the actual weld ops
		for (FIndex2i EdgePair : EdgeSewPairs)
		{
			// An edge may have already been merged while merging an adjacent one, so make sure the edges still
			// exist. If one doesn't exist, it's pair should still be addable to selection.
			if (!MeshToSew.IsEdge(EdgePair.A))
			{
				if (ensure(MeshToSew.IsEdge(EdgePair.B)))
				{
					NewSelection.SelectedIDs.Add(EdgePair.B);
				}
				continue;
			}
			if (!MeshToSew.IsEdge(EdgePair.B))
			{
				if (ensure(MeshToSew.IsEdge(EdgePair.A)))
				{
					NewSelection.SelectedIDs.Add(EdgePair.A);
				}
				continue;
			}

			FDynamicMesh3::FMergeEdgesInfo MergeInfo;
			EMeshResult Result = MeshToSew.MergeEdges(EdgePair[1], EdgePair[0], MergeInfo, false);
			if (Result == EMeshResult::Ok)
			{
				for (int i = 0; i < 2; ++i)
				{
					KeptVidsSet.Add(MergeInfo.KeptVerts[i]);
					NewSelection.SelectedIDs.Add(MergeInfo.KeptEdge);
				}
			}
			else
			{
				UE_LOGF(LogGeometry, Warning, "Failed to sew edge pair %d / %d. Failed with code %d", EdgePair[0], EdgePair[1], int(Result));
			}
		}
		checkSlow(MeshToSew.CheckValidity(FDynamicMesh3::FValidityOptions(true, true))); // Allow nonmanifold verts and reverse orientation

		TArray<int32> RemainingVids;
		for (int32 KeptVid : KeptVidsSet)
		{
			// We have to do this check because we may have performed multiple merge actions, so a "kept vert"
			// from one action may have ended up getting removed in a later one.
			if (MeshToSew.IsVertex(KeptVid))
			{
				RemainingVids.Add(KeptVid);
			}
		}

		Target->UpdateUnwrapCanonicalOverlayFromPositions(&RemainingVids, &ChangedTids);
		Target->UpdateAllFromUnwrapCanonical(&RemainingVids, &ChangedTids, &ChangedTids);
		checkSlow(MeshToSew.IsSameAs(*Target->UnwrapPreview->PreviewMesh->GetMesh(), FDynamicMesh3::FSameAsOptions()));

		EmitChangeAPI->EmitToolIndependentUnwrapCanonicalChange(Target,
			ChangeTracker.EndChange(), TransactionName);

		if (!NewSelection.IsEmpty())
		{
			NewSelections.Add(MoveTemp(NewSelection));
		}
	}

	if (!bAtLeastOneValidCandidate)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("SewErrorSelectionNotBoundary", "Cannot sew UVs. No viable sew candidate edges selected."),
			EToolMessageLevel::UserWarning);
		return false;
	}

	SelectionAPI->SetSelections(NewSelections, true, true); // broadcast, emit
	EmitChangeAPI->EndUndoTransaction();
	return true;
}

#undef LOCTEXT_NAMESPACE
