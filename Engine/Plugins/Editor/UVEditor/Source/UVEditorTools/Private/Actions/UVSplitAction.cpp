// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/UVSplitAction.h"

#include "ContextObjects/UVToolContextObjects.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Selection/UVToolSelection.h"
#include "Selection/UVToolSelectionAPI.h"
#include "ToolTargets/UVEditorToolMeshInput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVSplitAction)


#define LOCTEXT_NAMESPACE "UUVSplitAction"

using namespace UE::Geometry;

namespace UVSplitActionLocals
{
	bool ApplySplitEdges(const FUVToolSelection& Selection, FUVToolSelection& NewSelectionOut, 
		UUVToolEmitChangeAPI& EmitChangeAPI, const FText& TransactionName)
	{
		UUVEditorToolMeshInput* Target = Selection.Target.Get();
		
		if (!ensure(Target && Target->AreMeshesValid()))
		{
			return false;
		}

		// Gather up the corresponding edge IDs in the applied (3d) mesh.
		TSet<int32> AppliedEidSet;
		for (int32 Eid : Selection.SelectedIDs)
		{
			// Note that we don't check whether edges are already boundary edges because we allow such edges
			// to be selected for splitting of any attached bowties.

			FIndex2i EdgeUnwrapVids = Target->UnwrapCanonical->GetEdgeV(Eid);

			int32 AppliedEid = Target->AppliedCanonical->FindEdge(
				Target->UnwrapVidToAppliedVid(EdgeUnwrapVids.A),
				Target->UnwrapVidToAppliedVid(EdgeUnwrapVids.B));

			if (ensure(AppliedEid != IndexConstants::InvalidID))
			{
				AppliedEidSet.Add(AppliedEid);
			}
		}

		// Perform the cut in the overlay, but don't propagate to unwrap yet because we'll need to prep for undo
		FUVEditResult UVEditResult;
		FDynamicMeshUVEditor UVEditor(Target->AppliedCanonical.Get(),
			Target->UVLayerIndex, false);
		UVEditor.CreateSeamsAtEdges(AppliedEidSet, &UVEditResult);

		// Figure out the triangles that need to be saved in the unwrap for undo
		TSet<int32> TidSet;
		for (int32 UnwrapVid : UVEditResult.NewUVElements)
		{
			FDynamicMesh3::FLocalIntArray VertTids;
			Target->AppliedCanonical->GetVtxTriangles(Target->UnwrapVidToAppliedVid(UnwrapVid), VertTids);
			TidSet.Append(VertTids);
		}

		FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
		ChangeTracker.BeginChange();
		ChangeTracker.SaveTriangles(TidSet, true);

		// Perform the update
		TArray<int32> AppliedTids = TidSet.Array();
		Target->UpdateAllFromAppliedCanonical(&UVEditResult.NewUVElements, &AppliedTids, &AppliedTids);

		// Gather up the corresponding eids in the unwrap. We select only one of the newly
		// created border edges because this turns out to be very convenient for splitting
		// and then moving the edge. This is slightly awkward in some edge cases, namely if
		// we select an existing border edge, we end up with the other border edge selected
		// at the end, but dealing with those is unlikely to be worth the code complexity.
		TSet<int32> UnwrapEidSet;
		for (int32 AppliedEid : AppliedEidSet)
		{
			int32 Tid = Target->AppliedCanonical->GetEdgeT(AppliedEid).A;
			FIndex3i TriAppliedEids = Target->AppliedCanonical->GetTriEdges(Tid);
			int32 EdgeSubIndex = IndexUtil::FindTriIndex(AppliedEid, TriAppliedEids);

			FIndex3i TriUnwrapEids = Target->UnwrapCanonical->GetTriEdges(Tid);

			if (ensure(EdgeSubIndex != IndexConstants::InvalidID))
			{
				UnwrapEidSet.Add(TriUnwrapEids[EdgeSubIndex]);
			}
		}

		NewSelectionOut.Target = Selection.Target;
		NewSelectionOut.Type = Selection.Type;
		NewSelectionOut.SelectedIDs = MoveTemp(UnwrapEidSet);

		// Emit update transaction
		EmitChangeAPI.EmitToolIndependentUnwrapCanonicalChange(
			Target, ChangeTracker.EndChange(), TransactionName);

		return true;
	}

	bool ApplySplitBowtieVertices(const FUVToolSelection& Selection, FUVToolSelection& NewSelectionOut, 
		UUVToolEmitChangeAPI& EmitChangeAPI, const FText& TransactionName)
	{
		UUVEditorToolMeshInput* Target = Selection.Target.Get();

		if (!ensure(Target && Target->AreMeshesValid()))
		{
			return false;
		}

		// Gather the corresponding vert ID's in the applied mesh
		TSet<int32> AppliedVidSet;
		for (int32 UnwrapVid : Selection.SelectedIDs)
		{
			int32 AppliedVid = Target->UnwrapVidToAppliedVid(UnwrapVid);
			AppliedVidSet.Add(AppliedVid);
		}

		// Split any bowties in the applied mesh overlay
		TArray<int32> NewUVElements;
		FDynamicMeshUVOverlay* Overlay = Target->AppliedCanonical->Attributes()->GetUVLayer(Target->UVLayerIndex);
		for (int32 Vid : AppliedVidSet)
		{
			Overlay->SplitBowtiesAtVertex(Vid, &NewUVElements);
		}

		// Prep for undo transaction
		TSet<int32> TidSet;
		for (int32 UnwrapVid : NewUVElements)
		{
			FDynamicMesh3::FLocalIntArray VertTids;
			Target->AppliedCanonical->GetVtxTriangles(Target->UnwrapVidToAppliedVid(UnwrapVid), VertTids);
			TidSet.Append(VertTids);
		}

		FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
		ChangeTracker.BeginChange();
		ChangeTracker.SaveTriangles(TidSet, true);

		// Perform the update
		TArray<int32> AppliedTids = TidSet.Array();
		Target->UpdateAllFromAppliedCanonical(&NewUVElements, &AppliedTids, &AppliedTids);

		// Emit update transaction
		EmitChangeAPI.EmitToolIndependentUnwrapCanonicalChange(
			Target, ChangeTracker.EndChange(), TransactionName);

		// Set up the new selection to include the new elements
		NewSelectionOut = Selection;
		NewSelectionOut.SelectedIDs.Append(NewUVElements);

		return true;
	}

	bool ApplyConnectedRegionSplits(const FUVToolSelection& Selection, FUVToolSelection& NewSelectionOut,
		UUVToolEmitChangeAPI& EmitChangeAPI, const FText& TransactionName)
	{
		UUVEditorToolMeshInput* Target = Selection.Target.Get();

		if (!ensure(Target && Target->AreMeshesValid()))
		{
			return false;
		}

		TSet<int32> UnwrapRegionBoundaryEdges;
		for (int32 Tid : Selection.SelectedIDs)
		{
			if (!ensure(Target->UnwrapCanonical->IsTriangle(Tid)))
			{
				continue;
			}

			// Gather the boundary edges for each region
			FIndex3i TriEids = Target->UnwrapCanonical->GetTriEdges(Tid);
			for (int i = 0; i < 3; ++i)
			{
				FIndex2i EdgeTids = Target->UnwrapCanonical->GetEdgeT(TriEids[i]);
				for (int j = 0; j < 2; ++j)
				{
					if (EdgeTids[j] != Tid && !Selection.SelectedIDs.Contains(EdgeTids[j]))
					{
						UnwrapRegionBoundaryEdges.Add(TriEids[i]);
						break;
					}
				}
			}//end for tri edges
		}//end for selection tids

		TSet<int32> AppliedEidSet;
		for (int32 Eid : UnwrapRegionBoundaryEdges)
		{
			// Note that we don't check whether edges are already boundary edges because we allow such edges
			// to be selected for splitting of any attached bowties.

			FIndex2i EdgeUnwrapVids = Target->UnwrapCanonical->GetEdgeV(Eid);

			int32 AppliedEid = Target->AppliedCanonical->FindEdge(
				Target->UnwrapVidToAppliedVid(EdgeUnwrapVids.A),
				Target->UnwrapVidToAppliedVid(EdgeUnwrapVids.B));

			if (ensure(AppliedEid != IndexConstants::InvalidID))
			{
				AppliedEidSet.Add(AppliedEid);
			}
		}

		// Perform the cut in the overlay, but don't propagate to unwrap yet because we'll need to prep for undo
		FUVEditResult UVEditResult;
		FDynamicMeshUVEditor UVEditor(Target->AppliedCanonical.Get(),
			Target->UVLayerIndex, false);
		UVEditor.CreateSeamsAtEdges(AppliedEidSet, &UVEditResult);

		// Figure out the triangles that need to be saved in the unwrap for undo
		TSet<int32> TidSet;
		for (int32 UnwrapVid : UVEditResult.NewUVElements)
		{
			FDynamicMesh3::FLocalIntArray VertTids;
			Target->AppliedCanonical->GetVtxTriangles(Target->UnwrapVidToAppliedVid(UnwrapVid), VertTids);
			TidSet.Append(VertTids);
		}

		FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
		ChangeTracker.BeginChange();
		ChangeTracker.SaveTriangles(TidSet, true);

		// Perform the update
		TArray<int32> AppliedTids = TidSet.Array();
		Target->UpdateAllFromAppliedCanonical(&UVEditResult.NewUVElements, &AppliedTids, &AppliedTids);

		// Since we are simply splitting triangles away, the selection doesn't actually change after the split.
		NewSelectionOut.Target = Selection.Target;
		NewSelectionOut.Type = Selection.Type;
		NewSelectionOut.SelectedIDs = Selection.SelectedIDs;

		// Emit update transaction
		EmitChangeAPI.EmitToolIndependentUnwrapCanonicalChange(
			Target, ChangeTracker.EndChange(), TransactionName);

		return true;

	}
}

bool UUVSplitAction::CanExecuteAction() const
{
	return SelectionAPI->HaveSelections()
		&& (SelectionAPI->GetSelectionsType() == FUVToolSelection::EType::Vertex
			|| SelectionAPI->GetSelectionsType() == FUVToolSelection::EType::Edge
			|| SelectionAPI->GetSelectionsType() == FUVToolSelection::EType::Triangle);
}

bool UUVSplitAction::ExecuteAction()
{
	using namespace UVSplitActionLocals;

	if (!ensure(CanExecuteAction()))
	{
		return false;
	}

	bool bAllSucceeded = true;
	TArray<FUVToolSelection> Selections = SelectionAPI->GetSelections();
	FUVToolSelection::EType SelectionType = SelectionAPI->GetSelectionsType();
	TArray<FUVToolSelection> NewSelections;

	FText TransactionName;
	if (SelectionType == FUVToolSelection::EType::Edge)
	{
		TransactionName = LOCTEXT("SplitEdgesTransactionName", "Split Edges");
	}
	else if(SelectionType == FUVToolSelection::EType::Vertex)
	{
		TransactionName = LOCTEXT("SplitBowtieVerticesTransactionName", "Split Bowties");
	}
	else if (SelectionType == FUVToolSelection::EType::Triangle)
	{
		TransactionName = LOCTEXT("SplitRegionsTransactionName", "Split Regions");
	}
	EmitChangeAPI->BeginUndoTransaction(TransactionName);

	SelectionAPI->ClearSelections(false, true); // don't broadcast, do emit
	for (const FUVToolSelection& Selection : Selections)
	{
		FUVToolSelection NewSelection;
		if (Selection.Type == FUVToolSelection::EType::Edge)
		{
			bAllSucceeded &= ApplySplitEdges(Selection, NewSelection, *EmitChangeAPI, TransactionName);
		}
		else if (Selection.Type == FUVToolSelection::EType::Vertex)
		{
			bAllSucceeded &= ApplySplitBowtieVertices(Selection, NewSelection, *EmitChangeAPI, TransactionName);
		}
		else if (Selection.Type == FUVToolSelection::EType::Triangle)
		{
			bAllSucceeded &= ApplyConnectedRegionSplits(Selection, NewSelection, *EmitChangeAPI, TransactionName);
		}

		if (!NewSelection.IsEmpty())
		{
			NewSelections.Add(NewSelection);
		}
	}

	SelectionAPI->SetSelections(NewSelections, true, true); // broadcast and emit.

	EmitChangeAPI->EndUndoTransaction();

	return bAllSucceeded;
}

#undef LOCTEXT_NAMESPACE
