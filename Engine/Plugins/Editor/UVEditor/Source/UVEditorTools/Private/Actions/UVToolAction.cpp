// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/UVToolAction.h"

#include "ContextObjectStore.h"
#include "Selection/UVToolSelectionAPI.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVToolAction)

#define LOCTEXT_NAMESPACE "UUVToolAction"

using namespace UE::Geometry;

void UUVToolAction::Setup(UInteractiveToolManager* ToolManagerIn)
{
	ToolManager = ToolManagerIn;

	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();
	SelectionAPI = ContextStore->FindContext<UUVToolSelectionAPI>();
	checkSlow(SelectionAPI);
	EmitChangeAPI = ContextStore->FindContext<UUVToolEmitChangeAPI>();
}

void UUVToolAction::Shutdown()
{
	SelectionAPI = nullptr;
	EmitChangeAPI = nullptr;
}

#undef LOCTEXT_NAMESPACE
