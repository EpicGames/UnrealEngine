// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextObjects/UVToolContextObjects.h"

#include "DynamicMesh/MeshIndexUtil.h"
#include "Engine/World.h"
#include "InputRouter.h" // Need to define this and UWorld so weak pointers know they are UThings
#include "InteractiveGizmoManager.h"
#include "InteractiveToolManager.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "UDIMUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVToolContextObjects)

using namespace UE::Geometry;

namespace UVToolContextObjectLocals
{
	/**
	 * A wrapper change that applies a given change to the unwrap canonical mesh of an input, and uses that
	 * to update the other views. Causes a broadcast of OnCanonicalModified.
	 */
	class  FUVEditorMeshChange : public FToolCommandChange
	{
	public:
		FUVEditorMeshChange(UUVEditorToolMeshInput* UVToolInputObjectIn, TUniquePtr<FDynamicMeshChange> UnwrapCanonicalMeshChangeIn)
			: UVToolInputObject(UVToolInputObjectIn)
			, UnwrapCanonicalMeshChange(MoveTemp(UnwrapCanonicalMeshChangeIn))
		{
			ensure(UVToolInputObjectIn);
			ensure(UnwrapCanonicalMeshChange);
		};

		virtual void Apply(UObject* Object) override
		{
			UnwrapCanonicalMeshChange->Apply(UVToolInputObject->UnwrapCanonical.Get(), false);
			UVToolInputObject->UpdateFromCanonicalUnwrapUsingMeshChange(*UnwrapCanonicalMeshChange);
		}

		virtual void Revert(UObject* Object) override
		{
			UnwrapCanonicalMeshChange->Apply(UVToolInputObject->UnwrapCanonical.Get(), true);
			UVToolInputObject->UpdateFromCanonicalUnwrapUsingMeshChange(*UnwrapCanonicalMeshChange);
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			return !(UVToolInputObject.IsValid() && UVToolInputObject->IsValid() && UnwrapCanonicalMeshChange);
		}


		virtual FString ToString() const override
		{
			return TEXT("FUVEditorMeshChange");
		}

	protected:
		TWeakObjectPtr<UUVEditorToolMeshInput> UVToolInputObject;
		TUniquePtr<FDynamicMeshChange> UnwrapCanonicalMeshChange;
	};
}

void UUVToolEmitChangeAPI::EmitToolIndependentChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description)
{
	ToolManager->GetContextTransactionsAPI()->AppendChange(TargetObject, MoveTemp(Change), Description);
}

void UUVToolEmitChangeAPI::EmitToolIndependentUnwrapCanonicalChange(UUVEditorToolMeshInput* InputObject, 
	TUniquePtr<FDynamicMeshChange> UnwrapCanonicalMeshChange, const FText& Description)
{
	ToolManager->GetContextTransactionsAPI()->AppendChange(InputObject, 
		MakeUnique<UVToolContextObjectLocals::FUVEditorMeshChange>(InputObject, MoveTemp(UnwrapCanonicalMeshChange)), Description);
}

void UUVToolEmitChangeAPI::EmitToolDependentChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description)
{
	// This should wrap the change in the proper wrapper that will expire it when the tool changes
	ToolManager->EmitObjectChange(TargetObject, MoveTemp(Change), Description);
}

void UUVToolLivePreviewAPI::Initialize(UWorld* WorldIn, UInputRouter* RouterIn,
	TUniqueFunction<void(FViewCameraState& CameraStateOut)> GetLivePreviewCameraStateFuncIn,
	TUniqueFunction<void(const FAxisAlignedBox3d& BoundingBox)> SetLivePreviewCameraToLookAtVolumeFuncIn,
	TUniqueFunction<void(const EMouseCursor::Type Cursor, bool bEnableOverride)> SetCursorOverrideFuncIn, 
	UInteractiveGizmoManager* GizmoManagerIn)
{
	World = WorldIn;
	InputRouter = RouterIn;
	GetLivePreviewCameraStateFunc = MoveTemp(GetLivePreviewCameraStateFuncIn);
	SetLivePreviewCameraToLookAtVolumeFunc = MoveTemp(SetLivePreviewCameraToLookAtVolumeFuncIn);
	SetCursorOverrideFunc = MoveTemp(SetCursorOverrideFuncIn);
	GizmoManager = GizmoManagerIn;
}

void UUVToolLivePreviewAPI::OnToolEnded(UInteractiveTool* DeadTool)
{
	OnDrawHUD.RemoveAll(DeadTool);
	OnRender.RemoveAll(DeadTool);
	ClearCursorOverride();
}

int32 FUDIMBlock::BlockU() const
{
	int32 BlockU, BlockV;
	UE::TextureUtilitiesCommon::ExtractUDIMCoordinates(UDIM, BlockU, BlockV);
	return BlockU;
}

int32 FUDIMBlock::BlockV() const
{
	int32 BlockU, BlockV;
	UE::TextureUtilitiesCommon::ExtractUDIMCoordinates(UDIM, BlockU, BlockV);
	return BlockV;
}

void FUDIMBlock::SetFromBlocks(int32 BlockU, int32 BlockV)
{
	UDIM = UE::TextureUtilitiesCommon::GetUDIMIndex(BlockU, BlockV);
}

void UUVToolAABBTreeStorage::Set(FDynamicMesh3* MeshKey, TSharedPtr<FDynamicMeshAABBTree3> Tree, 
	UUVEditorToolMeshInput* InputObject)
{
	AABBTreeStorage.Add(MeshKey, TreeInputObjectPair(Tree, InputObject));
	InputObject->OnCanonicalModified.AddWeakLambda(this, [MeshKey, Tree]
	(UUVEditorToolMeshInput* InputObject, const UUVEditorToolMeshInput::FCanonicalModifiedInfo& Info) {
		if (InputObject->UnwrapCanonical.Get() == MeshKey
			|| (Info.bAppliedMeshShapeChanged && InputObject->AppliedCanonical.Get() == MeshKey))
		{
			Tree->Build();
		}
	});
}

TSharedPtr<FDynamicMeshAABBTree3> UUVToolAABBTreeStorage::Get(FDynamicMesh3* MeshKey)
{
	TreeInputObjectPair* FoundPtr = AABBTreeStorage.Find(MeshKey);
	return FoundPtr ? FoundPtr->Key : nullptr;
}

void UUVToolAABBTreeStorage::Remove(FDynamicMesh3* MeshKey)
{
	TreeInputObjectPair* Result = AABBTreeStorage.Find(MeshKey);
	if (!Result)
	{
		return;
	}
	if (Result->Value.IsValid())
	{
		Result->Value->OnCanonicalModified.RemoveAll(this);
	}

	AABBTreeStorage.Remove(MeshKey);
}

void UUVToolAABBTreeStorage::RemoveByPredicate(TUniqueFunction<
	bool(const FDynamicMesh3* Mesh, TWeakObjectPtr<UUVEditorToolMeshInput> InputObject,
		TSharedPtr<FDynamicMeshAABBTree3> Tree)> Predicate)
{
	for (auto It = AABBTreeStorage.CreateIterator(); It; ++It)
	{
		TWeakObjectPtr<UUVEditorToolMeshInput> InputObject = It->Value.Value;
		if (Predicate(It->Key, InputObject, It->Value.Key)) // mesh, object, tree
		{
			if (InputObject.IsValid())
			{
				InputObject->OnCanonicalModified.RemoveAll(this);
			}
			It.RemoveCurrent();
		}
	}
}

void UUVToolAABBTreeStorage::Empty()
{
	for (TPair<FDynamicMesh3*, TreeInputObjectPair>& MapPair : AABBTreeStorage)
	{
		TWeakObjectPtr<UUVEditorToolMeshInput> InputObject = MapPair.Value.Value;
		if (InputObject.IsValid())
		{
			InputObject->OnCanonicalModified.RemoveAll(this);
		}
	}
	AABBTreeStorage.Empty();
}

void UUVToolAABBTreeStorage::Shutdown()
{
	Empty();
}
