// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/BasicPointSetComponent.h"
#include "Async/ParallelFor.h"
#include "Engine/CollisionProfile.h"
#include "PrimitiveSceneProxy.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "DynamicMeshBuilder.h"
#include "SceneInterface.h"
#include "StaticMeshResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasicPointSetComponent)


struct FBasicPointSetMeshBatchData
{
	FMaterialRenderProxy* MaterialProxy = nullptr;
	int32 StartIndex = -1;
	int32 NumPrimitives = -1;
	int32 MinVertexIndex = -1;
	int32 MaxVertexIndex = -1;
};

class FBasicPointSetSceneProxy final : public FPrimitiveSceneProxy
{
public:

	template <typename BasicPointSetComponent>
	FBasicPointSetSceneProxy(BasicPointSetComponent* Component)
		: FPrimitiveSceneProxy(Component),
		MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform())),
		VertexFactory(GetScene().GetFeatureLevel(), "FPointSetSceneProxy")
	{
		const int32 NumPointVertices = Component->NumElements() * 4;
		const int32 NumPointIndices = Component->NumElements() * 6;
		const int32 TotalNumVertices = NumPointVertices;
		const int32 TotalNumIndices = NumPointIndices;
		constexpr int32 NumTextureCoordinates = 1;

		VertexBuffers.PositionVertexBuffer.Init(TotalNumVertices);
		VertexBuffers.StaticMeshVertexBuffer.Init(TotalNumVertices, NumTextureCoordinates);
		VertexBuffers.ColorVertexBuffer.Init(TotalNumVertices);
		IndexBuffer.Indices.SetNumUninitialized(TotalNumIndices);
			
		// Initialize points.
		// Points are represented as two tris, all of whose vertices are coincident.
		// The material then offsets them according to the signs of the vertex normals in a camera facing orientation.
		// Size of the point is given by U0.
		if (Component->NumElements() > 0)
		{
			MeshBatchDatas.Emplace();
			FBasicPointSetMeshBatchData& MeshBatchData = MeshBatchDatas.Last();
			MeshBatchData.MinVertexIndex = 0;
			MeshBatchData.MaxVertexIndex = NumPointVertices - 1;
			MeshBatchData.StartIndex = 0;
			MeshBatchData.NumPrimitives = Component->NumElements() * 2;
			if (Component->GetMaterial(0) != nullptr)
			{
				MeshBatchData.MaterialProxy = Component->GetMaterial(0)->GetRenderProxy();
			}
			else
			{
				MeshBatchData.MaterialProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			}

			const FVector TangentVectors[4] = {
				FVector(1.0f, -1.0f, 0.0f),
				FVector(1.0f, 1.0f, 0.0f),
				FVector(-1.0f, 1.0f, 0.0f),
				FVector(-1.0f, -1.0f, 0.0f)
			};

			const TArray<typename BasicPointSetComponent::GeometryType>& Points = Component->Elements;

			// The color stored in the vertices actually gets interpreted as a linear color by the material,
			// whereas it is more convenient for the user of the LineSet to specify colors as sRGB. So we actually
			// have to convert it back to linear. The ToFColor(false) call just scales back into 0-255 space.
			const FColor Color = FLinearColor::FromSRGBColor(Component->Color).ToFColor(false);

			const FVector2f UV(Component->Size, Component->DepthBias);

			ParallelFor(Component->NumElements(), [&](int32 i)
			{
				const int32 VertexBufferIndex = 4 * i;
				const int32 IndexBufferIndex = 6 * i;

				for (int j = 0; j < 4; ++j)
				{
					VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + j) = ToFVector3f(Points[i]);
					VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + j, 0, UV);
					VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + j) = Color;
					VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + j, FVector3f::ZeroVector, FVector3f::ZeroVector, static_cast<FVector3f>(TangentVectors[j]));
				}

				IndexBuffer.Indices[IndexBufferIndex + 0] = VertexBufferIndex + 0;
				IndexBuffer.Indices[IndexBufferIndex + 1] = VertexBufferIndex + 1;
				IndexBuffer.Indices[IndexBufferIndex + 2] = VertexBufferIndex + 2;
				IndexBuffer.Indices[IndexBufferIndex + 3] = VertexBufferIndex + 2;
				IndexBuffer.Indices[IndexBufferIndex + 4] = VertexBufferIndex + 3;
				IndexBuffer.Indices[IndexBufferIndex + 5] = VertexBufferIndex + 0;
			});
		}

		ENQUEUE_RENDER_COMMAND(OverlayVertexBuffersInit)(
			[this](FRHICommandListImmediate& RHICmdList)
		{
			VertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
			VertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
			VertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
			VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
			VertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&VertexFactory, Data);
			VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);
			VertexFactory.SetData(RHICmdList, Data);

			VertexFactory.InitResource(RHICmdList);
			IndexBuffer.InitResource(RHICmdList);
		});
	}

	virtual ~FBasicPointSetSceneProxy() override
	{
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OverlaySceneProxy_GetDynamicMeshElements);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				for (const FBasicPointSetMeshBatchData& MeshBatchData : MeshBatchDatas)
				{
					FMeshBatch& Mesh = Collector.AllocateMesh();
					FMeshBatchElement& BatchElement = Mesh.Elements[0];
					BatchElement.IndexBuffer = &IndexBuffer;
					Mesh.bWireframe = false;
					Mesh.VertexFactory = &VertexFactory;
					Mesh.MaterialRenderProxy = MeshBatchData.MaterialProxy;

					FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
					DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), GetBounds(), GetLocalBounds(), true, false, AlwaysHasVelocity());
					BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

					BatchElement.FirstIndex = MeshBatchData.StartIndex;
					BatchElement.NumPrimitives = MeshBatchData.NumPrimitives;
					BatchElement.MinVertexIndex = MeshBatchData.MinVertexIndex;
					BatchElement.MaxVertexIndex = MeshBatchData.MaxVertexIndex;
					Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
					Mesh.Type = PT_TriangleList;
					Mesh.DepthPriorityGroup = SDPG_World;
					Mesh.bCanApplyViewModeOverrides = false;
					Collector.AddMesh(ViewIndex, Mesh);
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }

	virtual SIZE_T GetTypeHash() const override
	{
		static SIZE_T UniquePointer;
		return reinterpret_cast<SIZE_T>(&UniquePointer);
	}

private:

	// Point type specific implementation to get an FVector3f out of the generic point type.
	static FVector3f ToFVector3f(const FVector2f& Vector) {	return FVector3f(Vector.X, Vector.Y, 0.0f); }
	static FVector3f ToFVector3f(const FVector3f& Vector) {	return Vector; }
	
	TArray<FBasicPointSetMeshBatchData> MeshBatchDatas;
	FMaterialRelevance MaterialRelevance;
	FLocalVertexFactory VertexFactory;
	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;
};

UBasicPointSetComponentBase::UBasicPointSetComponentBase()
{
	CastShadow = false;
	bSelectable = false;
	PrimaryComponentTick.bCanEverTick = false;

	UPrimitiveComponent::SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

void UBasicPointSetComponentBase::ClearComponent()
{
	MarkRenderStateDirty();
	bBoundsDirty = true;
}


void UBasicPointSetComponentBase::SetPointMaterial(UMaterialInterface* InPointMaterial)
{
	PointMaterial = InPointMaterial;
	UMeshComponent::SetMaterial(0, InPointMaterial);
}

void UBasicPointSetComponentBase::SetPointSetParameters(FColor InColor, float InSize, float InDepthBias)
{
	Color = InColor;
	Size = InSize;
	DepthBias = InDepthBias;
	bBoundsDirty = true;
}

void UBasicPointSetComponentBase::SetColor(const FColor InColor)
{
	Color = InColor;
}

void UBasicPointSetComponentBase::SetPointSize(const float InSize)
{
	Size = InSize;
	bBoundsDirty = true;
}


FPrimitiveSceneProxy* UBasic2DPointSetComponent::CreateSceneProxy()
{
	return Elements.Num() > 0 ? new FBasicPointSetSceneProxy(this) : nullptr;
}

FPrimitiveSceneProxy* UBasic3DPointSetComponent::CreateSceneProxy()
{
	return Elements.Num() > 0 ? new FBasicPointSetSceneProxy(this) : nullptr;
}
