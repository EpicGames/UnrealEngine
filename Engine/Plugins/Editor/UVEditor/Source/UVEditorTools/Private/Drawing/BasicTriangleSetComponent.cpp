// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/BasicTriangleSetComponent.h"
#include "Async/ParallelFor.h"
#include "Engine/CollisionProfile.h"
#include "PrimitiveSceneProxy.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "DynamicMeshBuilder.h"
#include "SceneInterface.h"
#include "StaticMeshResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasicTriangleSetComponent)


struct FBasicTriangleSetMeshBatchData
{
	FMaterialRenderProxy* MaterialProxy = nullptr;
	int32 StartIndex = -1;
	int32 NumPrimitives = -1;
	int32 MinVertexIndex = -1;
	int32 MaxVertexIndex = -1;
};

class FBasicTriangleSetSceneProxy final : public FPrimitiveSceneProxy
{
public:

	template <typename BasicTriangleSetComponent>
	FBasicTriangleSetSceneProxy(BasicTriangleSetComponent* Component)
		: FPrimitiveSceneProxy(Component),
		MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform())),
		VertexFactory(GetScene().GetFeatureLevel(), "FTriangleSetSceneProxy")
	{
		const int32 NumTriangleVertices = Component->NumElements() * 3;
		const int32 NumTriangleIndices = Component->NumElements() * 3;
		const int32 TotalNumVertices = NumTriangleVertices;
		const int32 TotalNumIndices = NumTriangleIndices;
		constexpr int32 NumTextureCoordinates = 1;

		VertexBuffers.PositionVertexBuffer.Init(TotalNumVertices);
		VertexBuffers.StaticMeshVertexBuffer.Init(TotalNumVertices, NumTextureCoordinates);
		VertexBuffers.ColorVertexBuffer.Init(TotalNumVertices);
		IndexBuffer.Indices.SetNumUninitialized(TotalNumIndices);

		// Initialize points.
		// Triangles are represented as two tris, all of whose vertices are coincident.
		// The material then offsets them according to the signs of the vertex normals in a camera facing orientation.
		// Size of the point is given by U0.
		if (Component->NumElements() > 0)
		{
			MeshBatchDatas.Emplace();
			FBasicTriangleSetMeshBatchData& MeshBatchData = MeshBatchDatas.Last();
			MeshBatchData.MinVertexIndex = 0;
			MeshBatchData.MaxVertexIndex = NumTriangleVertices - 1;
			MeshBatchData.StartIndex = 0;
			MeshBatchData.NumPrimitives = Component->NumElements();
			if (Component->GetMaterial(0) != nullptr)
			{
				MeshBatchData.MaterialProxy = Component->GetMaterial(0)->GetRenderProxy();
			}
			else
			{
				MeshBatchData.MaterialProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			}

			const TArray<typename BasicTriangleSetComponent::GeometryType>& Triangles = Component->Elements;

			// The color stored in the vertices actually gets interpreted as a linear color by the material,
			// whereas it is more convenient for the user of the TriangleSet to specify colors as sRGB. So we actually
			// have to convert it back to linear. The ToFColor(false) call just scales back into 0-255 space.
			const FColor Color = FLinearColor::FromSRGBColor(Component->Color).ToFColor(false);
			const FVector3f Normal = Component->Normal;
			ParallelFor(Component->NumElements(), [&](int32 i)
				{
					const int32 VertexBufferIndex = 3 * i;
					const int32 IndexBufferIndex = 3 * i;
					const int32 TriangleSize = BasicTriangleSetComponent::ElementSize;

					VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 0) = ToFVector3f(Triangles[i * TriangleSize + 0]);
					VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 1) = ToFVector3f(Triangles[i * TriangleSize + 1]);
					VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 2) = ToFVector3f(Triangles[i * TriangleSize + 2]);

					VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 0, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);
					VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 1, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);
					VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 2, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);

					VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 0, 0, FVector2f(0, 0));
					VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 1, 0, FVector2f(1, 0));
					VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 2, 0, FVector2f(0, 1));

					VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 0) = Color;
					VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 1) = Color;
					VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 2) = Color;

					IndexBuffer.Indices[IndexBufferIndex + 0] = VertexBufferIndex + 0;
					IndexBuffer.Indices[IndexBufferIndex + 1] = VertexBufferIndex + 1;
					IndexBuffer.Indices[IndexBufferIndex + 2] = VertexBufferIndex + 2;
				});

			ENQUEUE_RENDER_COMMAND(OverlayVertexBuffersInit)(
				[this](FRHICommandListImmediate& RHICmdList)
				{
					VertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
					VertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
					VertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

					FLocalVertexFactory::FDataType Data;
					VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
					VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
					VertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&VertexFactory, Data);
					VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);
					VertexFactory.SetData(RHICmdList, Data);

					VertexFactory.InitResource(RHICmdList);
					IndexBuffer.InitResource(RHICmdList);
				});
		}
	}

	virtual ~FBasicTriangleSetSceneProxy() override
	{
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OverlaySceneProxy_GetDynamicMeshElements);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				for (const FBasicTriangleSetMeshBatchData& MeshBatchData : MeshBatchDatas)
				{
					FMeshBatch& Mesh = Collector.AllocateMesh();
					FMeshBatchElement& BatchElement = Mesh.Elements[0];
					BatchElement.IndexBuffer = &IndexBuffer;
					Mesh.bWireframe = false;
					Mesh.VertexFactory = &VertexFactory;
					Mesh.MaterialRenderProxy = MeshBatchData.MaterialProxy;

					FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
					DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), GetBounds(), GetLocalBounds(), true, false, AlwaysHasVelocity());
					BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

					BatchElement.FirstIndex = MeshBatchData.StartIndex;
					BatchElement.NumPrimitives = MeshBatchData.NumPrimitives;
					BatchElement.MinVertexIndex = MeshBatchData.MinVertexIndex;
					BatchElement.MaxVertexIndex = MeshBatchData.MaxVertexIndex;
					Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
					Mesh.Type = PT_TriangleList;
					Mesh.DepthPriorityGroup = SDPG_World;
					Mesh.bCanApplyViewModeOverrides = false;
					Collector.AddMesh(ViewIndex, Mesh);
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }

	virtual SIZE_T GetTypeHash() const override
	{
		static SIZE_T UniquePointer;
		return reinterpret_cast<SIZE_T>(&UniquePointer);
	}

private:

	// Triangle type specific implementation to get an FVector3f out of the generic point type.
	static FVector3f ToFVector3f(const FVector2f& Vector) { return FVector3f(Vector.X, Vector.Y, 0.0f); }
	static FVector3f ToFVector3f(const FVector3f& Vector) { return Vector; }

	TArray<FBasicTriangleSetMeshBatchData> MeshBatchDatas;
	FMaterialRelevance MaterialRelevance;
	FLocalVertexFactory VertexFactory;
	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;
};


UBasicTriangleSetComponentBase::UBasicTriangleSetComponentBase()
{
	CastShadow = false;
	bSelectable = false;
	PrimaryComponentTick.bCanEverTick = false;

	UPrimitiveComponent::SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

void UBasicTriangleSetComponentBase::ClearComponent()
{
	MarkRenderStateDirty();
	bBoundsDirty = true;
}


void UBasicTriangleSetComponentBase::SetTriangleMaterial(UMaterialInterface* InTriangleMaterial)
{
	TriangleMaterial = InTriangleMaterial;
	UMeshComponent::SetMaterial(0, InTriangleMaterial);
}

void UBasicTriangleSetComponentBase::SetTriangleSetParameters(FColor InColor, const FVector3f& InNormal)
{
	Color = InColor;
	Normal = InNormal;
	bBoundsDirty = true;
}


FPrimitiveSceneProxy* UBasic2DTriangleSetComponent::CreateSceneProxy()
{
	return Elements.Num() > 0 ? new FBasicTriangleSetSceneProxy(this) : nullptr;
}

FPrimitiveSceneProxy* UBasic3DTriangleSetComponent::CreateSceneProxy()
{
	return Elements.Num() > 0 ? new FBasicTriangleSetSceneProxy(this) : nullptr;
}
