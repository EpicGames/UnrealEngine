// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operators/UVEditorUVTransformOp.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Parameterization/MeshUVPacking.h"
#include "Selections/MeshConnectedComponents.h"
#include "Parameterization/MeshUDIMClassifier.h"
#include "Async/ParallelFor.h"
#include "UVEditorUXSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorUVTransformOp)

using namespace UE::Geometry;

namespace TransformOpLocals
{

}

void FUVEditorUVTransformBaseOp::SetTransform(const FTransformSRT3d& Transform)
{
	ResultTransform = Transform;
}

void FUVEditorUVTransformBaseOp::SetSelection(const TSet<int32>& EdgeSelectionIn, const TSet<int32>& VertexSelectionIn)
{
	EdgeSelection = EdgeSelectionIn;
	VertexSelection = VertexSelectionIn;
}

void FUVEditorUVTransformBaseOp::SortComponentsByBoundingBox(bool bSortXThenY)
{
	SpatiallyOrderedComponentIndex.SetNum(UVComponents->Num());
	for (int32 Index = 0; Index < UVComponents->Num(); ++Index)
	{
		SpatiallyOrderedComponentIndex[Index] = Index;
	}
	TArrayView<int32> IndexView(SpatiallyOrderedComponentIndex.GetData(), SpatiallyOrderedComponentIndex.Num());
	TArrayView<FAxisAlignedBox2d> BBView(PerComponentBoundingBoxes.GetData(), PerComponentBoundingBoxes.Num());

	const int32 SortDirection = bSortXThenY ? 1 : 0;

	auto BBIndexSort = [&BBView, SortDirection](const int32& A, const int32& B)
	{
		FVector2D CenterDiff = BBView[A].Center() - BBView[B].Center();

		if (CenterDiff[1-SortDirection] < 0)
		{
			return true;
		}
		else if (FMath::IsNearlyZero(CenterDiff[1 - SortDirection]) && CenterDiff[SortDirection] < 0)
		{
			return true;
		}
		return false;
	};

	IndexView.StableSort(BBIndexSort);
}


void FUVEditorUVTransformBaseOp::RebuildBoundingBoxes()
{	
	int32 NumComponents = UVComponents->Num();
	PerComponentBoundingBoxes.SetNum(NumComponents);
	OverallBoundingBox = FAxisAlignedBox2d::Empty();

	ParallelFor(NumComponents, [&](int32 k)
		{
			PerComponentBoundingBoxes[k] = FAxisAlignedBox2d::Empty();
			const TArray<int>& Vertices = (*UVComponents)[k].Indices;
			for (int32 Vid : Vertices)
			{				
				PerComponentBoundingBoxes[k].Contain((FVector2d)FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(ResultMesh->GetVertexRef(Vid)));
			}
		});

	for (int32 Cid = 0; Cid < NumComponents; ++Cid)
	{
		OverallBoundingBox.Contain(PerComponentBoundingBoxes[Cid]);
	}

}

void FUVEditorUVTransformBaseOp::CollectTransformElements()
{
	if (VertexSelection.IsSet())
	{
		TransformingElements = TSet<int32>();
	}

	int32 NumComponents = UVComponents->Num();

	for (int32 k = 0; k < NumComponents; ++k)
	{
		const TArray<int>& Vertices = (*UVComponents)[k].Indices;
		for (int Vid : Vertices)
		{
			if (VertexSelection.IsSet() && VertexSelection.GetValue().Contains(Vid))
			{
				TransformingElements.GetValue().Add(Vid);
			}
			ElementToComponent.Add(Vid, k);
		}
	}
}

FVector2f FUVEditorUVTransformBaseOp::GetAlignmentPointFromBoundingBoxAndDirection(EUVEditorAlignDirectionBackend Direction, const FAxisAlignedBox2d& BoundingBox)
{
	FVector2f AlignmentPoint = FVector2f();

	switch (Direction)
	{
	case EUVEditorAlignDirectionBackend::None:
		AlignmentPoint = FVector2f(0,0);
		break;
	case EUVEditorAlignDirectionBackend::Top:
		AlignmentPoint = FVector2f(static_cast<float>(BoundingBox.Center().X), static_cast<float>(BoundingBox.Max.Y));
		break;
	case EUVEditorAlignDirectionBackend::Bottom:
		AlignmentPoint = FVector2f(static_cast<float>(BoundingBox.Center().X), static_cast<float>(BoundingBox.Min.Y));
		break;
	case EUVEditorAlignDirectionBackend::Left:
		AlignmentPoint = FVector2f(static_cast<float>(BoundingBox.Min.X), static_cast<float>(BoundingBox.Center().Y));
		break;
	case EUVEditorAlignDirectionBackend::Right:
		AlignmentPoint = FVector2f(static_cast<float>(BoundingBox.Max.X), static_cast<float>(BoundingBox.Center().Y));
		break;
	case EUVEditorAlignDirectionBackend::CenterVertically:
		AlignmentPoint = FVector2f(static_cast<float>(BoundingBox.Center().X), static_cast<float>(BoundingBox.Center().Y));
		break;
	case EUVEditorAlignDirectionBackend::CenterHorizontally:
		AlignmentPoint = FVector2f(static_cast<float>(BoundingBox.Center().X), static_cast<float>(BoundingBox.Center().Y));
		break;
	default:
		ensure(false);
	}

	return AlignmentPoint;
}

FVector2f FUVEditorUVTransformBaseOp::GetAlignmentPointFromUDIMAndDirection(EUVEditorAlignDirectionBackend Direction, FVector2i UDIMTile)
{
	FVector2f AlignmentPoint = FVector2f();

	FVector2f UDIMLowerCorner(UDIMTile);

	switch (Direction)
	{
	case EUVEditorAlignDirectionBackend::None:
		AlignmentPoint = FVector2f(0, 0);
		break;
	case EUVEditorAlignDirectionBackend::Top:
		AlignmentPoint = FVector2f(UDIMLowerCorner.X + 0.5f, UDIMLowerCorner.Y + 1.0f);
		break;
	case EUVEditorAlignDirectionBackend::Bottom:
		AlignmentPoint = FVector2f(UDIMLowerCorner.X + 0.5f, UDIMLowerCorner.Y);
		break;
	case EUVEditorAlignDirectionBackend::Left:
		AlignmentPoint = FVector2f(UDIMLowerCorner.X, UDIMLowerCorner.Y + 0.5f);
		break;
	case EUVEditorAlignDirectionBackend::Right:
		AlignmentPoint = FVector2f(UDIMLowerCorner.X + 1.0f, UDIMLowerCorner.Y + 0.5f);
		break;
	case EUVEditorAlignDirectionBackend::CenterVertically:
		AlignmentPoint = FVector2f(UDIMLowerCorner.X + 0.5f, UDIMLowerCorner.Y + 0.5f);
		break;
	case EUVEditorAlignDirectionBackend::CenterHorizontally:
		AlignmentPoint = FVector2f(UDIMLowerCorner.X + 0.5f, UDIMLowerCorner.Y + 0.5f);
		break;
	default:
		ensure(false);
	}

	return AlignmentPoint;
}

void FUVEditorUVTransformBaseOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(*OriginalMesh, true, true, true, true);
	if (!ensureMsgf(ResultMesh->HasAttributes(), TEXT("Attributes not found on mesh? Conversion should always create them, so this operator should not need to do so.")))
	{
		ResultMesh->EnableAttributes();
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	VisualizationPivots.Empty();
	int UVLayerInput = UVLayerIndex;
	ActiveUVLayer = ResultMesh->Attributes()->GetUVLayer(UVLayerInput);
	ensure(ActiveUVLayer);

	auto UVIslandPredicate = [this](int32 Vertex0, int32 Vertex1)
	{
		if (GroupingMode == EUVEditorAlignDistributeGroupingModeBackend::IndividualVertices)
		{
			return false;
		}

		if (GroupingMode != EUVEditorAlignDistributeGroupingModeBackend::IndividualVertices
			&& EdgeSelection.IsSet())
		{
			const TSet<int32>& SelectedEdges = EdgeSelection.GetValue();
			return SelectedEdges.Contains(ResultMesh->FindEdge(Vertex0, Vertex1));
		}
		else
		{
			return true;
		}
	};

	UVComponents = MakeShared<FMeshConnectedComponents>(ResultMesh.Get());
	if(GroupingMode == EUVEditorAlignDistributeGroupingModeBackend::EnclosingBoundingBox)
	{
		// If we're using the enclosing bounding box, we'll just "fake" one big component and proceed as normal.
		UVComponents->Components.Add( new FMeshConnectedComponents::FComponent());
		if (VertexSelection.IsSet())
		{
			UVComponents->Components[0].Indices = VertexSelection.GetValue().Array();
		}
		else
		{
			UVComponents->Components[0].Indices.Reserve(ResultMesh.Get()->VertexCount());
			for (int32 Vid : ResultMesh.Get()->VertexIndicesItr())
			{
				UVComponents->Components[0].Indices.Add(Vid);
			}
		}

	}
	else
	{
		if (VertexSelection.IsSet())
		{
			UVComponents->FindConnectedVertices(VertexSelection.GetValue().Array(), UVIslandPredicate);
		}
		else
		{
			UVComponents->FindConnectedVertices(UVIslandPredicate);
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	CollectTransformElements();

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	HandleTransformationOp(Progress);
}

FVector2f FUVEditorUVTransformOp::GetPivotFromMode(int32 ElementID, EUVEditorPivotTypeBackend Mode)
{
	FVector2f Pivot = FVector2f::ZeroVector;
	const int32* Component;

	switch (Mode)
	{
	case EUVEditorPivotTypeBackend::Origin:
		Pivot = FVector2f(0, 0);
		break;
	case EUVEditorPivotTypeBackend::BoundingBoxCenter:
		Pivot = FVector2f(OverallBoundingBox.Center());
		break;
	case EUVEditorPivotTypeBackend::IndividualBoundingBoxCenter:
		Component = ElementToComponent.Find(ElementID);
		if (ensure(Component != nullptr))
		{
			Pivot = FVector2f(PerComponentBoundingBoxes[*Component].Center());
		}
		break;
	case EUVEditorPivotTypeBackend::Manual:
		Pivot = FVector2f(ManualPivot);
		break;
	}
	return Pivot;
}

void FUVEditorUVTransformOp::HandleTransformationOp(FProgressCancel * Progress)
{
	auto ScaleFunc = [this](int32 Vid)
	{
		FVector2f ScalePivot = GetPivotFromMode(Vid, PivotMode);
		FVector2f UV = FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(ResultMesh->GetVertexRef(Vid));
		UV = (UV - ScalePivot);
		UV[0] *= static_cast<float>(Scale[0]);
		UV[1] *= static_cast<float>(Scale[1]);
		UV = (UV + ScalePivot);
		ResultMesh->SetVertex(Vid, FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(UV));
	};

	auto BaseRotFunc = [this](int32 Vid, float RotationIn, const FVector2f& Pivot)
	{
		FVector2f UV = FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(ResultMesh->GetVertexRef(Vid));
		FVector2f UV_Rotated;
		// We are flipping the sign here to match the conventions in other UV editors
		// where positive values are clockwise and negative values are counterclockwise.
		double RotationInRadians = -RotationIn / 180.0 * UE_PI;
		UV = (UV - Pivot);
		UV_Rotated[0] = UV[0] * static_cast<float>(FMath::Cos(RotationInRadians)) - UV[1] * static_cast<float>(FMath::Sin(RotationInRadians));
		UV_Rotated[1] = UV[0] * static_cast<float>(FMath::Sin(RotationInRadians)) + UV[1] * static_cast<float>(FMath::Cos(RotationInRadians));
		UV = (UV_Rotated + Pivot);
		ResultMesh->SetVertex(Vid, FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(UV));
	};

	auto QuickRotFunc = [this, &BaseRotFunc](int32 Vid)
	{
		FVector2f RotationPivot = GetPivotFromMode(Vid, EUVEditorPivotTypeBackend::BoundingBoxCenter);
		BaseRotFunc(Vid, QuickRotation, RotationPivot);
	};

	auto RotFunc = [this, &BaseRotFunc](int32 Vid)
	{
		FVector2f RotationPivot = GetPivotFromMode(Vid, PivotMode);
		BaseRotFunc(Vid, Rotation, RotationPivot);
	};

	auto QuickTranslateFunc = [this](int32 Vid)
	{
		FVector2f UV = FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(ResultMesh->GetVertexRef(Vid));
		UV = (UV + FVector2f(QuickTranslation));
		ResultMesh->SetVertex(Vid, FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(UV));
	};

	auto TranslateFunc = [this](int32 Vid)
	{	
		FVector2f RotationPivot = FVector2f(0,0);
		if (TranslationMode == EUVEditorTranslationModeBackend::Absolute)
		{
			RotationPivot = GetPivotFromMode(Vid, PivotMode);
		}				
		FVector2f UV = FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(ResultMesh->GetVertexRef(Vid));
		UV = (UV + FVector2f(Translation) - RotationPivot);
		ResultMesh->SetVertex(Vid, FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(UV));
	};

	auto ApplyTransformFunc = [this, &Progress](TFunction<void(int32 Vid)> TransformFunc)
	{
		RebuildBoundingBoxes();

		if (TransformingElements.IsSet())
		{
			for (int ElementID : TransformingElements.GetValue())
			{
				TransformFunc(ElementID);

				if (Progress && Progress->Cancelled())
				{
					return;
				}
			}
		}
		else
		{
			for (int ElementID : ResultMesh->VertexIndicesItr())
			{
				TransformFunc(ElementID);

				if (Progress && Progress->Cancelled())
				{
					return;
				}
			}
		}
	};

	if (!FMath::IsNearlyZero(QuickRotation))
	{
		ApplyTransformFunc(QuickRotFunc);
	}
	if (!FMath::IsNearlyZero(QuickTranslation.X) || !FMath::IsNearlyZero(QuickTranslation.Y) )
	{
		ApplyTransformFunc(QuickTranslateFunc);
	}

	if (!FMath::IsNearlyEqual(Scale.X, 1.0f) || !FMath::IsNearlyEqual(Scale.Y, 1.0f))
	{
		ApplyTransformFunc(ScaleFunc);
	}
	if (!FMath::IsNearlyZero(Rotation))
	{
		ApplyTransformFunc(RotFunc);
	}
	if (!FMath::IsNearlyZero(Translation.X) || !FMath::IsNearlyZero(Translation.Y)
		|| TranslationMode == EUVEditorTranslationModeBackend::Absolute)
	{
		ApplyTransformFunc(TranslateFunc);
	}

	// Provide pivots for tool visualization after operation completes.
	switch (PivotMode)
	{
		case EUVEditorPivotTypeBackend::Origin:
			VisualizationPivots.Add(FVector2D(0, 0));
			break;
		case EUVEditorPivotTypeBackend::BoundingBoxCenter:		
			RebuildBoundingBoxes();
			VisualizationPivots.Add((FVector2D)OverallBoundingBox.Center());
			break;
		case EUVEditorPivotTypeBackend::IndividualBoundingBoxCenter:
			RebuildBoundingBoxes();
			for (const FAxisAlignedBox2d& ComponentBoundingBox : PerComponentBoundingBoxes)
			{
				VisualizationPivots.Add((FVector2D)ComponentBoundingBox.Center());
			}
			break;
		case EUVEditorPivotTypeBackend::Manual:
			VisualizationPivots.Add(ManualPivot);
			break;
		default:
			ensure(false);
	}
	
}


void FUVEditorUVAlignOp::HandleTransformationOp(FProgressCancel* Progress)
{
	int32 NumComponents = UVComponents->Num();
	RebuildBoundingBoxes();

	auto TranslateFunc = [this](const FVector2f& Translation, int32 Vid)
	{
		FVector2f UV = FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(ResultMesh->GetVertexRef(Vid));
		UV = (UV + Translation);
		ResultMesh->SetVertex(Vid, FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(UV));
	};

	auto TranslationFromAlignmentPoints = [this](const FVector2f& PointTo, const FVector2f& PointFrom)
	{
		switch (AlignDirection)
		{
		case EUVEditorAlignDirectionBackend::None:
			return FVector2f(0, 0);
		case EUVEditorAlignDirectionBackend::Top:
			return FVector2f(0, PointTo.Y - PointFrom.Y);
		case EUVEditorAlignDirectionBackend::Bottom:
			return FVector2f(0, PointTo.Y - PointFrom.Y);
		case EUVEditorAlignDirectionBackend::Left:
			return FVector2f(PointTo.X - PointFrom.X, 0);
		case EUVEditorAlignDirectionBackend::Right:
			return FVector2f(PointTo.X - PointFrom.X, 0);
		case EUVEditorAlignDirectionBackend::CenterVertically:
			return FVector2f(PointTo.X - PointFrom.X, 0);
		case EUVEditorAlignDirectionBackend::CenterHorizontally:
			return FVector2f(0, PointTo.Y - PointFrom.Y);
		default:
			ensure(false);
			return FVector2f(0, 0);
		}
	};

	auto ComponentToUDIM = [this](int32 ComponentID)
	{
		return FDynamicMeshUDIMClassifier::ClassifyBoundingBoxToUDIM(ActiveUVLayer, PerComponentBoundingBoxes[ComponentID]);
	};
	auto UVElementToUDIM = [this](int32 Vid)
	{
		return FDynamicMeshUDIMClassifier::ClassifyPointToUDIM(FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(ResultMesh->GetVertexRef(Vid)));
	};
	auto ComponentToPoint = [this](int32 ComponentID)
	{
		return GetAlignmentPointFromBoundingBoxAndDirection(AlignDirection, PerComponentBoundingBoxes[ComponentID]);
	};
	auto UVElementToPoint = [this](int32 Vid)
	{
		return FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(ResultMesh->GetVertexRef(Vid));
	};


	auto GetNeededTranslateOffset = [this, TranslationFromAlignmentPoints](int32 PointID, EUVEditorAlignAnchorBackend AlignAnchorMode,
		TFunction<FVector2i(int32 ID)> PointIDToUDIM, TFunction<FVector2f(int32 ID)> PointIDToPoint)
	{
		FVector2f PointToAlign = PointIDToPoint(PointID);
		switch (AlignAnchorMode)
		{
		case EUVEditorAlignAnchorBackend::UDIMTile:
		{
			FVector2i UDIM = PointIDToUDIM(PointID);
			FVector2f TileAlignmentPoint = GetAlignmentPointFromUDIMAndDirection(AlignDirection, UDIM);
			return TranslationFromAlignmentPoints(TileAlignmentPoint, PointToAlign);
		}
		case EUVEditorAlignAnchorBackend::BoundingBox:
		{
			FVector2f BoundingBoxAlignmentPoint = GetAlignmentPointFromBoundingBoxAndDirection(AlignDirection, OverallBoundingBox);
			return TranslationFromAlignmentPoints(BoundingBoxAlignmentPoint, PointToAlign);
		}
		case EUVEditorAlignAnchorBackend::Manual:
		{
			return TranslationFromAlignmentPoints((FVector2f)ManualAnchor, PointToAlign);
		}
		default:
			ensure(false);
			return FVector2f();
		}
	};

	if (TransformingElements.IsSet())
	{
		for (int ElementID : TransformingElements.GetValue())
		{
			int32* ComponentID = ElementToComponent.Find(ElementID);
			if (ensure(ComponentID))
			{
				FVector2f TranslateOffset = GetNeededTranslateOffset(*ComponentID, AlignAnchor, ComponentToUDIM, ComponentToPoint);
				TranslateFunc(TranslateOffset, ElementID);
			}
			if (Progress && Progress->Cancelled())
			{
				return;
			}
		}
	}
	else
	{
		for (int ElementID : ActiveUVLayer->ElementIndicesItr())
		{

			int32* ComponentID = ElementToComponent.Find(ElementID);
			if (ensure(ComponentID))
			{
				FVector2f TranslateOffset = GetNeededTranslateOffset(*ComponentID, AlignAnchor, ComponentToUDIM, ComponentToPoint);
				TranslateFunc(TranslateOffset, ElementID);
			}
			if (Progress && Progress->Cancelled())
			{
				return;
			}
		}
	}
	
	// Provide pivots for tool visualization after operation completes.
	switch (AlignAnchor)
	{
		case EUVEditorAlignAnchorBackend::BoundingBox:
		{
			if (AlignDirection != EUVEditorAlignDirectionBackend::None)
			{
				FVector2f BoundingBoxAlignmentPoint = GetAlignmentPointFromBoundingBoxAndDirection(AlignDirection, OverallBoundingBox);
				VisualizationPivots.Add((FVector2D)BoundingBoxAlignmentPoint);
			}
		}
		break;
		case EUVEditorAlignAnchorBackend::Manual:
		{
			VisualizationPivots.Add(ManualAnchor);
		}
		break;
		default:
			break; // Currently we don't support visualizing anchors for UDIMTiles...
	}
}


void FUVEditorUVDistributeOp::HandleTransformationOp(FProgressCancel* Progress)
{
	int32 NumComponents = UVComponents->Num();
	RebuildBoundingBoxes();	
	TArray<FVector2f> PerComponentTranslation;
	

	auto ComputeDistributeTranslations = [this, NumComponents](bool bVertical, EUVEditorAlignDirectionBackend EdgeDirection, float SpreadDirection, bool bEqualizeSpacing)
	{
		SortComponentsByBoundingBox(!bVertical);
		TArray<FVector2f> PerComponentTranslation;
		PerComponentTranslation.SetNum(NumComponents);
		float TotalDistance = 0.0f;
		for (int32 Cid = 0; Cid < NumComponents; ++Cid)
		{
			TotalDistance += static_cast<float>( bVertical ? PerComponentBoundingBoxes[Cid].Height() : PerComponentBoundingBoxes[Cid].Width());
		}
		float BoundingBoxDistance = static_cast<float>( bVertical ? OverallBoundingBox.Height() : OverallBoundingBox.Width());
		if (bEnableManualDistances)
		{
			BoundingBoxDistance = ManualExtent;
		}
		float GapSpace = FMath::Max(0, (BoundingBoxDistance - TotalDistance) / (NumComponents - 1));
		if(bEnableManualDistances && bEqualizeSpacing)
		{
			GapSpace = ManualSpacing;
		}
		float PerComponentSpace = BoundingBoxDistance / (NumComponents - 1);

		float NextPosition = 0.0f;
		FVector2f OverallAlignmentPoint;
		switch (EdgeDirection) {
		case EUVEditorAlignDirectionBackend::CenterHorizontally:
			OverallAlignmentPoint = GetAlignmentPointFromBoundingBoxAndDirection(EUVEditorAlignDirectionBackend::Left, OverallBoundingBox);
			break;
		case EUVEditorAlignDirectionBackend::CenterVertically:
			OverallAlignmentPoint = GetAlignmentPointFromBoundingBoxAndDirection(EUVEditorAlignDirectionBackend::Bottom, OverallBoundingBox);
			break;
		default:
			OverallAlignmentPoint = GetAlignmentPointFromBoundingBoxAndDirection(EdgeDirection, OverallBoundingBox);
			break;
		}

		for (int32 Cid: SpatiallyOrderedComponentIndex)
		{
			FVector2f ComponentAlignmentPoint = GetAlignmentPointFromBoundingBoxAndDirection(EdgeDirection, PerComponentBoundingBoxes[Cid]);
			if (bVertical)
			{
				PerComponentTranslation[Cid] = FVector2f(0.0, NextPosition + (OverallAlignmentPoint.Y - ComponentAlignmentPoint.Y));
			}
			else
			{
				PerComponentTranslation[Cid] = FVector2f(NextPosition + (OverallAlignmentPoint.X - ComponentAlignmentPoint.X), 0.0);
			}
			if (bEqualizeSpacing)
			{
				float ComponentSpace = static_cast<float>( bVertical ? PerComponentBoundingBoxes[Cid].Height() : PerComponentBoundingBoxes[Cid].Width());
				NextPosition += SpreadDirection * (ComponentSpace + GapSpace);
			}
			else
			{
				NextPosition += SpreadDirection * PerComponentSpace;
			}
		}
		return PerComponentTranslation;
	};

	auto ComputeMinimallyRemoveOverlap = [this, NumComponents]()
	{
		SortComponentsByBoundingBox();
		FVector2D Center = OverallBoundingBox.Center();

		// Create a local copy so we can move things around...
		TArray<FAxisAlignedBox2d> ComponentBoundingBoxes = PerComponentBoundingBoxes;
		// This could be tunable, if we think it needs to be, or perhaps based off bounding box size somehow?
		double StepAmount = 0.01;

		if (bEnableManualDistances)
		{
			for (int32 Cid = 0; Cid < NumComponents; ++Cid)
			{
				ComponentBoundingBoxes[Cid].Expand(ManualSpacing / 2.0);
			}
		}

		bool bAnyBoxesIntersecting = false;
		// The following loop will iteratively move all overlapping bounding boxes away from each other and from the center.
		// This will eventually terminate, by "exploding" all boxes away from the center point in the worst case. If a box
		// doesn't overlap, it won't move
		// TODO: Implement a closer fit version that looks at real overlaps between island elements instead of just bounding boxes
		// TODO: Implement an improvement that uses clustering to prevent far away islands from moving too much away from a global center
		do
		{
			bAnyBoxesIntersecting = false;
			for (int32 Cid = 0; Cid < NumComponents; ++Cid) 
			{
				FVector2D Adjustment = FVector2D::Zero();
				bool bIsOverlapping = false;
				for (int32 CidOverlap = 0; CidOverlap < NumComponents; ++CidOverlap)
				{
					if (Cid == CidOverlap)
					{
						continue;
					}
					// Move away from overlapping boxes
					if (ComponentBoundingBoxes[Cid].Intersects(ComponentBoundingBoxes[CidOverlap]))
					{
						FVector2D Offset = (ComponentBoundingBoxes[Cid].Center() - ComponentBoundingBoxes[CidOverlap].Center());
						Offset.Normalize();
						if(Offset.IsNearlyZero())
						{
							// Edge case where two bounding boxes have identically overlapping center points.
							// We need to break them up so they don't move in concert
							// Our "emergency" separation vector places the components deterministically
							// around a circle so they all follow different paths.
							Offset.X = FMath::Cos((2 * UE_PI / NumComponents) * Cid);
							Offset.Y = FMath::Sin((2 * UE_PI / NumComponents) * Cid);							
						}
						Adjustment = Adjustment + (Offset * StepAmount);
						bIsOverlapping = true;
					}
				}
				ComponentBoundingBoxes[Cid].Min += Adjustment;
				ComponentBoundingBoxes[Cid].Max += Adjustment;
				bAnyBoxesIntersecting |= bIsOverlapping;
			}
		} while (bAnyBoxesIntersecting);

		// Once all bounding boxes have been moved, record their movement to the translation array return value
		TArray<FVector2f> PerComponentTranslation;
		PerComponentTranslation.SetNum(NumComponents);
		for (int32 Cid = 0; Cid < NumComponents; ++Cid)
		{
			PerComponentTranslation[Cid] = (FVector2f)(ComponentBoundingBoxes[Cid].Center() - PerComponentBoundingBoxes[Cid].Center());
		}
		return PerComponentTranslation;
	};

	switch (DistributeMode)
	{
		case EUVEditorDistributeModeBackend::None:
			PerComponentTranslation.Init(FVector2f::ZeroVector, NumComponents);
			break;
		case EUVEditorDistributeModeBackend::TopEdges:
			PerComponentTranslation = ComputeDistributeTranslations(true, EUVEditorAlignDirectionBackend::Top, -1.0f, false);
			break;
		case EUVEditorDistributeModeBackend::BottomEdges:
			PerComponentTranslation = ComputeDistributeTranslations(true, EUVEditorAlignDirectionBackend::Bottom, 1.0f, false);
			break;
		case EUVEditorDistributeModeBackend::LeftEdges:
			PerComponentTranslation = ComputeDistributeTranslations(false, EUVEditorAlignDirectionBackend::Left, 1.0f, false);
			break;
		case EUVEditorDistributeModeBackend::RightEdges:
			PerComponentTranslation = ComputeDistributeTranslations(false, EUVEditorAlignDirectionBackend::Right, -1.0f, false);
			break;
		case EUVEditorDistributeModeBackend::CentersVertically:
			PerComponentTranslation = ComputeDistributeTranslations(true, EUVEditorAlignDirectionBackend::CenterVertically, 1.0f, false);
			break;
		case EUVEditorDistributeModeBackend::CentersHorizontally:
			PerComponentTranslation = ComputeDistributeTranslations(false, EUVEditorAlignDirectionBackend::CenterHorizontally, 1.0f, false);
			break;
		case EUVEditorDistributeModeBackend::HorizontalSpace:
			PerComponentTranslation = ComputeDistributeTranslations(false, EUVEditorAlignDirectionBackend::Left, 1.0f, true);
			break;
		case EUVEditorDistributeModeBackend::VerticalSpace:
			PerComponentTranslation = ComputeDistributeTranslations(true, EUVEditorAlignDirectionBackend::Bottom, 1.0f, true);
			break;
		case EUVEditorDistributeModeBackend::MinimallyRemoveOverlap:
			PerComponentTranslation = ComputeMinimallyRemoveOverlap();
			break;
		default:
			ensure(false);
	}

	auto TranslateFunc = [this](const FVector2f& Translation, int32 Vid)
	{
		FVector2f UV = FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(ResultMesh->GetVertexRef(Vid));
		UV = (UV + Translation);
		ResultMesh->SetVertex(Vid, FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(UV));
	};

	if (TransformingElements.IsSet())
	{
		for (int ElementID : TransformingElements.GetValue())
		{
			TranslateFunc(PerComponentTranslation[*ElementToComponent.Find(ElementID)], ElementID);
			if (Progress && Progress->Cancelled())
			{
				return;
			}
		}
	}
	else
	{
		for (int ElementID : ActiveUVLayer->ElementIndicesItr())
		{
			TranslateFunc(PerComponentTranslation[*ElementToComponent.Find(ElementID)], ElementID);
			if (Progress && Progress->Cancelled())
			{
				return;
			}
		}
	}

}


TUniquePtr<FDynamicMeshOperator> UUVEditorUVTransformOperatorFactory::MakeNewOperator()
{
	switch (TransformType)
	{
	case EUVEditorUVTransformType::Transform:
	{
		UUVEditorUVTransformProperties* OpSettings = CastChecked<UUVEditorUVTransformProperties>(Settings.Get());

		TUniquePtr<FUVEditorUVTransformOp> Op = MakeUnique<FUVEditorUVTransformOp>();
		Op->OriginalMesh = OriginalMesh;
		Op->SetTransform(TargetTransform);
		if (EdgeSelection.IsSet() && VertexSelection.IsSet())
		{
			Op->SetSelection(EdgeSelection.GetValue(), VertexSelection.GetValue());
		}
		Op->UVLayerIndex = GetSelectedUVChannel();

		Op->Scale = OpSettings->Scale;
		Op->Rotation = OpSettings->Rotation;
		Op->Translation = OpSettings->Translation;

		Op->QuickTranslation = OpSettings->QuickTranslation;
		Op->QuickRotation = OpSettings->QuickRotation;

		switch (OpSettings->TranslationMode)
		{
		case EUVEditorTranslationMode::Relative:
			Op->TranslationMode = EUVEditorTranslationModeBackend::Relative;
			break;
		case EUVEditorTranslationMode::Absolute:
			Op->TranslationMode = EUVEditorTranslationModeBackend::Absolute;
			break;
		default:
			ensure(false);
		}

		switch (OpSettings->PivotMode)
		{
		case EUVEditorPivotType::Origin:
			Op->PivotMode = EUVEditorPivotTypeBackend::Origin;
			break;
		case EUVEditorPivotType::IndividualBoundingBoxCenter:
			Op->PivotMode = EUVEditorPivotTypeBackend::IndividualBoundingBoxCenter;
			break;
		case EUVEditorPivotType::BoundingBoxCenter:
			Op->PivotMode = EUVEditorPivotTypeBackend::BoundingBoxCenter;
			break;
		case EUVEditorPivotType::Manual:
			Op->PivotMode = EUVEditorPivotTypeBackend::Manual;
			break;
		default:
			ensure(false);
		}
		Op->ManualPivot = OpSettings->ManualPivot;
		Op->GroupingMode = EUVEditorAlignDistributeGroupingModeBackend::IndividualBoundingBoxes;

		return Op;
	}
	case EUVEditorUVTransformType::Align:
	{
		UUVEditorUVAlignProperties* OpSettings = CastChecked<UUVEditorUVAlignProperties>(Settings.Get());

		TUniquePtr<FUVEditorUVAlignOp> Op = MakeUnique<FUVEditorUVAlignOp>();
		Op->OriginalMesh = OriginalMesh;
		Op->SetTransform(TargetTransform);
		if (EdgeSelection.IsSet() && VertexSelection.IsSet())
		{
			Op->SetSelection(EdgeSelection.GetValue(), VertexSelection.GetValue());
		}
		Op->UVLayerIndex = GetSelectedUVChannel();

		switch (OpSettings->AlignAnchor)
		{
		//case EUVEditorAlignAnchor::FirstItem:
		//	Op->AlignAnchor = EUVEditorAlignAnchorBackend::FirstItem;
		//	break;
		case EUVEditorAlignAnchor::UDIMTile:
			Op->AlignAnchor = EUVEditorAlignAnchorBackend::UDIMTile;
			break;
		case EUVEditorAlignAnchor::BoundingBox:
			Op->AlignAnchor = EUVEditorAlignAnchorBackend::BoundingBox;
			break;
		case EUVEditorAlignAnchor::Manual:
			Op->AlignAnchor = EUVEditorAlignAnchorBackend::Manual;
			break;
		default:
			ensure(false);
		}

		switch (OpSettings->AlignDirection)
		{
		case EUVEditorAlignDirection::None:
			Op->AlignDirection = EUVEditorAlignDirectionBackend::None;
			break;
		case EUVEditorAlignDirection::Top:
			Op->AlignDirection = EUVEditorAlignDirectionBackend::Top;
			break;
		case EUVEditorAlignDirection::Bottom:
			Op->AlignDirection = EUVEditorAlignDirectionBackend::Bottom;
			break;
		case EUVEditorAlignDirection::Left:
			Op->AlignDirection = EUVEditorAlignDirectionBackend::Left;
			break;
		case EUVEditorAlignDirection::Right:
			Op->AlignDirection = EUVEditorAlignDirectionBackend::Right;
			break;
		case EUVEditorAlignDirection::CenterVertically:
			Op->AlignDirection = EUVEditorAlignDirectionBackend::CenterVertically;
			break;
		case EUVEditorAlignDirection::CenterHorizontally:
			Op->AlignDirection = EUVEditorAlignDirectionBackend::CenterHorizontally;
			break;
		default:
			ensure(false);
		}

		switch (OpSettings->Grouping)
		{
		case EUVEditorAlignDistributeGroupingMode::IndividualBoundingBoxes:
			Op->GroupingMode = EUVEditorAlignDistributeGroupingModeBackend::IndividualBoundingBoxes;
			break;
		case EUVEditorAlignDistributeGroupingMode::EnclosingBoundingBox:
			Op->GroupingMode = EUVEditorAlignDistributeGroupingModeBackend::EnclosingBoundingBox;
			break;
		case EUVEditorAlignDistributeGroupingMode::IndividualVertices:
			Op->GroupingMode = EUVEditorAlignDistributeGroupingModeBackend::IndividualVertices;
			break;
		default:
			ensure(false);
		}

		Op->ManualAnchor = OpSettings->ManualAnchor;

		return Op;
	}
	case EUVEditorUVTransformType::Distribute:
	{
		UUVEditorUVDistributeProperties* OpSettings = CastChecked<UUVEditorUVDistributeProperties>(Settings.Get());

		TUniquePtr<FUVEditorUVDistributeOp> Op = MakeUnique<FUVEditorUVDistributeOp>();
		Op->OriginalMesh = OriginalMesh;
		Op->SetTransform(TargetTransform);
		if (EdgeSelection.IsSet() && VertexSelection.IsSet())
		{
			Op->SetSelection(EdgeSelection.GetValue(), VertexSelection.GetValue());
		}
		Op->UVLayerIndex = GetSelectedUVChannel();

		switch (OpSettings->DistributeMode)
		{
		case EUVEditorDistributeMode::None:
			Op->DistributeMode = EUVEditorDistributeModeBackend::None;
			break;
		case EUVEditorDistributeMode::LeftEdges:
			Op->DistributeMode = EUVEditorDistributeModeBackend::LeftEdges;
			break;
		case EUVEditorDistributeMode::RightEdges:
			Op->DistributeMode = EUVEditorDistributeModeBackend::RightEdges;
			break;
		case EUVEditorDistributeMode::TopEdges:
			Op->DistributeMode = EUVEditorDistributeModeBackend::TopEdges;
			break;
		case EUVEditorDistributeMode::BottomEdges:
			Op->DistributeMode = EUVEditorDistributeModeBackend::BottomEdges;
			break;
		case EUVEditorDistributeMode::CentersVertically:
			Op->DistributeMode = EUVEditorDistributeModeBackend::CentersVertically;
			break;
		case EUVEditorDistributeMode::CentersHorizontally:
			Op->DistributeMode = EUVEditorDistributeModeBackend::CentersHorizontally;
			break;
		case EUVEditorDistributeMode::VerticalSpace:
			Op->DistributeMode = EUVEditorDistributeModeBackend::VerticalSpace;
			break;
		case EUVEditorDistributeMode::HorizontalSpace:
			Op->DistributeMode = EUVEditorDistributeModeBackend::HorizontalSpace;
			break;
		case EUVEditorDistributeMode::MinimallyRemoveOverlap:
			Op->DistributeMode = EUVEditorDistributeModeBackend::MinimallyRemoveOverlap;
			break;
		default:
			ensure(false);
		}

		switch (OpSettings->Grouping)
		{
		case EUVEditorAlignDistributeGroupingMode::IndividualBoundingBoxes:
			Op->GroupingMode = EUVEditorAlignDistributeGroupingModeBackend::IndividualBoundingBoxes;
			break;
		case EUVEditorAlignDistributeGroupingMode::EnclosingBoundingBox:
			Op->GroupingMode = EUVEditorAlignDistributeGroupingModeBackend::EnclosingBoundingBox;
			break;
		case EUVEditorAlignDistributeGroupingMode::IndividualVertices:
			Op->GroupingMode = EUVEditorAlignDistributeGroupingModeBackend::IndividualVertices;
			break;
		default:
			ensure(false);
		}

		Op->bEnableManualDistances = OpSettings->bEnableManualDistances;
		Op->ManualExtent = OpSettings->ManualExtent;
		Op->ManualSpacing = OpSettings->ManualSpacing;

		return Op;
	}
	default:
		return nullptr;
	}
}

