// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/UVEditorMeshSelectionMechanic.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "ContextObjectStore.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "ContextObjects/UVToolViewportButtonsAPI.h"
#include "Drawing/TriangleSetComponent.h"
#include "Drawing/LineSetComponent.h"
#include "Drawing/PointSetComponent.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Engine/World.h"
#include "Intersection/IntrTriangle2AxisAlignedBox2.h"
#include "Intersection/IntersectionQueries2.h"
#include "ConvexVolume.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "Polyline3.h"
#include "Selections/MeshConnectedComponents.h"
#include "Spatial/GeometrySet3.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"
#include "UVEditorUXSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorMeshSelectionMechanic)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UUVEditorMeshSelectionMechanic"

namespace UVEditorMeshSelectionMechanicLocals
{
	template <typename InElementType>
	void ToggleItem(TSet<InElementType>& Set, InElementType Item)
	{
		if (Set.Remove(Item) == 0)
		{
			Set.Add(Item);
		}
	}
	
	using ESelectionMode = UUVToolSelectionAPI::EUVEditorSelectionMode;
	using FModeChangeOptions = UUVToolSelectionAPI::FSelectionMechanicModeChangeOptions;

	FUVToolSelection::EType ToCompatibleDynamicMeshSelectionType(ESelectionMode Mode)
	{
		switch (Mode)
		{
			case ESelectionMode::Mesh:
			case ESelectionMode::Island:
			case ESelectionMode::Triangle:
				return FUVToolSelection::EType::Triangle;
			case ESelectionMode::Edge:
				return FUVToolSelection::EType::Edge;
			case ESelectionMode::Vertex:
				return FUVToolSelection::EType::Vertex;
			case ESelectionMode::None: //doesn't actually matter what we return
				return FUVToolSelection::EType::Vertex;
		}
		ensure(false);
		return FUVToolSelection::EType::Vertex;
	}

	// Returns the marquee selection rectangle, obtained from the given CameraRectangle, projected to the XY plane
	FAxisAlignedBox2d GetRectangleXY(const FCameraRectangle& CameraRectangle)
	{
		ensure(CameraRectangle.bIsInitialized);
		FAxisAlignedBox2d Result;
		
		double Offset = CameraRectangle.SelectionDomain.Plane.DistanceTo(FVector::ZeroVector);
		FCameraRectangle::FRectangleInPlane Domain = CameraRectangle.ProjectSelectionDomain(Offset);
		
		// This works because we know the UV axes are aligned with the XY axes, see the comment in UUVEditorMode::InitializeTargets
		const FVector MinPoint3D = CameraRectangle.PointUVToPoint3D(Domain.Plane, Domain.Rectangle.Min);
		const FVector MaxPoint3D = CameraRectangle.PointUVToPoint3D(Domain.Plane, Domain.Rectangle.Max);
		Result.Contain(FVector2d{MinPoint3D.X, MinPoint3D.Y}); // Convert to 2D and convert to double
		Result.Contain(FVector2d{MaxPoint3D.X, MaxPoint3D.Y});
	
		return Result;
	}
	
	FVector2d XY(const FVector3d& Point)
	{
		return {Point.X, Point.Y};
	}

	void AppendVertexIDs(const FDynamicMesh3& Mesh, int TriangleID, TArray<int>& VertexIDs)
	{
		const FIndex3i& Triangle = Mesh.GetTriangleRef(TriangleID);
		VertexIDs.Add(Triangle.A);
		VertexIDs.Add(Triangle.B);
		VertexIDs.Add(Triangle.C);
	}

	void AppendVertexIDsIfIntersectedRectangle(const FDynamicMesh3& MeshXY0, const FAxisAlignedBox2d& RectangleXY, int TriangleID, TArray<int>& VertexIDs)
	{
		const FIndex3i& Triangle = MeshXY0.GetTriangleRef(TriangleID);
		for (int32 Index = 0; Index < 3; ++Index)
		{
			if (RectangleXY.Contains(XY(MeshXY0.GetVertex(Triangle[Index]))))
			{
				VertexIDs.Add(Triangle[Index]);
			}
		}
	}

	void AppendVertexIDsIfIntersectedFrustum(const FDynamicMesh3& Mesh, const FConvexVolume& Frustum, int TriangleID, TArray<int>& VertexIDs)
	{
		const FIndex3i& Triangle = Mesh.GetTriangleRef(TriangleID);
		for (int32 Index = 0; Index < 3; ++Index)
		{
			if (Frustum.IntersectPoint(Mesh.GetVertex(Triangle[Index])))
			{
				VertexIDs.Add(Triangle[Index]);
			}
		}
	}
	
	void AppendEdgeIDs(const FDynamicMesh3& Mesh, int TriangleID, TArray<int>& EdgeIDs)
	{
		const FIndex3i& Edges = Mesh.GetTriEdgesRef(TriangleID);
		EdgeIDs.Add(Edges.A);
		EdgeIDs.Add(Edges.B);
		EdgeIDs.Add(Edges.C);
	}
	
	void AppendEdgeIDsIfIntersectedRectangle(const FDynamicMesh3& MeshXY0, const FAxisAlignedBox2d& RectangleXY, int TriangleID, TArray<int>& EdgeIDs)
	{
		const FIndex3i& Edges = MeshXY0.GetTriEdgesRef(TriangleID);
		for (int32 Index = 0; Index < 3; ++Index)
		{
			const FIndex2i& EdgeVerts = MeshXY0.GetEdgeRef(Edges[Index]).Vert;
			const FSegment2d Segment(XY(MeshXY0.GetVertex(EdgeVerts.A)), XY(MeshXY0.GetVertex(EdgeVerts.B)));
			if (TestIntersection(Segment, RectangleXY))
			{
				EdgeIDs.Add(Edges[Index]);
			}
		}
	}

	void AppendEdgeIDsIfIntersectedFrustum(const FDynamicMesh3& Mesh, const FConvexVolume& Frustum, int TriangleID, TArray<int>& EdgeIDs)
	{
		const FIndex3i& Edges = Mesh.GetTriEdgesRef(TriangleID);
		for (int32 Index = 0; Index < 3; ++Index)
		{
			const FIndex2i& EdgeVerts = Mesh.GetEdgeRef(Edges[Index]).Vert;
			if (Frustum.IntersectLineSegment(Mesh.GetVertex(EdgeVerts.A), Mesh.GetVertex(EdgeVerts.B)))
			{
				EdgeIDs.Add(Edges[Index]);
			}
		}
	}
	
	void AppendTriangleID(const FDynamicMesh3&, int TriangleID, TArray<int>& TriangleIDs)
	{
		TriangleIDs.Add(TriangleID);
	}

	void AppendTriangleIDIfIntersectedRectangle(const FDynamicMesh3& MeshXY0, const FAxisAlignedBox2d& RectangleXY, int TriangleID, TArray<int>& TriangleIDs)
	{
		const FIndex3i& Triangle = MeshXY0.GetTriangleRef(TriangleID);
		const FTriangle2d TriangleXY(XY(MeshXY0.GetVertex(Triangle.A)),
									 XY(MeshXY0.GetVertex(Triangle.B)),
									 XY(MeshXY0.GetVertex(Triangle.C)));
		
		// Check with bTriangleIsOriented = false since some triangles maybe oriented away from the camera
		if (FIntrTriangle2AxisAlignedBox2d Intersects(TriangleXY, RectangleXY, false); Intersects.Test())
		{
			TriangleIDs.Add(TriangleID);
		}
	}

	void AppendTriangleIDIfIntersectedFrustum(const FDynamicMesh3& Mesh, const FConvexVolume& Frustum, int TriangleID, TArray<int>& TriangleIDs)
	{
		FVector3d V1, V2, V3;
		bool bFullyContained;
		Mesh.GetTriVertices(TriangleID, V1, V2, V3);

		if (Frustum.IntersectTriangle((FVector)V1, (FVector)V2, (FVector)V3, bFullyContained))
		{
			TriangleIDs.Add(TriangleID);
		}
	}

	/*
	 * The tree processing in FindAllIntersections works for both 2d rectangles and 3d frustums as long as we
	 * provide the proper final intersection test. This template is a helper to allow us to use the proper
	 * intersection test for either type.
	*/ 
	template<typename RegionImpl>
	struct TTreeBoxIntersectHelper;

	template<>
	struct TTreeBoxIntersectHelper<FAxisAlignedBox2d>
	{
		typedef FVector2d PointType;
		typedef FAxisAlignedBox2d BoxType;

		static BoxType BuildBoxFromTree(const FDynamicMeshAABBTree3& Tree)
		{
			BoxType Box;
			Box.Contain(XY(Tree.GetBoundingBox().Min));
			Box.Contain(XY(Tree.GetBoundingBox().Max));
			return Box;
		}

		static BoxType BuildBoxFromBox3d(const FAxisAlignedBox3d& Box3d)
		{
			return BoxType(XY(Box3d.Min), XY(Box3d.Max));
		}

		static bool RegionBoxTest(const FAxisAlignedBox2d& Region, const FAxisAlignedBox2d& Box, bool& bFullyContained)
		{
			bFullyContained = false;
			if (Region.Intersects(Box))
			{
				if (Region.Contains(Box))
				{
					bFullyContained = true;
				}
				return true;
			}
			return false;
		}
	};

	template<>
	struct TTreeBoxIntersectHelper<FConvexVolume>
	{
		typedef FVector3d PointType;
		typedef FAxisAlignedBox3d BoxType;

		static BoxType BuildBoxFromTree(const FDynamicMeshAABBTree3& Tree)
		{
			BoxType Box;
			Box.Contain(Tree.GetBoundingBox().Min);
			Box.Contain(Tree.GetBoundingBox().Max);
			return Box;
		}

		static BoxType BuildBoxFromBox3d(const FAxisAlignedBox3d& Box3d)
		{
			return Box3d;
		}

		static bool RegionBoxTest(const FConvexVolume& Region, const FAxisAlignedBox3d& Box, bool& bFullyContained)
		{
			return Region.IntersectBox(Box.Center(), Box.Extents(), bFullyContained);
		}
	};

	struct FCircleRegion
	{
		FVector2d Center;
		double RadiusSquared = 0;
	};
	template<>
	struct TTreeBoxIntersectHelper<FCircleRegion>
	{
		typedef FVector2d PointType;
		typedef FAxisAlignedBox2d BoxType;

		static BoxType BuildBoxFromTree(const FDynamicMeshAABBTree3& Tree)
		{
			BoxType Box;
			Box.Contain(XY(Tree.GetBoundingBox().Min));
			Box.Contain(XY(Tree.GetBoundingBox().Max));
			return Box;
		}

		static BoxType BuildBoxFromBox3d(const FAxisAlignedBox3d& Box3d)
		{
			return BoxType(XY(Box3d.Min), XY(Box3d.Max));
		}

		static bool RegionBoxTest(const FCircleRegion& Region, const FAxisAlignedBox2d& Box, bool& bFullyContained)
		{
			// Check for full containment first.
			bool bIntersect = false;
			bFullyContained = true;
			for (int CornerIndex = 0; CornerIndex < 4; ++CornerIndex)
			{
				if (FVector2d::DistSquared(Box.GetCorner(CornerIndex), Region.Center) <= Region.RadiusSquared)
				{
					bIntersect = true;
				}
				else
				{
					bFullyContained = false;
					// No need to test others because our intersection test below covers it.
					break;
				}
			}

			if (bIntersect)
			{
				return true;
			}

			// Now check for intersection. This is done the same way as FMath::SphereAABBIntersection, which
			//  just checks our squared distance out from each box side, resulting in a sum if its past a corner.
			double DistSquared = 0;
			if (Region.Center.X < Box.Min.X)
			{
				DistSquared += FMath::Square(Region.Center.X - Box.Min.X);
			}
			else if (Region.Center.X > Box.Max.X)
			{
				DistSquared += FMath::Square(Region.Center.X - Box.Max.X);
			}
			if (Region.Center.Y < Box.Min.Y)
			{
				DistSquared += FMath::Square(Region.Center.Y - Box.Min.Y);
			}
			else if (Region.Center.Y > Box.Max.Y)
			{
				DistSquared += FMath::Square(Region.Center.Y - Box.Max.Y);
			}

			return DistSquared <= Region.RadiusSquared;
		}
	};

	void AppendTriangleIDIfIntersectedCircle(const FDynamicMesh3& Mesh, const FCircleRegion& Circle, int TriangleID, TArray<int>& TriangleIDs)
	{
		FVector3d V1, V2, V3;
		Mesh.GetTriVertices(TriangleID, V1, V2, V3);

		if (UE::Geometry::DoesTriangleIntersectCircle2D(FVector2d(V1), FVector2d(V2), FVector2d(V3), Circle.Center, Circle.RadiusSquared))
		{
			TriangleIDs.Add(TriangleID);
		}
	}


	// Returns indices, collected by the given functions, from triangles which are intersected by the given region,
	// which can be either a rectangle (in which case the underlying mesh should have its vertices in the XY plane
	// with a zero Z coordinate) or a frustum. 
	template<typename IDsFromTriangleF, typename IDsFromTriangleIfIntersectedF, typename RegionType>
	TArray<int32> FindAllIntersections(const FDynamicMeshAABBTree3& Tree,
									   const RegionType& Region,
									   IDsFromTriangleF AppendIDs,
									   IDsFromTriangleIfIntersectedF AppendIDsIfIntersected)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindAllIntersectionsAxisAlignedBox2);
		typedef TTreeBoxIntersectHelper<RegionType> RegionPolicy;

		check(Tree.GetMesh());
		
		TArray<int32> Result;
		typename RegionPolicy::BoxType TreeBox = RegionPolicy::BuildBoxFromTree(Tree);

		bool bFullyContained;
		bool bIntersects = RegionPolicy::RegionBoxTest(Region, TreeBox, bFullyContained);
		if (bFullyContained)
		{
			// Early out selecting everything
			Result.Reserve(Tree.GetMesh()->TriangleCount());
			for (int TriangleID : Tree.GetMesh()->TriangleIndicesItr())
			{
				AppendIDs(*Tree.GetMesh(), TriangleID, Result);
			}
			return Result;
		}
		
		int SelectAllDepth = TNumericLimits<int>::Max();
		int CurrentDepth = -1;
		
		// Traversal is depth first
		FDynamicMeshAABBTree3::FTreeTraversal Traversal;
		
		Traversal.NextBoxF =
			[&Region, &SelectAllDepth, &CurrentDepth](const FAxisAlignedBox3d& Box3d, int Depth)
		{
			CurrentDepth = Depth;
			if (Depth > SelectAllDepth)
			{
				// We are deeper than the depth whose AABB was first detected to be contained in the RectangleXY,
				// descend and collect all leaf triangles
				return true;
			}
			
			SelectAllDepth = TNumericLimits<int>::Max();
			typename RegionPolicy::BoxType RegionBox = RegionPolicy::BuildBoxFromBox3d(Box3d);

			bool bFullyContained;
			bool bIntersects = RegionPolicy::RegionBoxTest(Region, RegionBox, bFullyContained);

			if(bFullyContained)
			{
				SelectAllDepth = Depth;
			}
				
			return bIntersects;
		};
		
		Traversal.NextTriangleF =
			[&Region, &SelectAllDepth, &CurrentDepth, &Tree, &Result, &AppendIDs, &AppendIDsIfIntersected]
			(int TriangleID)
		{
			if (CurrentDepth >= SelectAllDepth)
			{
				// This TriangleID is entirely contained in the selection rectangle so we can skip intersection testing
				return AppendIDs(*Tree.GetMesh(), TriangleID, Result);
			}
			return AppendIDsIfIntersected(*Tree.GetMesh(), Region, TriangleID, Result);
		};
		
		Tree.DoTraversal(Traversal);

		return Result;
	}


	bool ConvertToHitElementList(ESelectionMode SelectionMode,
		const FDynamicMesh3& Mesh, const FDynamicMesh3& UnwrapMesh, FDynamicMeshUVOverlay& UVOverlay, int32 HitTid, const FViewCameraState& CameraState,
		const FRay& Ray, TArray<int32>& IDsOut)
	{
		if (!ensure(HitTid != IndexConstants::InvalidID && Mesh.IsTriangle(HitTid)))
		{
			return false;
		}

		IDsOut.Reset();

		switch (SelectionMode)
		{
		case ESelectionMode::Island:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Component);
			FMeshConnectedComponents MeshSelectedComponent(&Mesh);
			TArray<int32> SeedTriangles;
			SeedTriangles.Add(HitTid);			
			auto UVIslandPredicate = [&UnwrapMesh,&UVOverlay](int32 Triangle0, int32 Triangle1)
			{
				if (UnwrapMesh.IsTriangle(Triangle0))
				{
					return UnwrapMesh.GetTriNeighbourTris(Triangle0).Contains(Triangle1);
				}
				else { // Triangle0 is unset, return true if Triangle1 is also.
					return !UnwrapMesh.IsTriangle(Triangle1);
				}
			};
			MeshSelectedComponent.FindTrianglesConnectedToSeeds(SeedTriangles, UVIslandPredicate);
			ensure(MeshSelectedComponent.Components.Num() == 1); // Expect each triangle to only be in a single component
			IDsOut.Append(MoveTemp(MeshSelectedComponent.Components[0].Indices));
			break;
		}
		case ESelectionMode::Edge:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Edge);
			// TODO: We'll need the ability to hit occluded triangles to see if there is a better edge to snap to.

			// Try to snap to one of the edges.
			FIndex3i Eids = Mesh.GetTriEdges(HitTid);

			FGeometrySet3 GeometrySet;
			for (int i = 0; i < 3; ++i)
			{
				FIndex2i Vids = Mesh.GetEdgeV(Eids[i]);
				FPolyline3d Polyline(Mesh.GetVertex(Vids.A), Mesh.GetVertex(Vids.B));
				GeometrySet.AddCurve(Eids[i], Polyline);
			}

			FGeometrySet3::FNearest Nearest;
			if (GeometrySet.FindNearestCurveToRay(Ray, Nearest,
				[&CameraState](const FVector3d& Position1, const FVector3d& Position2) {
					return ToolSceneQueriesUtil::PointSnapQuery(CameraState,
						Position1, Position2,
						ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD()); }))
			{
				IDsOut.Add(Nearest.ID);
			}
			break;
		}
		case ESelectionMode::Vertex:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Vertex);
			// TODO: Improve this to handle super narrow, sliver triangles better, where testing near vertices can be difficult.

			// Try to snap to one of the vertices
			FIndex3i Vids = Mesh.GetTriangle(HitTid);

			FGeometrySet3 GeometrySet;
			for (int i = 0; i < 3; ++i)
			{
				GeometrySet.AddPoint(Vids[i], Mesh.GetTriVertex(HitTid, i));
			}

			FGeometrySet3::FNearest Nearest;
			if (GeometrySet.FindNearestPointToRay(Ray, Nearest,
				[&CameraState](const FVector3d& Position1, const FVector3d& Position2) {
					return ToolSceneQueriesUtil::PointSnapQuery(CameraState,
						Position1, Position2,
						ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD()); }))
			{
				IDsOut.Add(Nearest.ID);
			}
			break;
		}
		case ESelectionMode::Triangle:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Triangle);

			IDsOut.Add(HitTid);
			break;
		}
		case ESelectionMode::Mesh:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Mesh);

			for (int32 Tid : Mesh.TriangleIndicesItr())
			{
				IDsOut.Add(Tid);
			}
			break;
		}
		default:
			ensure(false);
			break;
		}

		return !IDsOut.IsEmpty();
	}

	TArray<int32> Convert3DHitsTo2DHits(ESelectionMode SelectionMode,
		const FDynamicMesh3& UnwrapMesh, const FDynamicMesh3& AppliedMesh, const FDynamicMeshUVOverlay& UVOverlay, const TArray<int32>& IDsIn, TArray<int32>& AppliedMeshOnlyIDsOut)
	{
		TArray<int32> IDsOut;
		switch (SelectionMode)
		{
		case ESelectionMode::Triangle:
		case ESelectionMode::Island:
		case ESelectionMode::Mesh:
			IDsOut = FUVToolSelection::ConvertAppliedElementIdsToUnwrappedElementIds(FUVToolSelection::EType::Triangle, AppliedMesh, UnwrapMesh,
				UVOverlay, IDsIn, AppliedMeshOnlyIDsOut);
			break;
		case ESelectionMode::Edge:
			IDsOut = FUVToolSelection::ConvertAppliedElementIdsToUnwrappedElementIds(FUVToolSelection::EType::Edge, AppliedMesh, UnwrapMesh,
		UVOverlay, IDsIn, AppliedMeshOnlyIDsOut);
			break;
		case ESelectionMode::Vertex:
			IDsOut = FUVToolSelection::ConvertAppliedElementIdsToUnwrappedElementIds(FUVToolSelection::EType::Vertex, AppliedMesh, UnwrapMesh,
		UVOverlay, IDsIn, AppliedMeshOnlyIDsOut);
			break;
		default:
			ensure(false);
			break;
		}
		return IDsOut;
	}
	
	TArray<int32> Convert2DHitsTo3DHits(ESelectionMode SelectionMode,
		const FDynamicMesh3& UnwrapMesh, const FDynamicMesh3& AppliedMesh, const FDynamicMeshUVOverlay& UVOverlay, TArray<int32>& IDsIn)
	{
		TArray<int32> IDsOut;
		switch (SelectionMode)
		{
		case ESelectionMode::Triangle:
		case ESelectionMode::Island:
		case ESelectionMode::Mesh:
			IDsOut = FUVToolSelection::ConvertUnwrappedElementIdsToAppliedElementIds(FUVToolSelection::EType::Triangle, UnwrapMesh, AppliedMesh,
				UVOverlay, IDsIn);
			break;
		case ESelectionMode::Edge:
			IDsOut = FUVToolSelection::ConvertUnwrappedElementIdsToAppliedElementIds(FUVToolSelection::EType::Edge, UnwrapMesh, AppliedMesh,
				UVOverlay, IDsIn);
			break;
		case ESelectionMode::Vertex:
			IDsOut = FUVToolSelection::ConvertUnwrappedElementIdsToAppliedElementIds(FUVToolSelection::EType::Vertex, UnwrapMesh, AppliedMesh,
				UVOverlay, IDsIn);
			break;
		default:
			ensure(false);
			break;
		}
		return IDsOut;
	}


	/**
	 * Undo/redo transaction for selection mode changes
	 */
	class  FModeChange : public FToolCommandChange
	{
	public:
		FModeChange(ESelectionMode BeforeIn, 
			ESelectionMode AfterIn)
			: Before(BeforeIn)
			, After(AfterIn)
		{};

		virtual void Apply(UObject* Object) override
		{
			UUVEditorMeshSelectionMechanic* SelectionMechanic = Cast<UUVEditorMeshSelectionMechanic>(Object);
			FModeChangeOptions Options;
			Options.bConvertExisting = false;
			Options.bBroadcastIfConverted = false;
			Options.bEmitChanges = false;
			SelectionMechanic->SetSelectionMode(After, Options);
		}

		virtual void Revert(UObject* Object) override
		{
			UUVEditorMeshSelectionMechanic* SelectionMechanic = Cast<UUVEditorMeshSelectionMechanic>(Object);
			FModeChangeOptions Options;
			Options.bConvertExisting = false;
			Options.bBroadcastIfConverted = false;
			Options.bEmitChanges = false;
			SelectionMechanic->SetSelectionMode(Before, Options);
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			UUVEditorMeshSelectionMechanic* SelectionMechanic = Cast<UUVEditorMeshSelectionMechanic>(Object);
			return !SelectionMechanic->IsEnabled();
		}


		virtual FString ToString() const override
		{
			return TEXT("UVEditorMeshSelectionMechanicLocals::FModeChange");
		}

	protected:
		ESelectionMode Before;
		ESelectionMode After;
	};
} // namespace UVEditorMeshSelectionMechanicLocals


void UUVEditorMeshSelectionMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);

	UContextObjectStore* ContextStore = GetParentTool()->GetToolManager()->GetContextObjectStore();
	EmitChangeAPI = ContextStore->FindContext<UUVToolEmitChangeAPI>();
	check(EmitChangeAPI);

	UnwrapClickTargetRouter = NewObject<ULocalSingleClickInputBehavior>();

	UnwrapClickTargetRouter->IsHitByClickFunc = [this](const FInputDeviceRay& ClickPos) { return IsHitByClick(ClickPos, false); };
	UnwrapClickTargetRouter->OnClickedFunc = [this](const FInputDeviceRay& ClickPos) { OnClicked(ClickPos, false); };
	UnwrapClickTargetRouter->OnUpdateModifierStateFunc = [this](int ModifierID, bool bIsOn) { OnUpdateModifierState(ModifierID,bIsOn); };

	UnwrapHoverBehaviorTargetRouter = NewObject<ULocalMouseHoverBehavior>();

	UnwrapHoverBehaviorTargetRouter->BeginHitTestFunc = [this](const FInputDeviceRay& PressPos) { return BeginHoverSequenceHitTest(PressPos, false); };
	UnwrapHoverBehaviorTargetRouter->OnBeginHoverFunc = [this](const FInputDeviceRay& DevicePos) { return OnBeginHover(DevicePos); };
	UnwrapHoverBehaviorTargetRouter->OnUpdateHoverFunc = [this](const FInputDeviceRay& DevicePos) {return OnUpdateHover(DevicePos, false); };
	UnwrapHoverBehaviorTargetRouter->OnEndHoverFunc = [this]() {return OnEndHover(); };

	LivePreviewClickTargetRouter = NewObject<ULocalSingleClickInputBehavior>();

	LivePreviewClickTargetRouter->IsHitByClickFunc = [this](const FInputDeviceRay& ClickPos) { return IsHitByClick(ClickPos, true); };
	LivePreviewClickTargetRouter->OnClickedFunc = [this](const FInputDeviceRay& ClickPos) { OnClicked(ClickPos, true); };
	LivePreviewClickTargetRouter->OnUpdateModifierStateFunc = [this](int ModifierID, bool bIsOn) { OnUpdateModifierState(ModifierID, bIsOn); };

	LivePreviewHoverBehaviorTargetRouter = NewObject<ULocalMouseHoverBehavior>();

	LivePreviewHoverBehaviorTargetRouter->BeginHitTestFunc = [this](const FInputDeviceRay& PressPos) {
		return BeginHoverSequenceHitTest(PressPos, true);
	};
	LivePreviewHoverBehaviorTargetRouter->OnBeginHoverFunc = [this](const FInputDeviceRay& DevicePos) { return OnBeginHover(DevicePos); };
	LivePreviewHoverBehaviorTargetRouter->OnUpdateHoverFunc = [this](const FInputDeviceRay& DevicePos) {return OnUpdateHover(DevicePos, true); };
	LivePreviewHoverBehaviorTargetRouter->OnEndHoverFunc = [this]() {return OnEndHover(); };

	// This will be the target for the click drag behavior below
	MarqueeMechanic = NewObject<URectangleMarqueeMechanic>();
	MarqueeMechanic->bUseExternalClickDragBehavior = true;
	MarqueeMechanic->Setup(ParentToolIn);
	MarqueeMechanic->OnDragRectangleStarted.AddUObject(this, &UUVEditorMeshSelectionMechanic::OnDragRectangleStarted);
	// TODO(Performance) :DynamicMarqueeSelection It would be cool to have the marquee selection update dynamically as
	//  the rectangle gets changed, right now this isn't interactive for large meshes so we disabled it
	//MarqueeMechanic->OnDragRectangleChanged.AddUObject(this, &UUVEditorMeshSelectionMechanic::OnDragRectangleChanged);
	MarqueeMechanic->OnDragRectangleFinished.AddLambda( [this](const FCameraRectangle& Rectangle, bool bCancelled)
		{
				OnDragRectangleFinished(Rectangle, bCancelled, false);
		});

	USingleClickOrDragInputBehavior* ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(UnwrapClickTargetRouter, MarqueeMechanic);
	ClickOrDragBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	ClickOrDragBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	ParentTool->AddInputBehavior(ClickOrDragBehavior);

	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(UnwrapHoverBehaviorTargetRouter);
	ParentTool->AddInputBehavior(HoverBehavior);

	ViewportButtonsAPI = ContextStore->FindContext<UUVToolViewportButtonsAPI>();
	check(ViewportButtonsAPI);
	ViewportButtonsAPI->OnSelectionModeChange.AddWeakLambda(this,
		[this](UUVToolSelectionAPI::EUVEditorSelectionMode NewMode) {
			SetSelectionMode(NewMode);
		});

	// Make sure we match the activated button
	SelectionMode = ViewportButtonsAPI->GetSelectionMode();
	SetIsEnabled(bIsEnabled);

	LivePreviewAPI = ContextStore->FindContext<UUVToolLivePreviewAPI>();
	check(LivePreviewAPI);

	// Set things up for being able to add behaviors to live preview.
	LivePreviewBehaviorSet = NewObject<UInputBehaviorSet>();
	LivePreviewBehaviorSource = NewObject<ULocalInputBehaviorSource>();
	LivePreviewBehaviorSource->GetInputBehaviorsFunc = [this]() { return LivePreviewBehaviorSet; };

	// This will be the target for the click drag behavior below
	LivePreviewMarqueeMechanic = NewObject<URectangleMarqueeMechanic>();
	LivePreviewMarqueeMechanic->bUseExternalClickDragBehavior = true;
	LivePreviewMarqueeMechanic->bUseExternalUpdateCameraState = true;
	LivePreviewMarqueeMechanic->UpdateCameraStateFunc = [this]() {
		FViewCameraState LivePreviewCameraState;
		LivePreviewAPI->GetLivePreviewCameraState(LivePreviewCameraState);
		return LivePreviewCameraState;
	};
	LivePreviewMarqueeMechanic->Setup(ParentToolIn);
	LivePreviewMarqueeMechanic->OnDragRectangleStarted.AddUObject(this, &UUVEditorMeshSelectionMechanic::OnDragRectangleStarted);
	// TODO(Performance) :DynamicMarqueeSelection It would be cool to have the marquee selection update dynamically as
	//  the rectangle gets changed, right now this isn't interactive for large meshes so we disabled it
	//LivePreviewMarqueeMechanic->OnDragRectangleChanged.AddUObject(this, &UUVEditorMeshSelectionMechanic::OnDragRectangleChanged);
	LivePreviewMarqueeMechanic->OnDragRectangleFinished.AddLambda([this](const FCameraRectangle& Rectangle, bool bCancelled)
		{
			OnDragRectangleFinished(Rectangle, bCancelled, true);
		});

	// Set up click and hover behaviors both for the 2d (unwrap) viewport and the 3d 
	// (applied/live preview) viewport
	USingleClickOrDragInputBehavior* LivePreviewClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	LivePreviewClickOrDragBehavior->Initialize(LivePreviewClickTargetRouter, LivePreviewMarqueeMechanic);
	LivePreviewClickOrDragBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	LivePreviewClickOrDragBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	LivePreviewBehaviorSet->Add(LivePreviewClickOrDragBehavior, this);
	
	UMouseHoverBehavior* LivePreviewHoverBehavior = NewObject<UMouseHoverBehavior>();
	LivePreviewHoverBehavior->Initialize(LivePreviewHoverBehaviorTargetRouter);
	LivePreviewBehaviorSet->Add(LivePreviewHoverBehavior, this);

	// Give the live preview behaviors to the live preview input router
	if (LivePreviewAPI)
	{
		LivePreviewInputRouter = LivePreviewAPI->GetLivePreviewInputRouter();
		LivePreviewInputRouter->RegisterSource(LivePreviewBehaviorSource);
	}

}

void UUVEditorMeshSelectionMechanic::Initialize(UWorld* World, UWorld* LivePreviewWorld, UUVToolSelectionAPI* SelectionAPIIn)
{
	// It may be unreasonable to worry about Initialize being called more than once, but let's be safe anyway
	if (HoverGeometryActor)
	{
		HoverGeometryActor->Destroy();
	}

	SelectionAPI = SelectionAPIIn;
	
	HoverGeometryActor = World->SpawnActor<APreviewGeometryActor>();

	HoverTriangleSet = NewObject<UTriangleSetComponent>(HoverGeometryActor);
	HoverTriangleSetMaterial = ToolSetupUtil::GetCustomTwoSidedDepthOffsetMaterial(GetParentTool()->GetToolManager(),
		FUVEditorUXSettings::SelectionHoverTriangleFillColor,
		FUVEditorUXSettings::SelectionHoverTriangleDepthBias,
		FUVEditorUXSettings::SelectionHoverTriangleOpacity);
	HoverGeometryActor->SetRootComponent(HoverTriangleSet.Get());
	HoverTriangleSet->RegisterComponent();

	HoverPointSet = NewObject<UPointSetComponent>(HoverGeometryActor);
	HoverPointSet->SetPointMaterial(ToolSetupUtil::GetDefaultPointComponentMaterial(GetParentTool()->GetToolManager(), false));
	HoverPointSet->AttachToComponent(HoverTriangleSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	HoverPointSet->RegisterComponent();
	
	HoverLineSet = NewObject<ULineSetComponent>(HoverGeometryActor);
	HoverLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetParentTool()->GetToolManager(), false));
	HoverLineSet->AttachToComponent(HoverTriangleSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	HoverLineSet->RegisterComponent();

	LivePreviewHoverGeometryActor = LivePreviewWorld->SpawnActor<APreviewGeometryActor>();

	LivePreviewHoverTriangleSet = NewObject<UTriangleSetComponent>(LivePreviewHoverGeometryActor);
	LivePreviewHoverGeometryActor->SetRootComponent(LivePreviewHoverTriangleSet.Get());
	LivePreviewHoverTriangleSet->RegisterComponent();

	LivePreviewHoverPointSet = NewObject<UPointSetComponent>(LivePreviewHoverGeometryActor);
	LivePreviewHoverPointSet->SetPointMaterial(ToolSetupUtil::GetDefaultPointComponentMaterial(GetParentTool()->GetToolManager(), false));
	LivePreviewHoverPointSet->AttachToComponent(LivePreviewHoverTriangleSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	LivePreviewHoverPointSet->RegisterComponent();

	LivePreviewHoverLineSet = NewObject<ULineSetComponent>(LivePreviewHoverGeometryActor);
	LivePreviewHoverLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetParentTool()->GetToolManager(), false));
	LivePreviewHoverLineSet->AttachToComponent(LivePreviewHoverTriangleSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	LivePreviewHoverLineSet->RegisterComponent();
}

void UUVEditorMeshSelectionMechanic::SetIsEnabled(bool bIsEnabledIn)
{
	bIsEnabled = bIsEnabledIn;
	if (MarqueeMechanic)
	{
		MarqueeMechanic->SetIsEnabled(bIsEnabled && SelectionMode != ESelectionMode::None);
	}
	if (LivePreviewMarqueeMechanic)
	{
		bool bLiveViewportMarqueeActive = bIsEnabled;
		bLiveViewportMarqueeActive &= SelectionMode != ESelectionMode::None;		
		LivePreviewMarqueeMechanic->SetIsEnabled(bLiveViewportMarqueeActive);
	}
	if (ViewportButtonsAPI)
	{
		ViewportButtonsAPI->SetSelectionButtonsEnabled(bIsEnabledIn);
	}
}

TArray<UE::Geometry::FUVToolSelection> UUVEditorMeshSelectionMechanic::GetAllCanonicalTrianglesInUnwrapRadius(
	const FVector2d& UnwrapWorldHitPoint, double Radius) const
{
	using namespace UVEditorMeshSelectionMechanicLocals;

	TArray<UE::Geometry::FUVToolSelection> Output;

	FCircleRegion CircleRegion;
	CircleRegion.Center = UnwrapWorldHitPoint;
	CircleRegion.RadiusSquared = FMath::Square(FMath::Max(0, Radius));

	for (int32 AssetID = 0; AssetID < Targets.Num(); ++AssetID)
	{
		const FDynamicMeshAABBTree3* Tree = GetMeshSpatial(AssetID, true).Get();
		if (!ensure(Tree))
		{
			continue;
		}

		TArray<int32> TidsInRadius = FindAllIntersections(*Tree, CircleRegion, AppendTriangleID, AppendTriangleIDIfIntersectedCircle);
		if (TidsInRadius.Num() > 0)
		{
			FUVToolSelection& Selection = Output.Emplace_GetRef();
			Selection.Target = Targets[AssetID];
			Selection.Type = FUVToolSelection::EType::Triangle;
			Selection.SelectedIDs.Append(TidsInRadius);
		}
	}
	return Output;
}

void UUVEditorMeshSelectionMechanic::SetShowHoveredElements(bool bShow)
{
	bShowHoveredElements = bShow;
	if (!bShowHoveredElements)
	{
		HoverPointSet->Clear();
		HoverLineSet->Clear();
		HoverTriangleSet->Clear();

		LivePreviewHoverPointSet->Clear();
		LivePreviewHoverLineSet->Clear();
		LivePreviewHoverTriangleSet->Clear();
	}
}

void UUVEditorMeshSelectionMechanic::Shutdown()
{
	// Note that the ForceTerminateSource should happen before destroying everything else so that we
	// can shut down behaviors properly.
	if (LivePreviewInputRouter.IsValid())
	{
		LivePreviewInputRouter->DeregisterSource(LivePreviewBehaviorSource);
	}

	if (HoverGeometryActor)
	{
		HoverGeometryActor->Destroy();
		HoverGeometryActor = nullptr;
	}
	if (LivePreviewHoverGeometryActor)
	{
		LivePreviewHoverGeometryActor->Destroy();
		LivePreviewHoverGeometryActor = nullptr;
	}

	SelectionAPI = nullptr;
	ViewportButtonsAPI = nullptr;
	LivePreviewAPI = nullptr;
	EmitChangeAPI = nullptr;
	MarqueeMechanic = nullptr;
	HoverTriangleSetMaterial = nullptr;
	LivePreviewMarqueeMechanic = nullptr;
}

void UUVEditorMeshSelectionMechanic::SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
{
	Targets = TargetsIn;

	// Retrieve cached AABB tree storage, or else set it up
	UContextObjectStore* ContextStore = ParentTool->GetToolManager()->GetContextObjectStore();
	UUVToolAABBTreeStorage* TreeStore = ContextStore->FindContext<UUVToolAABBTreeStorage>();
	if (!TreeStore)
	{
		TreeStore = NewObject<UUVToolAABBTreeStorage>();
		ContextStore->AddContextObject(TreeStore);
	}

	// Get or create spatials
	// Initialize the AABB trees from cached values, or make new ones
	UnwrapMeshSpatials.Reset();
	AppliedMeshSpatials.Reset();
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		TSharedPtr<FDynamicMeshAABBTree3> UnwrapTree = TreeStore->Get(Target->UnwrapCanonical.Get());
		if (!UnwrapTree)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildAABBTreeForTarget);
			UnwrapTree = MakeShared<FDynamicMeshAABBTree3>();
			UnwrapTree->SetMesh(Target->UnwrapCanonical.Get(), false);
			// For now we split round-robin on the X/Y axes TODO Experiment with better splitting heuristics
			FDynamicMeshAABBTree3::GetSplitAxisFunc GetSplitAxis = [](int Depth, const FAxisAlignedBox3d&) { return Depth % 2; };
			// Note: 16 tris/leaf was chosen with data collected by SpatialBenchmarks.cpp in GeometryProcessingUnitTests
			UnwrapTree->SetBuildOptions(16, MoveTemp(GetSplitAxis));
			UnwrapTree->Build();
			TreeStore->Set(Target->UnwrapCanonical.Get(), UnwrapTree, Target);
		}
		UnwrapMeshSpatials.Add(UnwrapTree);
		TSharedPtr<FDynamicMeshAABBTree3> AppliedTree = TreeStore->Get(Target->AppliedCanonical.Get());
		if (!AppliedTree)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildAABBTreeForTarget);
			AppliedTree = MakeShared<FDynamicMeshAABBTree3>();
			AppliedTree->SetMesh(Target->AppliedCanonical.Get(), false);
			AppliedTree->Build();
			TreeStore->Set(Target->AppliedCanonical.Get(), AppliedTree, Target);
		}
		AppliedMeshSpatials.Add(AppliedTree);
	}

}

TSharedPtr<FDynamicMeshAABBTree3> UUVEditorMeshSelectionMechanic::GetMeshSpatial(int32 TargetId, bool bUseUnwrap) const
{
	if (bUseUnwrap)
	{
		return UnwrapMeshSpatials[TargetId];
	}
	else
	{
		return AppliedMeshSpatials[TargetId];
	}
}

void UUVEditorMeshSelectionMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	MarqueeMechanic->Render(RenderAPI);

	// Cache the camera state
	GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
}

void UUVEditorMeshSelectionMechanic::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	MarqueeMechanic->DrawHUD(Canvas, RenderAPI);
}

void UUVEditorMeshSelectionMechanic::LivePreviewRender(IToolsContextRenderAPI* RenderAPI)
{
	LivePreviewMarqueeMechanic->Render(RenderAPI);
}

void UUVEditorMeshSelectionMechanic::LivePreviewDrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	LivePreviewMarqueeMechanic->DrawHUD(Canvas, RenderAPI);
}

FInputRayHit UUVEditorMeshSelectionMechanic::IsHitByClick(const FInputDeviceRay& ClickPos, bool bSourceIsLivePreview)
{
	FInputRayHit Hit;

	// If enabled, return a hit so we always capture and can clear the selection
	Hit.bHit = bIsEnabled && SelectionMode != ESelectionMode::None;
	return Hit;	
}


void UUVEditorMeshSelectionMechanic::SetSelectionMode(
	ESelectionMode TargetMode, const FModeChangeOptions& Options)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_ChangeSelectionMode);

	const FText TransactionName = LOCTEXT("ChangeSelectionMode", "Change Selection Mode");
	
	using namespace UVEditorMeshSelectionMechanicLocals;

	ESelectionMode OldMode = SelectionMode;
	if (OldMode == TargetMode)
	{
		return;
	}
	SelectionMode = TargetMode;

	if (SelectionAPI == nullptr)
	{
		return; // From this point on, if we don't have a SelectionAPI assigned yet, we won't do anything further for this.
		        // This might happen during initialization.
	}

	if (ViewportButtonsAPI)
	{
		// Not clear whether we should or shouldn't broadcast this. A user could conceivably set selection
		// via mechanic and expect for a notification from the viewport buttons, but it feels wrong to
		// knowingly trigger a second call into this function if we broadcast, and that example seems like
		// questionable code organization...
		ViewportButtonsAPI->SetSelectionMode(SelectionMode, false);
	}

	if (Options.bEmitChanges)
	{
		EmitChangeAPI->BeginUndoTransaction(TransactionName);
		EmitChangeAPI->EmitToolIndependentChange(this, MakeUnique<FModeChange>(OldMode, SelectionMode), TransactionName);
	}

	MarqueeMechanic->SetIsEnabled(bIsEnabled && SelectionMode != ESelectionMode::None);

	// See whether a conversion is not necessary
	FUVToolSelection::EType ExpectedSelectionType = ToCompatibleDynamicMeshSelectionType(SelectionMode);
	FUVToolSelection::EType CurrentSelectionType = SelectionAPI->GetSelectionsType();
	if ((!SelectionAPI->HaveSelections() && !SelectionAPI->HaveUnsetElementAppliedMeshSelections())
		|| ExpectedSelectionType == CurrentSelectionType
		|| !Options.bConvertExisting || SelectionMode == ESelectionMode::None)
	{
		// No conversion needed
		if (Options.bEmitChanges)
		{
			EmitChangeAPI->EndUndoTransaction();
		}
		return;
	}
	
	// We're going to convert the existing selection.
	const TArray<FUVToolSelection>& OriginalSelections = SelectionAPI->GetSelections();
	const TArray<FUVToolSelection>& OriginalUnsetSelections = SelectionAPI->GetUnsetElementAppliedMeshSelections();

	TArray<FUVToolSelection> NewSelections, NewUnsetSelections;
	NewSelections.Reserve(OriginalSelections.Num());
	NewUnsetSelections.Reserve(OriginalUnsetSelections.Num());
	for (const FUVToolSelection& OriginalSelection : OriginalSelections)
	{
		NewSelections.Add(OriginalSelection.GetConvertedSelection(*OriginalSelection.Target->UnwrapCanonical, ExpectedSelectionType));
	}
	for (const FUVToolSelection& OriginalUnsetSelection : OriginalUnsetSelections)
	{
		NewUnsetSelections.Add(OriginalUnsetSelection.GetConvertedSelection(*OriginalUnsetSelection.Target->AppliedCanonical, ExpectedSelectionType));
	}

	// Remove any selections that end up empty after conversion
	NewSelections.RemoveAll([](const FUVToolSelection& Selection) {
		return Selection.SelectedIDs.IsEmpty() && 
			!(Selection.Type == FUVToolSelection::EType::Edge && Selection.HasStableEdgeIdentifiers());
		});

	NewUnsetSelections.RemoveAll([](const FUVToolSelection& Selection) {
		return Selection.SelectedIDs.IsEmpty() &&
			!(Selection.Type == FUVToolSelection::EType::Edge && Selection.HasStableEdgeIdentifiers());
		});

	// Apply selection change
	SelectionAPI->BeginChange();
	SelectionAPI->ClearSelections(false, false);
	SelectionAPI->ClearUnsetElementAppliedMeshSelections(false, false);
	SelectionAPI->SetSelections(NewSelections, false, false);
	SelectionAPI->SetUnsetElementAppliedMeshSelections(NewUnsetSelections, false, false);
	SelectionAPI->EndChangeAndEmitIfModified(Options.bBroadcastIfConverted);

	if (Options.bEmitChanges)
	{
		EmitChangeAPI->EndUndoTransaction();
	}
	
	return;
}

void UUVEditorMeshSelectionMechanic::ModifyExistingSelection(TSet<int32>& SelectionSetToModify, 
	const TArray<int32>& SelectedIDs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_UpdateCurrentSelection);
	
	using namespace UVEditorMeshSelectionMechanicLocals;

	if (ShouldAddToSelection())
	{
		SelectionSetToModify.Append(SelectedIDs);
	}
	else if (ShouldToggleFromSelection())
	{
		for (int32 ID : SelectedIDs)
		{
			ToggleItem(SelectionSetToModify, ID);
		}
	}
	else if (ShouldRemoveFromSelection())
	{
		SelectionSetToModify = SelectionSetToModify.Difference(TSet<int32>(SelectedIDs));
	}
	else
	{
		// We shouldn't be trying to modify an existing selection if we're supposed to restart
		ensure(false);
	}
}

void UUVEditorMeshSelectionMechanic::OnClicked(const FInputDeviceRay& ClickPos, bool bSourceIsLivePreview)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_OnClicked);
	
	using namespace UVEditorMeshSelectionMechanicLocals;

	// IsHitByClick should prevent us being here with !bIsEnabled
	if (!ensure(bIsEnabled))
	{
		return;
	}

	FUVToolSelection::EType ElementType = ToCompatibleDynamicMeshSelectionType(SelectionMode);

	int32 HitAssetID = IndexConstants::InvalidID;
	int32 HitTid = IndexConstants::InvalidID;
	int32 ExistingSelectionIndex = IndexConstants::InvalidID;
	TArray<int32> NewIDs, NewUnsetIDs;

	bool bTidIsHit = GetHitTid(ClickPos, HitTid, HitAssetID, !bSourceIsLivePreview, &ExistingSelectionIndex);
	bool bHitConversionSucceeded = false;
	if (bTidIsHit)
	{
		FDynamicMesh3* UnwrapMesh = Targets[HitAssetID]->UnwrapCanonical.Get();
		FDynamicMesh3* AppliedMesh = Targets[HitAssetID]->AppliedCanonical.Get();
		FDynamicMesh3* Mesh = bSourceIsLivePreview ? AppliedMesh : UnwrapMesh;
		FDynamicMeshUVOverlay& UVOverlay = *Mesh->Attributes()->GetUVLayer(Targets[HitAssetID]->UVLayerIndex);
		bHitConversionSucceeded = ConvertToHitElementList(SelectionMode, *Mesh, *UnwrapMesh, UVOverlay, HitTid, CameraState, ClickPos.WorldRay, NewIDs);

		if (bSourceIsLivePreview)
		{
			NewIDs = Convert3DHitsTo2DHits(SelectionMode, *UnwrapMesh, *AppliedMesh, UVOverlay, NewIDs, NewUnsetIDs);
		}
	}
	if(!bTidIsHit || !bHitConversionSucceeded)
	{
		// Failed to select an element. See if selection needs clearing, and exit.
		if (ShouldRestartSelection())
		{
			SelectionAPI->BeginChange();
			if (SelectionAPI->HaveSelections())
			{
				SelectionAPI->ClearSelections(false, false);
			}
			if (SelectionAPI->HaveUnsetElementAppliedMeshSelections())
			{
				SelectionAPI->ClearUnsetElementAppliedMeshSelections(false, false);
			}
			SelectionAPI->EndChangeAndEmitIfModified(true); // broadcast and emit
		}
		return;
	}

	TArray<FUVToolSelection> NewSelections;
	TArray<FUVToolSelection> NewUnsetSelections;
	if (!ShouldRestartSelection())
	{
		NewSelections = SelectionAPI->GetSelections();
		NewUnsetSelections = SelectionAPI->GetUnsetElementAppliedMeshSelections();
	}

	auto ProcessNewSelections = [this, &ExistingSelectionIndex, &HitAssetID, &ElementType](const TArray<int32>& IdsIn, TArray<FUVToolSelection>& Selections)
	{
		if (IdsIn.IsEmpty())
		{
			// Nothing to add or modify.
		}
		else if (ShouldRestartSelection()
			|| (ExistingSelectionIndex == IndexConstants::InvalidID && !ShouldRemoveFromSelection()))
		{
			// Make a new selection object
			Selections.Emplace();
			Selections.Last().Target = Targets[HitAssetID];
			Selections.Last().Type = ElementType;
			Selections.Last().SelectedIDs.Append(IdsIn);
		}
		else if (ExistingSelectionIndex != IndexConstants::InvalidID)
		{
			// Modify the existing selection object
			ModifyExistingSelection(Selections[ExistingSelectionIndex].SelectedIDs, IdsIn);

			// Object may end up empty due to subtraction or toggle, in which case it needs to be removed.
			if (Selections[ExistingSelectionIndex].IsEmpty())
			{
				Selections.RemoveAt(ExistingSelectionIndex);
			}
		}
		else
		{
			// The only way we can get here is if didn't have an existing selection and were trying
			// to remove selection, in which case we do nothing.
			ensure(ExistingSelectionIndex == IndexConstants::InvalidID && ShouldRemoveFromSelection());
		}
	};
	ProcessNewSelections(NewIDs, NewSelections);
	ProcessNewSelections(NewUnsetIDs, NewUnsetSelections);
	SelectionAPI->BeginChange();
	SelectionAPI->SetSelections(NewSelections, false, false);
	SelectionAPI->SetUnsetElementAppliedMeshSelections(NewUnsetSelections, false, false);
	SelectionAPI->EndChangeAndEmitIfModified(true); // broadcast and emit
}

// Usages of this should probably be replaced by RaycastCanonicals, but it doesn't seem worth the refactor.
bool UUVEditorMeshSelectionMechanic::GetHitTid(const FInputDeviceRay& ClickPos,
	int32& TidOut, int32& AssetIDOut, bool bUseUnwrap, int32* ExistingSelectionObjectIndexOut)
{
	FRaycastResult HitResult;
	bool bSuccess = RaycastCanonicals(ClickPos.WorldRay, bUseUnwrap, true, HitResult);
	AssetIDOut = HitResult.AssetID;
	TidOut = HitResult.Tid;
	if (ExistingSelectionObjectIndexOut)
	{
		*ExistingSelectionObjectIndexOut = IndexConstants::InvalidID;
		if (bSuccess)
		{
			const TArray<FUVToolSelection>& Selections = SelectionAPI->GetSelections();
			*ExistingSelectionObjectIndexOut = Selections.IndexOfByPredicate([this, AssetIDOut](const FUVToolSelection& Selection)
			{
				return Selection.Target == Targets[AssetIDOut];
			});
		}
	}
	return TidOut != IndexConstants::InvalidID;
}

bool UUVEditorMeshSelectionMechanic::RaycastCanonicals(const FRay& WorldRay, bool bRaycastIsForUnwrap,
	bool bPreferSelected, FRaycastResult& HitOut) const
{
	auto RayCastSpatial = [this, &WorldRay, &HitOut, bRaycastIsForUnwrap](int32 AssetID) {
		FTransform3d TargetTransform;
		if (bRaycastIsForUnwrap)
		{
			TargetTransform = Targets[AssetID]->UnwrapPreview->PreviewMesh->GetTransform();
		}
		else
		{
			TargetTransform = Targets[AssetID]->AppliedPreview->PreviewMesh->GetTransform();
		}

		FRay3d LocalRay(
			(FVector3d)TargetTransform.InverseTransformPosition(WorldRay.Origin),
			(FVector3d)TargetTransform.InverseTransformVector(WorldRay.Direction));

		double RayT = 0;
		if (GetMeshSpatial(AssetID, bRaycastIsForUnwrap)->FindNearestHitTriangle(LocalRay, RayT, HitOut.Tid))
		{
			HitOut.AssetID = AssetID;
			HitOut.HitPosition = TargetTransform.TransformPosition(LocalRay.PointAt(RayT));
			return true;
		}
		return false;
	};

	// Try raycasting the selected meshes first
	TArray<bool> SpatialTriedFlags;
	SpatialTriedFlags.SetNum(Targets.Num());
	const TArray<FUVToolSelection>& Selections = SelectionAPI->GetSelections();

	if (bPreferSelected)
	{
		for (int32 SelectionIndex = 0; SelectionIndex < Selections.Num(); ++SelectionIndex)
		{
			const FUVToolSelection& Selection = Selections[SelectionIndex];
			if (ensure(Selection.Target.IsValid() && Selection.Target->AssetID < Targets.Num()))
			{
				if (RayCastSpatial(Selection.Target->AssetID))
				{
					return true;
				}
				SpatialTriedFlags[Selection.Target->AssetID] = true;
			}
		}
	}

	// Try raycasting the other meshes
	for (int32 AssetID = 0; AssetID < Targets.Num(); ++AssetID)
	{
		if (SpatialTriedFlags[AssetID])
		{
			continue;
		}
		if (RayCastSpatial(AssetID))
		{
			return true;
		}
	}

	return false;
}

void UUVEditorMeshSelectionMechanic::OnDragRectangleStarted()
{
	using namespace UVEditorMeshSelectionMechanicLocals;

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_OnDragRectangleStarted); // Mark start of drag sequence
	
	PreDragSelections = SelectionAPI->GetSelections();
	PreDragUnsetSelections = SelectionAPI->GetUnsetElementAppliedMeshSelections();
	SelectionAPI->BeginChange();

	AssetIDToPreDragSelection.Reset();
	AssetIDToPreDragUnsetSelection.Reset();
	AssetIDToPreDragSelection.SetNumZeroed(Targets.Num());
	AssetIDToPreDragUnsetSelection.SetNumZeroed(Targets.Num());
	FUVToolSelection::EType ExpectedSelectionType = ToCompatibleDynamicMeshSelectionType(SelectionMode);

	if (SelectionAPI->HaveSelections() 
		&& SelectionAPI->GetSelectionsType() == ExpectedSelectionType)
	{
		for (FUVToolSelection& Selection : PreDragSelections)
		{
			if (ensure(Selection.Type == ExpectedSelectionType))
			{
				AssetIDToPreDragSelection[Selection.Target->AssetID] = &Selection;
			}
		}
	}
	if (SelectionAPI->HaveUnsetElementAppliedMeshSelections()
		&& SelectionAPI->GetSelectionsType() == ExpectedSelectionType)
	{
		for (FUVToolSelection& Selection : PreDragUnsetSelections)
		{
			if (ensure(Selection.Type == ExpectedSelectionType))
			{
				AssetIDToPreDragUnsetSelection[Selection.Target->AssetID] = &Selection;
			}
		}
	}

}

void UUVEditorMeshSelectionMechanic::OnDragRectangleChanged(const FCameraRectangle& CurrentRectangle, bool bSourceIsLivePreview)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_OnDragRectangleChanged);

	using namespace UVEditorMeshSelectionMechanicLocals;

	TArray<FUVToolSelection> NewSelections, NewUnsetSelections;
	FUVToolSelection::EType SelectionType = ToCompatibleDynamicMeshSelectionType(SelectionMode);

	// Gather IDs in each target
	for (int32 AssetID = 0; AssetID < Targets.Num(); ++AssetID)
	{
		// Used for Unwrap Selection
		FAxisAlignedBox2d RectangleXY = GetRectangleXY(CurrentRectangle);

		// Used for Live Preview Selection
		// TODO: Frustum transformation is done the same way in GroupTopologySelector.cpp and
		// FractureEditorMode.cpp- should this be placed somewhere common? Not sure where to put it though.
		FTransform3d TargetTransform = Targets[AssetID]->AppliedPreview->PreviewMesh->GetTransform();
		FConvexVolume WorldSpaceFrustum = CurrentRectangle.FrustumAsConvexVolume();
		FMatrix InverseTargetTransform(FTransform(TargetTransform).ToInverseMatrixWithScale());
		FConvexVolume LocalFrustum;
		LocalFrustum.Planes.Empty(6);
		for (const FPlane& Plane : WorldSpaceFrustum.Planes)
		{
			LocalFrustum.Planes.Add(Plane.TransformBy(InverseTargetTransform));
		}
		LocalFrustum.Init();

		const FDynamicMesh3& Mesh = *Targets[AssetID]->AppliedCanonical;
		const FDynamicMesh3& UnwrapMesh = *Targets[AssetID]->UnwrapCanonical;
		const FDynamicMeshUVOverlay& UVOverlay = *Mesh.Attributes()->GetUVLayer(Targets[AssetID]->UVLayerIndex);

		TArray<int32> RectangleSelectedIDs, UnsetRectangleSelectedIDs;
		const FDynamicMeshAABBTree3& Tree = *GetMeshSpatial(AssetID, !bSourceIsLivePreview);

		if (SelectionMode == ESelectionMode::Vertex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_OnDragRectangleChanged_Vertex);

			if (bSourceIsLivePreview)
			{
				RectangleSelectedIDs = FindAllIntersections(
					Tree, LocalFrustum, AppendVertexIDs, AppendVertexIDsIfIntersectedFrustum);
			}
			else
			{
				RectangleSelectedIDs = FindAllIntersections(
					Tree, RectangleXY, AppendVertexIDs, AppendVertexIDsIfIntersectedRectangle);
			}
		}
		else if (SelectionMode == ESelectionMode::Edge)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_OnDragRectangleChanged_Edge);

			if (bSourceIsLivePreview)
			{
				RectangleSelectedIDs = FindAllIntersections(
					Tree, LocalFrustum, AppendEdgeIDs, AppendEdgeIDsIfIntersectedFrustum);
			}
			else
			{
				RectangleSelectedIDs = FindAllIntersections(
					Tree, RectangleXY, AppendEdgeIDs, AppendEdgeIDsIfIntersectedRectangle);
			}
		}
		else if (SelectionMode == ESelectionMode::Triangle)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_OnDragRectangleChanged_Triangle);

			if (bSourceIsLivePreview)
			{
				RectangleSelectedIDs = FindAllIntersections(
					Tree, LocalFrustum, AppendTriangleID, AppendTriangleIDIfIntersectedFrustum);
			}
			else
			{
				RectangleSelectedIDs = FindAllIntersections(
					Tree, RectangleXY, AppendTriangleID, AppendTriangleIDIfIntersectedRectangle);
			}
		}
		else if (SelectionMode == ESelectionMode::Island)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_OnDragRectangleChanged_Component);

			TArray<int32> SeedTriangles;
			if (bSourceIsLivePreview)
			{
				SeedTriangles = FindAllIntersections(
					Tree, LocalFrustum, AppendTriangleID, AppendTriangleIDIfIntersectedFrustum);
			}
			else
			{
				SeedTriangles = FindAllIntersections(
					Tree, RectangleXY, AppendTriangleID, AppendTriangleIDIfIntersectedRectangle);
			}

			// TODO(Performance) For large meshes and selections following code is MUCH slower than AABB traversal,
			//  consider precomputing the connected components an only updating them when the mesh topology changes
			//  rather than every time the selection changes.
			FMeshConnectedComponents MeshSelectedComponent(Tree.GetMesh());
			auto ConnectedComponentsPredicate = [&UnwrapMesh](int32 TriangleA, int32 TriangleB)
			{
				if (UnwrapMesh.IsTriangle(TriangleA))
				{
					return UnwrapMesh.GetTriNeighbourTris(TriangleA).Contains(TriangleB);
				}
				else
				{
					return !UnwrapMesh.IsTriangle(TriangleB);
				}
			};
			if (bSourceIsLivePreview)
			{
				MeshSelectedComponent.FindTrianglesConnectedToSeeds(SeedTriangles, ConnectedComponentsPredicate);
			}
			else
			{
				MeshSelectedComponent.FindTrianglesConnectedToSeeds(SeedTriangles);
			}
			for (int ComponentIndex = 0; ComponentIndex < MeshSelectedComponent.Components.Num(); ComponentIndex++)
			{
				RectangleSelectedIDs.Append(MoveTemp(MeshSelectedComponent.Components[ComponentIndex].Indices));
			}
		}
		else if (SelectionMode == ESelectionMode::Mesh)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_OnDragRectangleChanged_Mesh);

			// TODO: This shouldn't be a "find all". We can return early after the first success
			// since we're selecting the whole mesh
			TArray<int32> SelectedIDs;
			if (bSourceIsLivePreview)
			{
				SelectedIDs = FindAllIntersections(
					Tree, LocalFrustum, AppendTriangleID, AppendTriangleIDIfIntersectedFrustum);
			}
			else
			{
				SelectedIDs = FindAllIntersections(
					Tree, RectangleXY, AppendTriangleID, AppendTriangleIDIfIntersectedRectangle);
			}
			if (!SelectedIDs.IsEmpty())
			{
				for (int32 Tid : Tree.GetMesh()->TriangleIndicesItr())
				{
					RectangleSelectedIDs.Add(Tid);
				}
			}
		}
		else
		{
			checkSlow(false);
		}

		if (bSourceIsLivePreview)
		{
			RectangleSelectedIDs = Convert3DHitsTo2DHits(SelectionMode, UnwrapMesh, Mesh, UVOverlay, RectangleSelectedIDs, UnsetRectangleSelectedIDs);
		}

		// See if we have an object in our selection list that corresponds to this asset
		const FUVToolSelection* PreDragSelection = AssetIDToPreDragSelection[AssetID];
		const FUVToolSelection* PreDragUnsetSelection = AssetIDToPreDragUnsetSelection[AssetID];

		auto ProcessSelection = [this, &AssetID, &SelectionType](const TArray<int32>& IdsIn, const FUVToolSelection* OldSelection, TArray<FUVToolSelection>& Selections) 
		{
			if (IdsIn.IsEmpty())
			{
				if (!ShouldRestartSelection() && OldSelection)
				{
					// Keep the existing selection object with no modification.
					Selections.Emplace(*OldSelection);
				}
			}
			else if (ShouldRestartSelection() || (!OldSelection && !ShouldRemoveFromSelection()))
			{
				// Make a new selection object
				Selections.Emplace();
				Selections.Last().Target = Targets[AssetID];
				Selections.Last().Type = SelectionType;
				Selections.Last().SelectedIDs.Append(IdsIn);
			}
			else if (OldSelection)
			{
				// Modify the existing selection object
				FUVToolSelection NewSelection(*OldSelection);
				ModifyExistingSelection(NewSelection.SelectedIDs, IdsIn);

				// The object may become empty from a removal or toggle, in which case don't add it.
				if (!NewSelection.IsEmpty())
				{
					Selections.Add(MoveTemp(NewSelection));
				}
			}
			else
			{
				// The only way we can get here is if didn't have an existing selection and were trying
				// to remove selection, in which case we do nothing.
				ensure(!OldSelection && ShouldRemoveFromSelection());
			}
		};
		ProcessSelection(RectangleSelectedIDs, PreDragSelection, NewSelections);
		ProcessSelection(UnsetRectangleSelectedIDs, PreDragUnsetSelection, NewUnsetSelections);
	}

	SelectionAPI->SetSelections(NewSelections, false, false);
	SelectionAPI->SetUnsetElementAppliedMeshSelections(NewUnsetSelections, false, false);
	OnDragSelectionChanged.Broadcast();
}


void UUVEditorMeshSelectionMechanic::OnDragRectangleFinished(const FCameraRectangle& CurrentRectangle, bool bCancelled, bool bSourceIsLivePreview)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_OnDragRectangleFinished); // Mark end of drag sequence

	// TODO(Performance) :DynamicMarqueeSelection Remove this call when marquee selection is fast enough to update
	//  dynamically for large meshes
	OnDragRectangleChanged(CurrentRectangle, bSourceIsLivePreview);

	if (!bCancelled)
	{
		SelectionAPI->EndChangeAndEmitIfModified(true);
	}
}

void UUVEditorMeshSelectionMechanic::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	switch (ModifierID)
	{
	case ShiftModifierID:
		bShiftToggle = bIsOn;
		break;
	case CtrlModifierID:
		bCtrlToggle = bIsOn;
		break;
	default:
		break;
	}
}

FInputRayHit UUVEditorMeshSelectionMechanic::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos, bool bSourceIsLivePreview)
{
	FInputRayHit Hit;
	if (!bIsEnabled || !bShowHoveredElements || SelectionMode == ESelectionMode::None)
	{
		Hit.bHit = false;
		return Hit;
	}

	ESelectionMode Mode = SelectionMode;
	if (Mode != ESelectionMode::Vertex && Mode != ESelectionMode::Edge)
	{
		Mode = ESelectionMode::Triangle;
	}

	// We don't bother with the depth since everything is in the same plane.
	int32 Tid = IndexConstants::InvalidID;
	int32 AssetID = IndexConstants::InvalidID;
	Hit.bHit = GetHitTid(PressPos, Tid, AssetID, !bSourceIsLivePreview);

	return Hit;
}

void UUVEditorMeshSelectionMechanic::OnBeginHover(const FInputDeviceRay& DevicePos)
{
}

bool UUVEditorMeshSelectionMechanic::OnUpdateHover(const FInputDeviceRay& DevicePos, bool bSourceIsLivePreview)
{
	using namespace UVEditorMeshSelectionMechanicLocals;

	if (!IsEnabled())
	{
		return false;
	}

	ESelectionMode Mode = SelectionMode;
	if (SelectionMode != ESelectionMode::Vertex && SelectionMode != ESelectionMode::Edge)
	{
		Mode = ESelectionMode::Triangle;
	}

	HoverPointSet->Clear();
	HoverLineSet->Clear();
	HoverTriangleSet->Clear();
	LivePreviewHoverPointSet->Clear();
	LivePreviewHoverLineSet->Clear();
	LivePreviewHoverTriangleSet->Clear();

	int32 Tid = IndexConstants::InvalidID;
	int32 AssetID = IndexConstants::InvalidID;
	if (!GetHitTid(DevicePos, Tid, AssetID, !bSourceIsLivePreview))
	{
		return false;
	}
	FDynamicMesh3* UnwrapMesh = Targets[AssetID]->UnwrapCanonical.Get();
	FDynamicMesh3* AppliedMesh = Targets[AssetID]->AppliedCanonical.Get();
	FTransform3d AppliedTargetTransform = Targets[AssetID]->AppliedPreview->PreviewMesh->GetTransform();
	FDynamicMesh3* Mesh = bSourceIsLivePreview ? AppliedMesh : UnwrapMesh;
	FDynamicMeshUVOverlay& UVOverlay = *AppliedMesh->Attributes()->GetUVLayer(Targets[AssetID]->UVLayerIndex);

	TArray<int32> Converted3DIDs, Unset3DIDs;
	TArray<int32> Converted2DIDs;
	if (SelectionMode == ESelectionMode::Vertex || SelectionMode == ESelectionMode::Edge)
	{
		if (bSourceIsLivePreview)
		{
			ConvertToHitElementList(SelectionMode, *Mesh, *UnwrapMesh, UVOverlay,
				Tid, CameraState, DevicePos.WorldRay, Converted3DIDs);
			Converted2DIDs = Convert3DHitsTo2DHits(SelectionMode, *UnwrapMesh, *AppliedMesh, UVOverlay, Converted3DIDs, Unset3DIDs);
		}
		else
		{
			ConvertToHitElementList(SelectionMode, *Mesh, *UnwrapMesh, UVOverlay,
				Tid, CameraState, DevicePos.WorldRay, Converted2DIDs);
			Converted3DIDs = Convert2DHitsTo3DHits(SelectionMode, *UnwrapMesh, *AppliedMesh, UVOverlay, Converted2DIDs);
		}
	}


	auto SetupVertexHighlight = [](TArray<int32>& IDs, FDynamicMesh3& Mesh, UPointSetComponent& PointSet, FTransform3d Transform = FTransform3d::Identity)
	{
		for (int32 ID : IDs)
		{
			const FVector3d P = Transform.TransformPosition(Mesh.GetVertexRef(ID));
			const FRenderablePoint PointToRender(P,
				FUVEditorUXSettings::SelectionHoverTriangleWireframeColor,
				FUVEditorUXSettings::SelectionPointThickness);

			PointSet.AddPoint(PointToRender);
		}
	};

	auto SetupEdgeHighlight = [](TArray<int32>& IDs, FDynamicMesh3& Mesh, ULineSetComponent& LineSet, FTransform3d Transform = FTransform3d::Identity)
	{
		for (int32 ID : IDs)
		{
			const FIndex2i EdgeVids = Mesh.GetEdgeV(ID);
			const FVector A = Transform.TransformPosition(Mesh.GetVertexRef(EdgeVids.A));
			const FVector B = Transform.TransformPosition(Mesh.GetVertexRef(EdgeVids.B));

			LineSet.AddLine(A, B,
				FUVEditorUXSettings::SelectionHoverTriangleWireframeColor,
				FUVEditorUXSettings::SelectionLineThickness,
				FUVEditorUXSettings::SelectionHoverWireframeDepthBias);
		}
	};

	auto SetupTriangleHighlight = [this, &UnwrapMesh, &Tid](FDynamicMesh3& Mesh, ULineSetComponent& LineSet, UTriangleSetComponent& TriangleSet, FTransform3d Transform = FTransform3d::Identity)
	{
		if (!UnwrapMesh->IsTriangle(Tid))
		{
			return;
		}

		const FIndex3i Vids = Mesh.GetTriangle(Tid);
		const FVector A = Transform.TransformPosition(Mesh.GetVertex(Vids[0]));
		const FVector B = Transform.TransformPosition(Mesh.GetVertex(Vids[1]));
		const FVector C = Transform.TransformPosition(Mesh.GetVertex(Vids[2]));

		LineSet.AddLine(A, B, FUVEditorUXSettings::SelectionHoverTriangleWireframeColor,
			FUVEditorUXSettings::SelectionLineThickness, FUVEditorUXSettings::SelectionHoverWireframeDepthBias);
		LineSet.AddLine(B, C, FUVEditorUXSettings::SelectionHoverTriangleWireframeColor,
			FUVEditorUXSettings::SelectionLineThickness, FUVEditorUXSettings::SelectionHoverWireframeDepthBias);
		LineSet.AddLine(C, A, FUVEditorUXSettings::SelectionHoverTriangleWireframeColor,
			FUVEditorUXSettings::SelectionLineThickness, FUVEditorUXSettings::SelectionHoverWireframeDepthBias);
		TriangleSet.AddTriangle(A, B, C, FVector::ZAxisVector,
			FUVEditorUXSettings::SelectionHoverTriangleFillColor, HoverTriangleSetMaterial);
	};

	// Generate hover visualizations for moused over geometry. This potentially generates more than one "item"
	// per mesh, since the 3D item might coorespond to more than one 2D item. In particular, vertices and edges
	// may have more than one match on the 2D side. We also handle the potential existence of hovering over geometry
	// that is part of unset UVs, which only generate hover visuals on the 3D side.
	if (SelectionMode == ESelectionMode::Vertex)
	{
		SetupVertexHighlight(Converted2DIDs, *UnwrapMesh, *HoverPointSet);
		SetupVertexHighlight(Converted3DIDs, *AppliedMesh, *LivePreviewHoverPointSet, AppliedTargetTransform);
		SetupVertexHighlight(Unset3DIDs, *AppliedMesh, *LivePreviewHoverPointSet, AppliedTargetTransform);
	}
	else if (SelectionMode == ESelectionMode::Edge)
	{
		SetupEdgeHighlight(Converted2DIDs, *UnwrapMesh, *HoverLineSet);
		SetupEdgeHighlight(Converted3DIDs, *AppliedMesh, *LivePreviewHoverLineSet, AppliedTargetTransform);
		SetupEdgeHighlight(Unset3DIDs, *AppliedMesh, *LivePreviewHoverLineSet, AppliedTargetTransform);
	}
	else
	{
		SetupTriangleHighlight(*UnwrapMesh, *HoverLineSet, *HoverTriangleSet);
		SetupTriangleHighlight(*AppliedMesh, *LivePreviewHoverLineSet, *LivePreviewHoverTriangleSet, AppliedTargetTransform);
	}

	return true;
}

void UUVEditorMeshSelectionMechanic::OnEndHover()
{
	if (ensure(HoverPointSet.IsValid()))
	{
		HoverPointSet->Clear();
	}
	if (ensure(HoverLineSet.IsValid()))
	{
		HoverLineSet->Clear();
	}
	if (ensure(HoverTriangleSet.IsValid()))
	{
		HoverTriangleSet->Clear();
	}
	if (ensure(LivePreviewHoverPointSet.IsValid()))
	{
		LivePreviewHoverPointSet->Clear();
	}
	if (ensure(LivePreviewHoverLineSet.IsValid()))
	{
		LivePreviewHoverLineSet->Clear();
	}
	if (ensure(LivePreviewHoverTriangleSet.IsValid()))
	{
		LivePreviewHoverTriangleSet->Clear();
	}
}

#undef LOCTEXT_NAMESPACE


