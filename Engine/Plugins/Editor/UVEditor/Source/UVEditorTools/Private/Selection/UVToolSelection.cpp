// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/UVToolSelection.h"

#define LOCTEXT_NAMESPACE "UUVToolSelection"

using namespace UE::Geometry;

void FUVToolSelection::SaveStableEdgeIdentifiers(const FDynamicMesh3& Mesh)
{
	if (Type == EType::Edge)
	{
		StableEdgeIDs.InitializeFromEdgeIDs(Mesh, SelectedIDs);
	}
}

void FUVToolSelection::RestoreFromStableEdgeIdentifiers(const FDynamicMesh3& Mesh)
{
	if (Type == EType::Edge)
	{
		StableEdgeIDs.GetEdgeIDs(Mesh, SelectedIDs);
	}
}

bool FUVToolSelection::AreElementsPresentInMesh(const FDynamicMesh3& Mesh) const
{
	switch (Type)
	{
	case EType::Vertex:
		for (int32 Vid : SelectedIDs)
		{
			if (!Mesh.IsVertex(Vid))
			{
				return false;
			}
		}
		break;
	case EType::Edge:
		for (int32 Eid : SelectedIDs)
		{
			if (!Mesh.IsEdge(Eid))
			{
				return false;
			}
		}
		break;
	case EType::Triangle:
		for (int32 Tid : SelectedIDs)
		{
			if (!Mesh.IsTriangle(Tid))
			{
				return false;
			}
		}
		break;
	}
	return true;
}

void FUVToolSelection::SelectAll(const FDynamicMesh3& Mesh, EType TypeIn)
{
	Type = TypeIn;
	StableEdgeIDs.Reset();
	SelectedIDs.Reset();
	switch (Type)
	{
	case EType::Vertex:
		SelectedIDs.Reserve(Mesh.VertexCount());
		for (int32 Vid : Mesh.VertexIndicesItr())
		{
			SelectedIDs.Add(Vid);
		}
		break;
	case EType::Edge:
		SelectedIDs.Reserve(Mesh.EdgeCount());
		for (int32 Eid : Mesh.EdgeIndicesItr())
		{
			SelectedIDs.Add(Eid);
		}
		break;
	case EType::Triangle:
		SelectedIDs.Reserve(Mesh.TriangleCount());
		for (int32 Tid : Mesh.TriangleIndicesItr())
		{
			SelectedIDs.Add(Tid);
		}
		break;
	default:
		ensure(false);
	}
}

bool FUVToolSelection::IsAllSelected(const FDynamicMesh3& Mesh) const
{
	switch (Type)
	{
	case EType::Vertex:
		if (SelectedIDs.Num() != Mesh.VertexCount())
		{
			return false;
		}
		break;
	case EType::Edge:
		if (SelectedIDs.Num() != Mesh.EdgeCount())
		{
			return false;
		}
		break;
	case EType::Triangle:
		if (SelectedIDs.Num() != Mesh.TriangleCount())
		{
			return false;
		}
		break;
	default:
		ensure(false);
	}

	return AreElementsPresentInMesh(Mesh);
}

FAxisAlignedBox3d FUVToolSelection::ToBoundingBox(const FDynamicMesh3& Mesh, const FTransform3d Transform) const
{
	FAxisAlignedBox3d BoundingBox;

	switch (Type)
	{
	case EType::Vertex:
		for (int32 Vid : SelectedIDs)
		{
			if (Mesh.IsVertex(Vid))
			{
				BoundingBox.Contain(Transform.TransformPosition(Mesh.GetVertexRef(Vid)));
			}
		}
		break;
	case EType::Edge:
		for (int32 Eid : SelectedIDs)
		{
			if (Mesh.IsEdge(Eid))
			{
				BoundingBox.Contain(Transform.TransformPosition(Mesh.GetVertexRef(Mesh.GetEdgeRef(Eid).Vert.A)));
				BoundingBox.Contain(Transform.TransformPosition(Mesh.GetVertexRef(Mesh.GetEdgeRef(Eid).Vert.B)));
			}
		}
		break;
	case EType::Triangle:
		for (int32 Tid : SelectedIDs)
		{
			if (Mesh.IsTriangle(Tid))
			{
				BoundingBox.Contain(Transform.TransformPosition(Mesh.GetVertexRef(Mesh.GetTriangleRef(Tid).A)));
				BoundingBox.Contain(Transform.TransformPosition(Mesh.GetVertexRef(Mesh.GetTriangleRef(Tid).B)));
				BoundingBox.Contain(Transform.TransformPosition(Mesh.GetVertexRef(Mesh.GetTriangleRef(Tid).C)));
			}
		}
		break;
	}
	return BoundingBox;
}

FUVToolSelection FUVToolSelection::GetConvertedSelection(const FDynamicMesh3& Mesh, FUVToolSelection::EType ExpectedSelectionType) const
{
	if (Type == ExpectedSelectionType)
	{
		return *this;
	}

	FUVToolSelection NewSelection;
	NewSelection.Target = Target;
	NewSelection.Type = ExpectedSelectionType;

	const TSet<int32>& OriginalIDs = SelectedIDs;
	TSet<int32>& NewIDs = NewSelection.SelectedIDs;

	auto VerticesToEdges = [&Mesh, &OriginalIDs, &NewIDs]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ToolSelection_GetConvertedSelection_VerticesToEdges);

		for (int32 Vid : OriginalIDs)
		{
			for (int32 Eid : Mesh.VtxEdgesItr(Vid))
			{
				if (!NewIDs.Contains(Eid))
				{
					FIndex2i Verts = Mesh.GetEdgeV(Eid);
					if (OriginalIDs.Contains(Verts.A) &&
						OriginalIDs.Contains(Verts.B))
					{
						NewIDs.Add(Eid);
					}
				}
			}
		}
	};

	auto VerticesToTriangles = [&Mesh, &OriginalIDs, &NewIDs]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ToolSelection_GetConvertedSelection_VerticesToTriangles);

		for (int32 Vid : OriginalIDs)
		{
			for (int32 Tid : Mesh.VtxTrianglesItr(Vid))
			{
				if (!NewIDs.Contains(Tid))
				{
					FIndex3i Verts = Mesh.GetTriangle(Tid);
					if (OriginalIDs.Contains(Verts.A) &&
						OriginalIDs.Contains(Verts.B) &&
						OriginalIDs.Contains(Verts.C))
					{
						NewIDs.Add(Tid);
					}
				}
			}
		}
	};

	auto EdgesToVertices = [&Mesh, &OriginalIDs, &NewIDs]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ToolSelection_GetConvertedSelection_EdgesToVertices);

		for (int32 Eid : OriginalIDs)
		{
			FIndex2i Verts = Mesh.GetEdgeV(Eid);
			NewIDs.Add(Verts.A);
			NewIDs.Add(Verts.B);
		}
	};

	// Triangles with two selected edges will be selected
	auto EdgesToTriangles = [&Mesh, &OriginalIDs, &NewIDs]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ToolSelection_GetConvertedSelection_EdgesToTriangles);

		TArray<int32> FoundTriangles;
		for (int32 Eid : OriginalIDs)
		{
			FIndex2i Tris = Mesh.GetEdgeT(Eid);
			FoundTriangles.Add(Tris.A);
			if (Tris.B != IndexConstants::InvalidID)
			{
				FoundTriangles.Add(Tris.B);
			}
		}

		if (FoundTriangles.Num() < 2)
		{
			return;
		}

		Algo::Sort(FoundTriangles);

		for (int I = 0; I < FoundTriangles.Num() - 1; I++)
		{
			if (FoundTriangles[I] == FoundTriangles[I + 1])
			{
				NewIDs.Add(FoundTriangles[I]);
				I++;
			}
		}
	};

	auto TrianglesToVertices = [&Mesh, &OriginalIDs, &NewIDs]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ToolSelection_GetConvertedSelection_TrianglesToVertices);

		for (int32 Tid : OriginalIDs)
		{
			FIndex3i Verts = Mesh.GetTriangle(Tid);
			NewIDs.Add(Verts.A);
			NewIDs.Add(Verts.B);
			NewIDs.Add(Verts.C);
		}
	};

	auto TrianglesToEdges = [Mesh, &OriginalIDs, &NewIDs]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ToolSelection_GetConvertedSelection_TrianglesToEdges);

		for (int32 Tid : OriginalIDs)
		{
			FIndex3i Edges = Mesh.GetTriEdgesRef(Tid);
			NewIDs.Add(Edges.A);
			NewIDs.Add(Edges.B);
			NewIDs.Add(Edges.C);
		}
	};

	switch (Type)
	{
	case FUVToolSelection::EType::Vertex:
		switch (ExpectedSelectionType)
		{
		case FUVToolSelection::EType::Vertex:
			ensure(false); // Should have been an early-out
			break;
		case FUVToolSelection::EType::Edge:
			VerticesToEdges();
			break;
		case FUVToolSelection::EType::Triangle:
			VerticesToTriangles();
			break;
		}
		break; // SelectionMode
	case FUVToolSelection::EType::Edge:
		switch (ExpectedSelectionType)
		{
		case FUVToolSelection::EType::Vertex:
			EdgesToVertices();
			break;
		case FUVToolSelection::EType::Edge:
			ensure(false); // Should have been an early-out
			break;
		case FUVToolSelection::EType::Triangle:
			EdgesToTriangles();
			break;
		}
		break; // SelectionMode
	case FUVToolSelection::EType::Triangle:
		switch (ExpectedSelectionType)
		{
		case FUVToolSelection::EType::Vertex:
			TrianglesToVertices();
			break;
		case FUVToolSelection::EType::Edge:
			TrianglesToEdges();
			break;
		case FUVToolSelection::EType::Triangle:
			ensure(false); // Should have been an early-out
			break;
		}
		break; // SelectionMode
	}
	
	return NewSelection;
}

FUVToolSelection FUVToolSelection::GetConvertedSelectionForUnwrappedMesh() const
{
	if (!Target.IsValid() || !Target->IsValid())
	{
		ensure(false);
		return FUVToolSelection();
	}

	TArray<int32> UnsetAppliedIds;;
	FUVToolSelection NewSelection;
	NewSelection.Target = Target;
	NewSelection.Type = Type;
	NewSelection.SelectedIDs = TSet<int32>(ConvertAppliedElementIdsToUnwrappedElementIds(Type, *Target->AppliedCanonical, *Target->UnwrapCanonical,
		*Target->AppliedCanonical->Attributes()->GetUVLayer(Target->UVLayerIndex), SelectedIDs.Array(), UnsetAppliedIds));

	return NewSelection;
}

FUVToolSelection FUVToolSelection::GetConvertedSelectionForAppliedMesh() const
{
	if (!Target.IsValid() || !Target->IsValid())
	{
		ensure(false);
		return FUVToolSelection();
	}

	FUVToolSelection NewSelection;
	NewSelection.Target = Target;
	NewSelection.Type = Type;
	NewSelection.SelectedIDs = TSet<int32>(ConvertUnwrappedElementIdsToAppliedElementIds(Type, *Target->UnwrapCanonical, *Target->AppliedCanonical,
		*Target->AppliedCanonical->Attributes()->GetUVLayer(Target->UVLayerIndex), SelectedIDs.Array()));

	return NewSelection;
}

TArray<int32> FUVToolSelection::ConvertUnwrappedElementIdsToAppliedElementIds(EType SelectionMode,const FDynamicMesh3& UnwrapMesh, const FDynamicMesh3& AppliedMesh,
	const FDynamicMeshUVOverlay& UVOverlay, const TArray<int32>& IDsIn)
{
	TArray<int32> IDsOut;

	auto FindAppliedEdge = [&UnwrapMesh, &AppliedMesh, &UVOverlay](int32 Eid)
	{
		int32 Vid1, Vid2;
		FDynamicMesh3::FEdge EdgeInfo = UnwrapMesh.GetEdge(Eid);
		Vid1 = UVOverlay.GetParentVertex(EdgeInfo.Vert[0]);
		Vid2 = UVOverlay.GetParentVertex(EdgeInfo.Vert[1]);
		if (Vid1 != IndexConstants::InvalidID && Vid2 != IndexConstants::InvalidID)
		{
			return AppliedMesh.FindEdge(Vid1, Vid2);
		}
		else
		{
			return IndexConstants::InvalidID;
		}
	};

	switch (SelectionMode)
	{
	case EType::Triangle:
		return IDsIn;
	case EType::Edge:
		for (int32 Eid : IDsIn)
		{
			int32 AppliedEid = FindAppliedEdge(Eid);
			if (AppliedEid != IndexConstants::InvalidID)
			{
				IDsOut.Add(AppliedEid);
			}
		}
		return IDsOut;
	case EType::Vertex:
		for (int32 Vid : IDsIn)
		{
			IDsOut.Add(UVOverlay.GetParentVertex(Vid));
		}
		return IDsOut;
	default:
		ensure(false);
		return IDsOut;
	}
}

TArray<int32> FUVToolSelection::ConvertAppliedElementIdsToUnwrappedElementIds(EType SelectionMode,	const FDynamicMesh3& AppliedMesh, const FDynamicMesh3& UnwrapMesh,
	const FDynamicMeshUVOverlay& UVOverlay, const TArray<int32>& IDsIn, TArray<int32>& AppliedMeshOnlyIDsOut)
{
	TSet<int32> IDsOut;
	AppliedMeshOnlyIDsOut.Empty();

	auto FindUnwrapEdges = [&AppliedMesh, &UnwrapMesh](int32 Eid, TSet<int32>& OutUnwrapEdges)
	{
		bool bFoundEdges = false;
		FIndex2i Triangles = AppliedMesh.GetEdgeT(Eid);
		int32 Triangle0EdgeIndex = AppliedMesh.GetTriEdges(Triangles[0]).IndexOf(Eid);
		int32 Triangle1EdgeIndex = Triangles[1] == IndexConstants::InvalidID ? IndexConstants::InvalidID : AppliedMesh.GetTriEdges(Triangles[1]).IndexOf(Eid);
		if (UnwrapMesh.IsTriangle(Triangles[0]))
		{
			OutUnwrapEdges.Add(UnwrapMesh.GetTriEdge(Triangles[0], Triangle0EdgeIndex));
			bFoundEdges = true;
		}
		if (UnwrapMesh.IsTriangle(Triangles[1]))
		{
			OutUnwrapEdges.Add(UnwrapMesh.GetTriEdge(Triangles[1], Triangle1EdgeIndex));
			bFoundEdges = true;
		}
		return bFoundEdges;
	};

	switch (SelectionMode)
	{
	case EType::Triangle:
		for (int32 Tid : IDsIn)
		{
			if (UVOverlay.IsSetTriangle(Tid))
			{
				IDsOut.Add(Tid);
			}
			else
			{
				AppliedMeshOnlyIDsOut.Add(Tid);
			}
		}
		break;
	case EType::Edge:
		for (int32 Eid : IDsIn)
		{
			if (!FindUnwrapEdges(Eid, IDsOut))
			{
				AppliedMeshOnlyIDsOut.Add(Eid);
			}
		}
		break;
	case EType::Vertex:
		for (int32 Vid : IDsIn)
		{
			TArray<int32> ElementsForVid;
			UVOverlay.GetVertexElements(Vid, ElementsForVid);
			IDsOut.Append(ElementsForVid);
			if (ElementsForVid.IsEmpty())
			{
				AppliedMeshOnlyIDsOut.Add(Vid);
			}
		}
		break;
	default:
		ensure(false);
		break;
	}
	return IDsOut.Array();
}

#undef LOCTEXT_NAMESPACE