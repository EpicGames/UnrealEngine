// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/UVToolSelectionAPI.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "Selection/UVEditorMeshSelectionMechanic.h"
#include "Selection/UVToolSelectionHighlightMechanic.h"
#include "UVEditorMechanicAdapterTool.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVToolSelectionAPI)

#define LOCTEXT_NAMESPACE "UUVIslandConformalUnwrapAction"

using namespace UE::Geometry;

namespace UVToolSelectionAPILocals
{
	FText SelectionChangeTransactionName = LOCTEXT("SelectionChangeTransaction", "UV Selection Change");

	auto DoSelectionSetsDiffer(const TArray<FUVToolSelection>& OldSelections, const TArray<FUVToolSelection>& NewSelections)
	{
		if (NewSelections.Num() != OldSelections.Num())
		{
			return true;
		}

		for (const FUVToolSelection& Selection : NewSelections)
		{
			// Find the selection that points to the same target.
			const FUVToolSelection* FoundSelection = OldSelections.FindByPredicate(
				[&Selection](const FUVToolSelection& OldSelection) { return OldSelection.Target == Selection.Target; }
			);
			if (!FoundSelection || *FoundSelection != Selection)
			{
				return true;
			}
		}
		return false;
	};
}

void UUVToolSelectionAPI::Initialize(
	UInteractiveToolManager* ToolManagerIn,
	UWorld* UnwrapWorld, UInputRouter* UnwrapInputRouterIn, 
	UUVToolLivePreviewAPI* LivePreviewAPI,
	UUVToolEmitChangeAPI* EmitChangeAPIIn)
{
	UnwrapInputRouter = UnwrapInputRouterIn;
	EmitChangeAPI = EmitChangeAPIIn;

	MechanicAdapter = NewObject<UUVEditorMechanicAdapterTool>();
	MechanicAdapter->ToolManager = ToolManagerIn;

	HighlightMechanic = NewObject<UUVToolSelectionHighlightMechanic>();
	HighlightMechanic->Setup(MechanicAdapter);
	HighlightMechanic->Initialize(UnwrapWorld, LivePreviewAPI->GetLivePreviewWorld());

	SelectionMechanic = NewObject<UUVEditorMeshSelectionMechanic>();
	SelectionMechanic->Setup(MechanicAdapter);
	SelectionMechanic->Initialize(UnwrapWorld, LivePreviewAPI->GetLivePreviewWorld(), this);

	UnwrapInputRouter->RegisterSource(MechanicAdapter);

	bIsActive = true;
}

void UUVToolSelectionAPI::SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
{
	Targets = TargetsIn;
	SelectionMechanic->SetTargets(Targets);

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->OnCanonicalModified.AddWeakLambda(this, [this](UUVEditorToolMeshInput* Target,
			const UUVEditorToolMeshInput::FCanonicalModifiedInfo ModifiedInfo)
			{
				FUVToolSelection* TargetSelection = CurrentSelections.FindByPredicate(
					[Target](const FUVToolSelection& CandidateSelection) { return CandidateSelection.Target == Target; });
				if (TargetSelection)
				{
					bCachedUnwrapSelectionBoundingBoxCenterValid = false;
					if (TargetSelection->Type == FUVToolSelection::EType::Edge)
					{
						TargetSelection->RestoreFromStableEdgeIdentifiers(*Target->UnwrapCanonical);
					}
				}				
			});
	}
}

void UUVToolSelectionAPI::Shutdown()
{
	if (UnwrapInputRouter.IsValid())
	{
		// Make sure that we stop any captures that our mechanics may have, then remove them from
		// the input router.
		UnwrapInputRouter->DeregisterSource(MechanicAdapter);
		UnwrapInputRouter = nullptr;
	}

	HighlightMechanic->Shutdown();
	SelectionMechanic->Shutdown();

	MechanicAdapter->Shutdown(EToolShutdownType::Completed);
	HighlightMechanic = nullptr;
	SelectionMechanic = nullptr;
	MechanicAdapter = nullptr;

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->OnCanonicalModified.RemoveAll(this);
	}
	Targets.Empty();

	bIsActive = false;
}

void UUVToolSelectionAPI::ClearSelections(bool bBroadcast, bool bEmitChange)
{
	SetSelections(TArray<FUVToolSelection>(), bBroadcast, bEmitChange);
}

void UUVToolSelectionAPI::OnToolEnded(UInteractiveTool* DeadTool)
{
	if (SelectionMechanic)
	{
		SetSelectionMechanicOptions(FSelectionMechanicOptions());
		SelectionMechanic->SetIsEnabled(false);
	}
	if (HighlightMechanic)
	{
		SetHighlightVisible(false, false);
		SetHighlightOptions(FHighlightOptions());
	}

	OnSelectionChanged.RemoveAll(DeadTool);
	OnPreSelectionChange.RemoveAll(DeadTool);
	OnDragSelectionChanged.RemoveAll(DeadTool);
	
	if (UnwrapInputRouter.IsValid())
	{
		// Make sure that we stop any captures that our mechanics may have.
		UnwrapInputRouter->ForceTerminateSource(MechanicAdapter);
	}
	
}

void UUVToolSelectionAPI::SetLivePreviewSelectionUXSettings(const FLivePreviewSelectionUXSettings& Settings) const
{
	if (Settings.SelectionColor)
	{
		HighlightMechanic->SetColor(Settings.SelectionColor.GetValue());
	}

	if (Settings.LineThickness)
	{
		HighlightMechanic->SetLineThickness(Settings.LineThickness.GetValue());
	}

	if (Settings.PointSize)
	{
		HighlightMechanic->SetPointSize(Settings.PointSize.GetValue());
	}
}

void UUVToolSelectionAPI::SetSelections(const TArray<FUVToolSelection>& SelectionsIn, bool bBroadcast, bool bEmitChange)
{
	using namespace UVToolSelectionAPILocals;

	bool bUsingExistingTransaction = bEmitChange;
	if (bEmitChange && ensureMsgf(!PendingSelectionChange && !PendingUnsetSelectionChange,
		TEXT("SetSelections called with bEmitChange while an existing SelectionAPI change transaction is already open. This should be safe but bEmitChange does not need to be explicitly set.")))
	{
		BeginChange();
		bUsingExistingTransaction = false;
	}

	bCachedUnwrapSelectionBoundingBoxCenterValid = false;

	// If we don't match with the current unset selection type, just clear them to keep everything consistent
	if (CurrentUnsetSelections.Num() > 0 && SelectionsIn.Num() > 0 && SelectionsIn[0].Type != CurrentUnsetSelections[0].Type)
	{
		CurrentUnsetSelections.Empty();
	}

	TArray<FUVToolSelection> NewSelections;
	for (const FUVToolSelection& NewSelection : SelectionsIn)
	{
		if (ensure(NewSelection.Target.IsValid() && NewSelection.Target->AreMeshesValid())
			// All of the selections should match type
			&& ensure(NewSelections.Num() == 0 || NewSelection.Type == NewSelections[0].Type)
			// Selection must not be empty, unless it's an edge selection stored as stable identifiers
			&& ensure(!NewSelection.IsEmpty() 
				|| (NewSelection.Type == FUVToolSelection::EType::Edge && NewSelection.HasStableEdgeIdentifiers()))
			// Shouldn't have selection objects pointing to same target
			&& ensure(!NewSelections.FindByPredicate(
				[&NewSelection](const FUVToolSelection& ExistingSelectionElement) {
					return NewSelection.Target == ExistingSelectionElement.Target;
				})))
		{
			NewSelections.Add(NewSelection);

			if (NewSelection.Target.IsValid() && ensure(NewSelection.Target->UnwrapCanonical))
			{
				checkSlow(NewSelection.AreElementsPresentInMesh(*NewSelection.Target->UnwrapCanonical));

				if (NewSelection.Type == FUVToolSelection::EType::Edge)
				{
					NewSelections.Last().SaveStableEdgeIdentifiers(*NewSelection.Target->UnwrapCanonical);
				}
			}
		}
	}

	/*
	* Depending on whether we're calling EndChangeAndEmitIfModified or not, the comparison of selections
	* and the broadcasting of PreSelectionChange and OnSelectionChanged might be done for us. If we're not 
	* calling EndChangeAndEmitIfModified (either because bEmitChange was false, or we're inside an existing
	* transaction that we expect the user to close), then we have to do these things ourselves.
	* 
	* This is a bit messy, but it allows us to not duplicate the change emitting code from EndChangeAndEmitIfModified.
	*/
	bool bWillCallEndChange = bEmitChange && !bUsingExistingTransaction;

	bool bSelectionsDiffer = false;
	if (!bWillCallEndChange)
	{
		bSelectionsDiffer = DoSelectionSetsDiffer(CurrentSelections, NewSelections);

		if (bBroadcast && bSelectionsDiffer)
		{
			OnPreSelectionChange.Broadcast(bEmitChange, (uint32)(ESelectionChangeTypeFlag::SelectionChanged));
		}
	}

	CurrentSelections = MoveTemp(NewSelections);

	if (bWillCallEndChange)
	{
		bSelectionsDiffer = EndChangeAndEmitIfModified(bBroadcast);
	}
	else
	{
		if (bBroadcast && bSelectionsDiffer)
		{
			OnSelectionChanged.Broadcast(bEmitChange, (uint32)(ESelectionChangeTypeFlag::SelectionChanged));
		}
	}

	if (bSelectionsDiffer)
	{

		if (HighlightOptions.bAutoUpdateUnwrap)
		{
			FTransform Transform = FTransform::Identity;
			if (HighlightOptions.bUseCentroidForUnwrapAutoUpdate)
			{
				Transform = FTransform(GetUnwrapSelectionBoundingBoxCenter());
			}
			RebuildUnwrapHighlight(Transform);
		}
		if (HighlightOptions.bAutoUpdateApplied)
		{
			RebuildAppliedPreviewHighlight();
		}
	}
}

void UUVToolSelectionAPI::SetSelectionMechanicOptions(const FSelectionMechanicOptions& Options)
{
	SelectionMechanic->SetShowHoveredElements(Options.bShowHoveredElements);
}

void UUVToolSelectionAPI::SetSelectionMechanicEnabled(bool bIsEnabled)
{
	SelectionMechanic->SetIsEnabled(bIsEnabled);
}

void UUVToolSelectionAPI::SetSelectionMechanicMode(EUVEditorSelectionMode Mode,
	const FSelectionMechanicModeChangeOptions& Options)
{
	SelectionMechanic->SetSelectionMode(Mode, Options);
}

FVector3d UUVToolSelectionAPI::GetUnwrapSelectionBoundingBoxCenter(bool bForceRecalculate)
{
	if (bCachedUnwrapSelectionBoundingBoxCenterValid && !bForceRecalculate)
	{
		return CachedUnwrapSelectionBoundingBoxCenter;
	}

	FVector3d& Center = CachedUnwrapSelectionBoundingBoxCenter;
	
	FAxisAlignedBox3d AllSelectionsBoundingBox;
	double Divisor = 0;
	for (const FUVToolSelection& Selection : GetSelections())
	{
		FDynamicMesh3* Mesh = Selection.Target->UnwrapCanonical.Get();
		AllSelectionsBoundingBox.Contain(Selection.ToBoundingBox(*Mesh));
	}
	Center = AllSelectionsBoundingBox.Center();

	bCachedUnwrapSelectionBoundingBoxCenterValid = true;
	return Center;
}


bool UUVToolSelectionAPI::HaveUnsetElementAppliedMeshSelections() const
{
	return CurrentUnsetSelections.Num() > 0;
}

void UUVToolSelectionAPI::SetUnsetElementAppliedMeshSelections(const TArray<FUVToolSelection>& UnsetSelectionsIn, bool bBroadcast, bool bEmitChange)
{
	using namespace UVToolSelectionAPILocals;

	bool bUsingExistingTransaction = bEmitChange;
	if (bEmitChange && ensureMsgf(!PendingSelectionChange && !PendingUnsetSelectionChange,
		TEXT("SetUnsetSelections called with bEmitChange while an existing SelectionAPI change transaction is already open. This should be safe but bEmitChange does not need to be explicitly set.")))
	{
		BeginChange();
		bUsingExistingTransaction = false;
	}

	// If we don't match with the current unset selection type, just clear them to keep everything consistent
	if (CurrentSelections.Num() > 0 && UnsetSelectionsIn.Num() > 0 && UnsetSelectionsIn[0].Type != CurrentSelections[0].Type)
	{
		CurrentSelections.Empty();
	}

	TArray<FUVToolSelection> NewUnsetSelections;
	for (const FUVToolSelection& NewUnsetSelection : UnsetSelectionsIn)
	{
		if (ensure(NewUnsetSelection.Target.IsValid() && NewUnsetSelection.Target->AreMeshesValid())
			// All of the unset selections should match type
			&& ensure(NewUnsetSelections.Num() == 0 || NewUnsetSelection.Type == NewUnsetSelections[0].Type)
			// Unset selection must not be empty
			&& ensure(!NewUnsetSelection.IsEmpty())
				// Shouldn't have unset selection objects pointing to same target
			&& ensure(!NewUnsetSelections.FindByPredicate(
				[&NewUnsetSelection](const FUVToolSelection& ExistingSelectionElement) {
					return NewUnsetSelection.Target == ExistingSelectionElement.Target;
				})))
		{
			NewUnsetSelections.Add(NewUnsetSelection);

			if (NewUnsetSelection.Target.IsValid() && ensure(NewUnsetSelection.Target->UnwrapCanonical))
			{
				// These are unset, so they better not be in the Unwrap mesh
				checkSlow(!NewUnsetSelection.AreElementsPresentInMesh(*NewUnsetSelection.Target->UnwrapCanonical));
			}
		}
	}

	/*
	* Depending on whether we're calling EndChangeAndEmitIfModified or not, the comparison of selections
	* and the broadcasting of PreSelectionChange and OnSelectionChanged might be done for us. If we're not 
	* calling EndChangeAndEmitIfModified (either because bEmitChange was false, or we're inside an existing
	* transaction that we expect the user to close), then we have to do these things ourselves.
	* 
	* This is a bit messy, but it allows us to not duplicate the change emitting code from EndChangeAndEmitIfModified.
	*/
	bool bWillCallEndChange = bEmitChange && !bUsingExistingTransaction;

	bool bSelectionsDiffer = false;
	if (!bWillCallEndChange)
	{
		bSelectionsDiffer = DoSelectionSetsDiffer(CurrentUnsetSelections, NewUnsetSelections);

		if (bBroadcast && bSelectionsDiffer)
		{
			OnPreSelectionChange.Broadcast(bEmitChange, (uint32)(ESelectionChangeTypeFlag::UnsetSelectionChanged));
		}
	}

	CurrentUnsetSelections = MoveTemp(NewUnsetSelections);

	if (bWillCallEndChange)
	{
		bSelectionsDiffer = EndChangeAndEmitIfModified(bBroadcast);
	}
	else
	{
		if (bBroadcast && bSelectionsDiffer)
		{
			OnSelectionChanged.Broadcast(bEmitChange, (uint32)(ESelectionChangeTypeFlag::UnsetSelectionChanged));
		}
	}

	if (bSelectionsDiffer)
	{
		if (HighlightOptions.bAutoUpdateApplied)
		{
			RebuildAppliedPreviewHighlight();
		}
	}
}

void UUVToolSelectionAPI::ClearUnsetElementAppliedMeshSelections(bool bBroadcast, bool bEmitChange)
{
	SetUnsetElementAppliedMeshSelections(TArray<FUVToolSelection>(), bBroadcast, bEmitChange);
}

void UUVToolSelectionAPI::SetHighlightVisible(bool bUnwrapHighlightVisible, bool bAppliedHighlightVisible, bool bRebuild)
{
	HighlightMechanic->SetIsVisible(bUnwrapHighlightVisible, bAppliedHighlightVisible);
	if (bRebuild)
	{
		ClearHighlight(!bUnwrapHighlightVisible, !bAppliedHighlightVisible);
		if (bUnwrapHighlightVisible)
		{
			FTransform UnwrapTransform = FTransform::Identity;
			if (HighlightOptions.bUseCentroidForUnwrapAutoUpdate)
			{
				UnwrapTransform = FTransform(GetUnwrapSelectionBoundingBoxCenter());
			}
			RebuildUnwrapHighlight(UnwrapTransform);
		}
		if (bAppliedHighlightVisible)
		{
			RebuildAppliedPreviewHighlight();
		}
	}
}

void UUVToolSelectionAPI::SetHighlightOptions(const FHighlightOptions& Options)
{
	HighlightOptions = Options;

	HighlightMechanic->SetEnablePairedEdgeHighlights(Options.bShowPairedEdgeHighlights);
}

void UUVToolSelectionAPI::ClearHighlight(bool bClearForUnwrap, bool bClearForAppliedPreview)
{
	if (bClearForUnwrap)
	{
		HighlightMechanic->RebuildUnwrapHighlight(TArray<FUVToolSelection>(), FTransform::Identity);
	}
	if (bClearForAppliedPreview)
	{
		HighlightMechanic->RebuildAppliedHighlightFromUnwrapSelection(TArray<FUVToolSelection>());
	}
}

void UUVToolSelectionAPI::RebuildUnwrapHighlight(const FTransform& StartTransform)
{
	// Unfortunately even when tids and vids correspond between unwrap and canonical unwraps, eids
	// may differ, so we need to convert these over to preview unwrap eids if we're using previews.
	if (HighlightOptions.bBaseHighlightOnPreviews 
		&& GetSelectionsType() == FUVToolSelection::EType::Edge)
	{
		// TODO: We should probably have some function like "GetSelectionsInPreview(bool bForceRebuild)" 
		// that is publicly available and does caching.
		TArray<FUVToolSelection> ConvertedSelections = CurrentSelections;
		for (FUVToolSelection& ConvertedSelection : ConvertedSelections)
		{
			ConvertedSelection.SaveStableEdgeIdentifiers(*ConvertedSelection.Target->UnwrapCanonical);
			ConvertedSelection.RestoreFromStableEdgeIdentifiers(*ConvertedSelection.Target->UnwrapPreview->PreviewMesh->GetMesh());
		}
		HighlightMechanic->RebuildUnwrapHighlight(ConvertedSelections, StartTransform,
			HighlightOptions.bBaseHighlightOnPreviews);
	}
	else
	{
		HighlightMechanic->RebuildUnwrapHighlight(CurrentSelections, StartTransform,
			HighlightOptions.bBaseHighlightOnPreviews);
	}
}

void UUVToolSelectionAPI::SetUnwrapHighlightTransform(const FTransform& NewTransform)
{
	HighlightMechanic->SetUnwrapHighlightTransform(NewTransform, 
		HighlightOptions.bShowPairedEdgeHighlights, HighlightOptions.bBaseHighlightOnPreviews);
}

FTransform UUVToolSelectionAPI::GetUnwrapHighlightTransform()
{
	return HighlightMechanic->GetUnwrapHighlightTransform();
}

void UUVToolSelectionAPI::RebuildAppliedPreviewHighlight()
{
	// Unfortunately even when tids and vids correspond between unwrap and canonical unwraps, eids
	// may differ, so we need to convert these over to preview unwrap eids if we're using previews.
	if (HighlightOptions.bBaseHighlightOnPreviews
		&& GetSelectionsType() == FUVToolSelection::EType::Edge)
	{
		TArray<FUVToolSelection> ConvertedSelections = CurrentSelections;
		for (FUVToolSelection& ConvertedSelection : ConvertedSelections)
		{
			ConvertedSelection.SaveStableEdgeIdentifiers(*ConvertedSelection.Target->UnwrapCanonical);
			ConvertedSelection.RestoreFromStableEdgeIdentifiers(*ConvertedSelection.Target->UnwrapPreview->PreviewMesh->GetMesh());
		}
		HighlightMechanic->RebuildAppliedHighlightFromUnwrapSelection(ConvertedSelections,
			HighlightOptions.bBaseHighlightOnPreviews);
	}
	else
	{
		HighlightMechanic->RebuildAppliedHighlightFromUnwrapSelection(CurrentSelections,
			HighlightOptions.bBaseHighlightOnPreviews);
	}
	HighlightMechanic->AppendAppliedHighlight(CurrentUnsetSelections,
		HighlightOptions.bBaseHighlightOnPreviews);
}

void UUVToolSelectionAPI::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionMechanic)
	{
		SelectionMechanic->Render(RenderAPI);
	}
}
void UUVToolSelectionAPI::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionMechanic)
	{
		SelectionMechanic->DrawHUD(Canvas, RenderAPI);
	}
}
void UUVToolSelectionAPI::LivePreviewRender(IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionMechanic)
	{
		SelectionMechanic->LivePreviewRender(RenderAPI);
	}
}
void UUVToolSelectionAPI::LivePreviewDrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionMechanic)
	{
		SelectionMechanic->LivePreviewDrawHUD(Canvas, RenderAPI);
	}
}

void UUVToolSelectionAPI::BeginChange()
{
	PendingSelectionChange = MakeUnique<FSelectionChange>();
	PendingUnsetSelectionChange = MakeUnique<FUnsetSelectionChange>();
	FSelectionChange* CastSelectionChange = static_cast<FSelectionChange*>(PendingSelectionChange.Get());
	CastSelectionChange->SetBefore(GetSelections());
	FUnsetSelectionChange* CastUnsetSelectionChange = static_cast<FUnsetSelectionChange*>(PendingUnsetSelectionChange.Get());
	CastUnsetSelectionChange->SetBefore(GetUnsetElementAppliedMeshSelections());

}

bool UUVToolSelectionAPI::EndChangeAndEmitIfModified(bool bBroadcast)
{
	using namespace UVToolSelectionAPILocals;

	if (!PendingSelectionChange && !PendingUnsetSelectionChange)
	{
		return false;
	}

	FSelectionChange* CastSelectionChange = static_cast<FSelectionChange*>(PendingSelectionChange.Get());
	FUnsetSelectionChange* CastUnsetSelectionChange = static_cast<FUnsetSelectionChange*>(PendingUnsetSelectionChange.Get());

	bool bSelectionDiffer = true;
	bool bUnsetSelectionDiffer = true;
	// See if the selection has changed
	if (!DoSelectionSetsDiffer(CastSelectionChange->GetBefore(), GetSelections()))
	{
		PendingSelectionChange.Reset();		
		bSelectionDiffer = false;
	}
	if (!DoSelectionSetsDiffer(CastUnsetSelectionChange->GetBefore(), GetUnsetElementAppliedMeshSelections()))
	{
		PendingUnsetSelectionChange.Reset();
		bUnsetSelectionDiffer = false;
	}
	if (!bSelectionDiffer && !bUnsetSelectionDiffer)
	{
		return false;
	}

	EmitChangeAPI->BeginUndoTransaction(SelectionChangeTransactionName);
	if (bBroadcast && (bSelectionDiffer || bUnsetSelectionDiffer))
	{
		OnPreSelectionChange.Broadcast(true, (uint32)((bSelectionDiffer ? ESelectionChangeTypeFlag::SelectionChanged : ESelectionChangeTypeFlag::None) |
			                                          (bUnsetSelectionDiffer ? ESelectionChangeTypeFlag::UnsetSelectionChanged : ESelectionChangeTypeFlag::None)));
	}
	if (bSelectionDiffer)
	{
		CastSelectionChange->SetAfter(GetSelections());
		EmitChangeAPI->EmitToolIndependentChange(this, MoveTemp(PendingSelectionChange),
			SelectionChangeTransactionName);
		PendingSelectionChange.Reset();
	}
	if (bUnsetSelectionDiffer)
	{
		CastUnsetSelectionChange->SetAfter(GetUnsetElementAppliedMeshSelections());
		EmitChangeAPI->EmitToolIndependentChange(this, MoveTemp(PendingUnsetSelectionChange),
			SelectionChangeTransactionName);
		PendingUnsetSelectionChange.Reset();
	}

	if (bBroadcast && (bSelectionDiffer || bUnsetSelectionDiffer))
	{
		OnSelectionChanged.Broadcast(true, (uint32)((bSelectionDiffer ? ESelectionChangeTypeFlag::SelectionChanged : ESelectionChangeTypeFlag::None) |
													(bUnsetSelectionDiffer ? ESelectionChangeTypeFlag::UnsetSelectionChanged : ESelectionChangeTypeFlag::None)));
	}
	EmitChangeAPI->EndUndoTransaction();

	return true;
}

void UUVToolSelectionAPI::FSelectionChange::SetBefore(TArray<FUVToolSelection> SelectionsIn)
{
	Before = MoveTemp(SelectionsIn);
	if (bUseStableUnwrapCanonicalIDsForEdges)
	{
		for (FUVToolSelection& Selection : Before)
		{
			if (Selection.Type == FUVToolSelection::EType::Edge
				&& Selection.Target.IsValid()
				&& Selection.Target->UnwrapCanonical)
			{
				Selection.SaveStableEdgeIdentifiers(*Selection.Target->UnwrapCanonical);
				Selection.SelectedIDs.Empty(); // Don't need to store since we'll always restore
			}
		}
	}
}

void UUVToolSelectionAPI::FSelectionChange::SetAfter(TArray<FUVToolSelection> SelectionsIn)
{
	After = MoveTemp(SelectionsIn);
	if (bUseStableUnwrapCanonicalIDsForEdges)
	{
		for (FUVToolSelection& Selection : After)
		{
			if (Selection.Type == FUVToolSelection::EType::Edge
				&& Selection.Target.IsValid()
				&& Selection.Target->UnwrapCanonical)
			{
				Selection.SaveStableEdgeIdentifiers(*Selection.Target->UnwrapCanonical);
				Selection.SelectedIDs.Empty(); // Don't need to store since we'll always restore
			}
		}
	}
}

TArray<FUVToolSelection> UUVToolSelectionAPI::FSelectionChange::GetBefore() const
{
	TArray<FUVToolSelection> SelectionsOut;
	for (const FUVToolSelection& Selection : Before)
	{
		if (bUseStableUnwrapCanonicalIDsForEdges
			&& Selection.Type == FUVToolSelection::EType::Edge
			&& Selection.Target.IsValid()
			&& Selection.Target->UnwrapCanonical)
		{
			FUVToolSelection UnpackedSelection = Selection;
			UnpackedSelection.RestoreFromStableEdgeIdentifiers(*Selection.Target->UnwrapCanonical);
			SelectionsOut.Add(UnpackedSelection);
		}
		else
		{
			SelectionsOut.Add(Selection);
		}
	}
	return SelectionsOut;
}

bool UUVToolSelectionAPI::FSelectionChange::HasExpired(UObject* Object) const
{
	UUVToolSelectionAPI* SelectionAPI = Cast<UUVToolSelectionAPI>(Object);
	return !(SelectionAPI && SelectionAPI->IsActive());
}

void UUVToolSelectionAPI::FSelectionChange::Apply(UObject* Object) 
{
	UUVToolSelectionAPI* SelectionAPI = Cast<UUVToolSelectionAPI>(Object);
	if (SelectionAPI)
	{
		if (bUseStableUnwrapCanonicalIDsForEdges)
		{
			for (FUVToolSelection& Selection : After)
			{
				if (Selection.Type == FUVToolSelection::EType::Edge
					&& Selection.Target.IsValid()
					&& Selection.Target->UnwrapCanonical)
				{
					Selection.RestoreFromStableEdgeIdentifiers(*Selection.Target->UnwrapCanonical);
				}
			}
		}
		SelectionAPI->SetSelections(After, true, false);
	}
}

void UUVToolSelectionAPI::FSelectionChange::Revert(UObject* Object)
{
	UUVToolSelectionAPI* SelectionAPI = Cast<UUVToolSelectionAPI>(Object);
	if (SelectionAPI)
	{
		if (bUseStableUnwrapCanonicalIDsForEdges)
		{
			for (FUVToolSelection& Selection : Before)
			{
				if (Selection.Type == FUVToolSelection::EType::Edge
					&& Selection.Target.IsValid()
					&& Selection.Target->UnwrapCanonical)
				{
					Selection.RestoreFromStableEdgeIdentifiers(*Selection.Target->UnwrapCanonical);
				}
			}
		}
		SelectionAPI->SetSelections(Before, true, false);
	}
}

FString UUVToolSelectionAPI::FSelectionChange::ToString() const
{
	return TEXT("UUVToolSelectionAPI::FSelectionChange");
}

void UUVToolSelectionAPI::FUnsetSelectionChange::SetBefore(TArray<FUVToolSelection> SelectionsIn)
{
	Before = MoveTemp(SelectionsIn);
}

void UUVToolSelectionAPI::FUnsetSelectionChange::SetAfter(TArray<FUVToolSelection> SelectionsIn)
{
	After = MoveTemp(SelectionsIn);
}

const TArray<FUVToolSelection>& UUVToolSelectionAPI::FUnsetSelectionChange::GetBefore() const
{
	return Before;
}

bool UUVToolSelectionAPI::FUnsetSelectionChange::HasExpired(UObject* Object) const
{
	UUVToolSelectionAPI* SelectionAPI = Cast<UUVToolSelectionAPI>(Object);
	return !(SelectionAPI && SelectionAPI->IsActive());
}

void UUVToolSelectionAPI::FUnsetSelectionChange::Apply(UObject* Object)
{
	UUVToolSelectionAPI* SelectionAPI = Cast<UUVToolSelectionAPI>(Object);
	if (SelectionAPI)
	{
		SelectionAPI->SetUnsetElementAppliedMeshSelections(After, true, false);
	}
}

void UUVToolSelectionAPI::FUnsetSelectionChange::Revert(UObject* Object)
{
	UUVToolSelectionAPI* SelectionAPI = Cast<UUVToolSelectionAPI>(Object);
	if (SelectionAPI)
	{
		SelectionAPI->SetUnsetElementAppliedMeshSelections(Before, true, false);
	}
}

FString UUVToolSelectionAPI::FUnsetSelectionChange::ToString() const
{
	return TEXT("UUVToolSelectionAPI::FUnsetSelectionChange");
}

#undef LOCTEXT_NAMESPACE