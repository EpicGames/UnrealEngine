// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/UVToolSelectionHighlightMechanic.h"

#include "Actions/UVSeamSewAction.h"
#include "Drawing/LineSetComponent.h"
#include "Drawing/BasicPointSetComponent.h"
#include "Drawing/BasicLineSetComponent.h"
#include "Drawing/BasicTriangleSetComponent.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/TriangleSetComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/World.h"
#include "MeshOpPreviewHelpers.h" // for the preview meshes
#include "Selection/UVToolSelection.h"
#include "ToolSetupUtil.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "UVEditorUXSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVToolSelectionHighlightMechanic)

#define LOCTEXT_NAMESPACE "UUVToolSelectionHighlightMechanic"

using namespace UE::Geometry;

namespace UVToolSelectionHighlightMechanicLocals
{
	FVector2f ToFVector2f(const FVector& VectorIn)
    {
         return FVector2f( static_cast<float>(VectorIn.X), static_cast<float>(VectorIn.Y) );
    }

	FVector3f ToFVector3f(const FVector3d& VectorIn)
    {
         return FVector3f( static_cast<float>(VectorIn.X), static_cast<float>(VectorIn.Y), static_cast<float>(VectorIn.Z));
    }
}

void UUVToolSelectionHighlightMechanic::Initialize(UWorld* UnwrapWorld, UWorld* LivePreviewWorld)
{
	// Initialize shouldn't be called more than once...
	if (!ensure(!UnwrapGeometryActor))
	{
		UnwrapGeometryActor->Destroy();
	}
	if (!ensure(!LivePreviewGeometryActor))
	{
		LivePreviewGeometryActor->Destroy();
	}

	// Owns most of the unwrap geometry except for the unselected paired edges, since we don't
	// want those to move if we change the actor transform via SetUnwrapHighlightTransform
	UnwrapGeometryActor = UnwrapWorld->SpawnActor<APreviewGeometryActor>(
		FVector::ZeroVector, FRotator(0, 0, 0), FActorSpawnParameters());

	UnwrapTriangleSet = NewObject<UBasic2DTriangleSetComponent>(UnwrapGeometryActor);
	// We are setting the TranslucencySortPriority here to handle the UV editor's use case in 2D
	// where multiple translucent layers are drawn on top of each other but still need depth sorting.
	UnwrapTriangleSet->TranslucencySortPriority = static_cast<int32>(FUVEditorUXSettings::SelectionTriangleDepthBias);
	TriangleSetMaterial = ToolSetupUtil::GetCustomTwoSidedDepthOffsetMaterial(GetParentTool()->GetToolManager(),
		FUVEditorUXSettings::SelectionTriangleFillColor,
		FUVEditorUXSettings::SelectionTriangleDepthBias,
		FUVEditorUXSettings::SelectionTriangleOpacity);
	UnwrapTriangleSet->SetTriangleMaterial(TriangleSetMaterial);
	UnwrapTriangleSet->SetTriangleSetParameters(FUVEditorUXSettings::SelectionTriangleFillColor, FVector3f(0, 0, 1));
	UnwrapGeometryActor->SetRootComponent(UnwrapTriangleSet.Get());
	UnwrapTriangleSet->RegisterComponent();

	UnwrapLineSet = NewObject<UBasic2DLineSetComponent>(UnwrapGeometryActor);
	UnwrapLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetParentTool()->GetToolManager(), true));
	UnwrapLineSet->SetLineSetParameters(FUVEditorUXSettings::SelectionTriangleWireframeColor,
							     			 FUVEditorUXSettings::SelectionLineThickness,
										     FUVEditorUXSettings::SelectionWireframeDepthBias);
	UnwrapLineSet->AttachToComponent(UnwrapTriangleSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	UnwrapLineSet->RegisterComponent();

	UnwrapPairedEdgeLineSet = NewObject<UBasic2DLineSetComponent>(UnwrapGeometryActor);
	UnwrapPairedEdgeLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetParentTool()->GetToolManager(), true));
	UnwrapPairedEdgeLineSet->AttachToComponent(UnwrapTriangleSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	UnwrapPairedEdgeLineSet->RegisterComponent();

	SewEdgePairingLeftLineSet = NewObject<UBasic2DLineSetComponent>(UnwrapGeometryActor);
	SewEdgePairingLeftLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(
		GetParentTool()->GetToolManager(), /*bDepthTested*/ true));
	SewEdgePairingLeftLineSet->SetLineSetParameters(FUVEditorUXSettings::SewSideLeftColor,
		                                                 FUVEditorUXSettings::SewLineHighlightThickness,
		                                                 FUVEditorUXSettings::SewLineDepthOffset);
	SewEdgePairingLeftLineSet->AttachToComponent(UnwrapTriangleSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	SewEdgePairingLeftLineSet->RegisterComponent();
	SewEdgePairingLeftLineSet->SetVisibility(bPairedEdgeHighlightsEnabled);

	SewEdgePairingRightLineSet = NewObject<UBasic2DLineSetComponent>(UnwrapGeometryActor);
	SewEdgePairingRightLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(
		GetParentTool()->GetToolManager(), /*bDepthTested*/ true));
	SewEdgePairingRightLineSet->SetLineSetParameters(FUVEditorUXSettings::SewSideRightColor,
		                                                  FUVEditorUXSettings::SewLineHighlightThickness,
		                                                  FUVEditorUXSettings::SewLineDepthOffset);
	SewEdgePairingRightLineSet->AttachToComponent(UnwrapTriangleSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	SewEdgePairingRightLineSet->RegisterComponent();
	SewEdgePairingRightLineSet->SetVisibility(bPairedEdgeHighlightsEnabled);

	// The unselected paired edges get their own, stationary, actor.
	UnwrapStationaryGeometryActor = UnwrapWorld->SpawnActor<APreviewGeometryActor>(
		FVector::ZeroVector, FRotator(0, 0, 0), FActorSpawnParameters());
	SewEdgeUnselectedPairingLineSet = NewObject<UBasic2DLineSetComponent>(UnwrapStationaryGeometryActor);
	SewEdgeUnselectedPairingLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(
		GetParentTool()->GetToolManager(), /*bDepthTested*/ true));
	SewEdgeUnselectedPairingLineSet->SetLineSetParameters(FUVEditorUXSettings::SewSideRightColor,
		                                                       FUVEditorUXSettings::SewLineHighlightThickness,
		                                                       FUVEditorUXSettings::SewLineDepthOffset);
	UnwrapStationaryGeometryActor->SetRootComponent(SewEdgeUnselectedPairingLineSet.Get());
	SewEdgeUnselectedPairingLineSet->RegisterComponent();
	SewEdgeUnselectedPairingLineSet->SetVisibility(bPairedEdgeHighlightsEnabled);

	UnwrapPointSet = NewObject<UBasic2DPointSetComponent>(UnwrapGeometryActor);
	UnwrapPointSet->SetPointMaterial(ToolSetupUtil::GetDefaultPointComponentMaterial(GetParentTool()->GetToolManager(), true));
	UnwrapPointSet->SetPointSetParameters(FUVEditorUXSettings::SelectionTriangleWireframeColor,
											   FUVEditorUXSettings::SelectionPointThickness,
											   FUVEditorUXSettings::SelectionWireframeDepthBias);
	UnwrapPointSet->AttachToComponent(UnwrapTriangleSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	UnwrapPointSet->RegisterComponent();

	// Owns the highlights in the live preview.
	LivePreviewGeometryActor = LivePreviewWorld->SpawnActor<APreviewGeometryActor>(
		FVector::ZeroVector, FRotator(0, 0, 0), FActorSpawnParameters());

	LivePreviewLineSet = NewObject<UBasic3DLineSetComponent>(LivePreviewGeometryActor);
	LivePreviewLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(
		GetParentTool()->GetToolManager(), /*bDepthTested*/ true));
	LivePreviewLineSet->SetLineSetParameters(FUVEditorUXSettings::SelectionTriangleWireframeColor,
	                                               FUVEditorUXSettings::LivePreviewHighlightThickness,
		                                           FUVEditorUXSettings::LivePreviewHighlightDepthOffset);
	LivePreviewGeometryActor->SetRootComponent(LivePreviewLineSet.Get());
	LivePreviewLineSet->RegisterComponent();

	LivePreviewPointSet = NewObject<UBasic3DPointSetComponent>(LivePreviewGeometryActor);
	LivePreviewPointSet->SetPointMaterial(ToolSetupUtil::GetDefaultPointComponentMaterial(
		GetParentTool()->GetToolManager(), /*bDepthTested*/ true));
	LivePreviewPointSet->SetPointSetParameters(FUVEditorUXSettings::SelectionTriangleWireframeColor,
													FUVEditorUXSettings::LivePreviewHighlightPointSize,
													FUVEditorUXSettings::LivePreviewHighlightDepthOffset);
	LivePreviewPointSet->AttachToComponent(LivePreviewLineSet.Get(), FAttachmentTransformRules::KeepWorldTransform);
	LivePreviewPointSet->RegisterComponent();
}

void UUVToolSelectionHighlightMechanic::Shutdown()
{
	if (UnwrapGeometryActor)
	{
		UnwrapGeometryActor->Destroy();
		UnwrapGeometryActor = nullptr;
	}
	if (UnwrapStationaryGeometryActor)
	{
		UnwrapStationaryGeometryActor->Destroy();
		UnwrapStationaryGeometryActor = nullptr;
	}
	if (LivePreviewGeometryActor)
	{
		LivePreviewGeometryActor->Destroy();
		LivePreviewGeometryActor = nullptr;
	}

	TriangleSetMaterial = nullptr;
}

void UUVToolSelectionHighlightMechanic::SetIsVisible(bool bUnwrapHighlightVisible, bool bLivePreviewHighlightVisible)
{
	if (UnwrapGeometryActor)
	{
		UnwrapGeometryActor->SetActorHiddenInGame(!bUnwrapHighlightVisible);
	}
	if (UnwrapStationaryGeometryActor)
	{
		UnwrapStationaryGeometryActor->SetActorHiddenInGame(!bUnwrapHighlightVisible);
	}
	if (LivePreviewGeometryActor)
	{
		LivePreviewGeometryActor->SetActorHiddenInGame(!bLivePreviewHighlightVisible);
	}
}

void UUVToolSelectionHighlightMechanic::RebuildUnwrapHighlight(
	const TArray<FUVToolSelection>& Selections, const FTransform& StartTransform, 
	bool bUsePreviews)
{
    using namespace UVToolSelectionHighlightMechanicLocals;

	if (!ensure(UnwrapGeometryActor))
	{
		return;
	}

	UnwrapTriangleSet->Clear();
	UnwrapLineSet->Clear();
	UnwrapPointSet->Clear();
	SewEdgePairingRightLineSet->Clear();
	SewEdgePairingLeftLineSet->Clear();
	SewEdgeUnselectedPairingLineSet->Clear();
	StaticPairedEdgeVidsPerMesh.Reset();

	UnwrapGeometryActor->SetActorTransform(StartTransform);

	for (const FUVToolSelection& Selection : Selections)
	{
		if (!ensure(Selection.Target.IsValid() && Selection.Target->AreMeshesValid()))
		{
			return;
		}

		const FDynamicMesh3& Mesh = bUsePreviews ? *Selection.Target->UnwrapPreview->PreviewMesh->GetMesh()
			: *Selection.Target->UnwrapCanonical;

		if (Selection.Type == FUVToolSelection::EType::Triangle)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UUVToolSelectionHighlightMechanic::AppendUnwrapHighlight_Triangle);

			UBasic2DTriangleSetComponent* UnwrapTriangleSetPtr = UnwrapTriangleSet.Get();
			UBasic2DLineSetComponent* UnwrapLineSetPtr = UnwrapLineSet.Get();
			UnwrapTriangleSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());
			UnwrapLineSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num() * 3);
			for (int32 Tid : Selection.SelectedIDs)
			{
				if (!ensure(Mesh.IsTriangle(Tid)))
				{
					continue;
				}
				
				FVector Points[3];
				Mesh.GetTriVertices(Tid, Points[0], Points[1], Points[2]);
				for (int i = 0; i < 3; ++i)
				{
					Points[i] = StartTransform.InverseTransformPosition(Points[i]);
				}
				UnwrapTriangleSetPtr->AddElement(ToFVector2f(Points[0]),ToFVector2f(Points[1]),ToFVector2f(Points[2]));
					
				for (int i = 0; i < 3; ++i)
				{
					int NextIndex = (i + 1) % 3;
					UnwrapLineSetPtr->AddElement(ToFVector2f(Points[i]),ToFVector2f(Points[NextIndex]));
				}
			}
			UnwrapTriangleSetPtr->MarkRenderStateDirty();
			UnwrapLineSetPtr->MarkRenderStateDirty();
		}
		else if (Selection.Type == FUVToolSelection::EType::Edge)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_RebuildDrawnElements_Edge);

			StaticPairedEdgeVidsPerMesh.Emplace();
			StaticPairedEdgeVidsPerMesh.Last().Key = Selection.Target;

			const FDynamicMesh3& AppliedMesh = bUsePreviews ? *Selection.Target->AppliedPreview->PreviewMesh->GetMesh()
				: *Selection.Target->AppliedCanonical;

			UBasic2DLineSetComponent* UnwrapLineSetPtr = UnwrapLineSet.Get();
			UBasic2DLineSetComponent* SewEdgePairingLeftLineSetPtr = SewEdgePairingLeftLineSet.Get();
			UBasic2DLineSetComponent* SewEdgePairingRightLineSetPtr = SewEdgePairingRightLineSet.Get();
			UBasic2DLineSetComponent* SewEdgeUnselectedPairingLineSetPtr = SewEdgeUnselectedPairingLineSet.Get();

			UnwrapLineSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());
			SewEdgePairingLeftLineSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());
			SewEdgePairingRightLineSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());
			SewEdgeUnselectedPairingLineSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());
			for (int32 Eid : Selection.SelectedIDs)
			{
				if (!ensure(Mesh.IsEdge(Eid)))
				{
					continue;
				}
				
				FVector Points[2];
				Mesh.GetEdgeV(Eid, Points[0], Points[1]);
				Points[0] = StartTransform.InverseTransformPosition(Points[0]);
				Points[1] = StartTransform.InverseTransformPosition(Points[1]);
				UnwrapLineSetPtr->AddElement(ToFVector2f(Points[0]),ToFVector2f(Points[1]));

				if (bPairedEdgeHighlightsEnabled)
				{
					bool bWouldPreferReverse = false;
					int32 PairedEid = UUVSeamSewAction::FindSewEdgeOppositePairing(
						Mesh, AppliedMesh, Selection.Target->UVLayerIndex, Eid, bWouldPreferReverse);

					bool bPairedEdgeIsSelected = Selection.SelectedIDs.Contains(PairedEid);

					if (PairedEid == IndexConstants::InvalidID
						// When both sides are selected, merge order depends on adjacent tid values, so
						// deal with the pair starting with the other edge.
						|| (bPairedEdgeIsSelected && bWouldPreferReverse))
					{
						continue;
					}

					Mesh.GetEdgeV(Eid, Points[0], Points[1]);
					Points[0] = StartTransform.InverseTransformPosition(Points[0]);
					Points[1] = StartTransform.InverseTransformPosition(Points[1]);
					SewEdgePairingLeftLineSetPtr->AddElement(ToFVector2f(Points[0]),ToFVector2f(Points[1]));

					// The paired edge may need to go into a separate line set if it is not selected so that it does
					// not get affected by transformations of the selected highlights in SetUnwrapHighlightTransform
					FIndex2i Vids2 = Mesh.GetEdgeV(PairedEid);
					Mesh.GetEdgeV(PairedEid, Points[0], Points[1]);
					if (bPairedEdgeIsSelected)
					{
						Points[0] = StartTransform.InverseTransformPosition(Points[0]);
						Points[1] = StartTransform.InverseTransformPosition(Points[1]);
						SewEdgePairingRightLineSetPtr->AddElement(ToFVector2f(Points[0]),ToFVector2f(Points[1]));
					}
					else
					{
						StaticPairedEdgeVidsPerMesh.Last().Value.Add(TPair<int32, int32>(Vids2.A, Vids2.B));
						SewEdgeUnselectedPairingLineSetPtr->AddElement(ToFVector2f(Points[0]),
 							                                           ToFVector2f(Points[1]));
					}
				}//end if visualizing paired edges
			}//end for each edge
			UnwrapLineSetPtr->MarkRenderStateDirty();
			SewEdgePairingLeftLineSetPtr->MarkRenderStateDirty();
			SewEdgePairingRightLineSetPtr->MarkRenderStateDirty();
			SewEdgeUnselectedPairingLineSetPtr->MarkRenderStateDirty();
		}
		else if (Selection.Type == FUVToolSelection::EType::Vertex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshSelectionMechanic_RebuildDrawnElements_Vertex);

			UBasic2DPointSetComponent* UnwrapPointSetPtr = UnwrapPointSet.Get();
			UnwrapPointSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());

			for (const int32 Vid : Selection.SelectedIDs)
			{
				if (!ensure(Mesh.IsVertex(Vid)))
				{
					continue;
				}

				const FVector3d Position = StartTransform.InverseTransformPosition(Mesh.GetVertex(Vid));
				UnwrapPointSetPtr->AddElement(ToFVector2f(Position));
			}
			UnwrapPointSetPtr->MarkRenderStateDirty();
		}
	}
}

void UUVToolSelectionHighlightMechanic::SetUnwrapHighlightTransform(const FTransform& Transform, 
	bool bRebuildStaticPairedEdges, bool bUsePreviews)
{
    using namespace UVToolSelectionHighlightMechanicLocals;

	if (ensure(UnwrapGeometryActor))
	{
		UnwrapGeometryActor->SetActorTransform(Transform);
	}
	if (bPairedEdgeHighlightsEnabled && bRebuildStaticPairedEdges)
	{		
		int32 PairedEdgeCount = 0;
		for (const TPair<TWeakObjectPtr<UUVEditorToolMeshInput>,
			TArray<TPair<int32, int32>>>& MeshVidPairs : StaticPairedEdgeVidsPerMesh)
		{
			PairedEdgeCount += MeshVidPairs.Value.Num();
		}

		UBasic2DLineSetComponent* SewEdgeUnselectedPairingLineSetPtr = SewEdgeUnselectedPairingLineSet.Get();
		SewEdgeUnselectedPairingLineSetPtr->Clear();
		SewEdgeUnselectedPairingLineSetPtr->ReserveElements(PairedEdgeCount);

		for (const TPair<TWeakObjectPtr<UUVEditorToolMeshInput>, 
			TArray<TPair<int32, int32>>>& MeshVidPairs : StaticPairedEdgeVidsPerMesh)
		{
			TWeakObjectPtr<UUVEditorToolMeshInput> Target = MeshVidPairs.Key;
			if (!ensure(Target.IsValid()))
			{
				continue;
			}

			const FDynamicMesh3& Mesh = bUsePreviews ? *Target->UnwrapPreview->PreviewMesh->GetMesh()
				: *Target->UnwrapCanonical;

			for (const TPair<int32, int32>& VidPair : MeshVidPairs.Value)
			{
				if (!ensure(Mesh.IsVertex(VidPair.Key) && Mesh.IsVertex(VidPair.Value)))
				{
					continue;
				}
				FVector3d VertA = Mesh.GetVertex(VidPair.Key);
				FVector3d VertB = Mesh.GetVertex(VidPair.Value);
				SewEdgeUnselectedPairingLineSetPtr->AddElement(ToFVector2f(VertA),ToFVector2f(VertB));
			}
			SewEdgeUnselectedPairingLineSetPtr->MarkRenderStateDirty();
		}
	}
}

FTransform UUVToolSelectionHighlightMechanic::GetUnwrapHighlightTransform()
{
	if (ensure(UnwrapGeometryActor))
	{
		return UnwrapGeometryActor->GetActorTransform();
	}
	return FTransform::Identity;
}

void UUVToolSelectionHighlightMechanic::RebuildAppliedHighlightFromUnwrapSelection(
	const TArray<FUVToolSelection>& UnwrapSelections, bool bUsePreviews)
{
    using namespace UVToolSelectionHighlightMechanicLocals;
	if (!ensure(LivePreviewGeometryActor))
	{
		return;
	}

	UBasic3DLineSetComponent* LivePreviewLineSetPtr = LivePreviewLineSet.Get();
	UBasic3DPointSetComponent* LivePreviewPointSetPtr = LivePreviewPointSet.Get();

	LivePreviewLineSetPtr->Clear();
	LivePreviewPointSetPtr->Clear();

	for (const FUVToolSelection& Selection : UnwrapSelections)
	{
		if (!ensure(Selection.Target.IsValid() && Selection.Target->AreMeshesValid()))
		{
			return;
		}

		UUVEditorToolMeshInput* Target = Selection.Target.Get();

		const FDynamicMesh3& AppliedMesh = bUsePreviews ? *Target->AppliedPreview->PreviewMesh->GetMesh()
			: *Target->AppliedCanonical;
		const FDynamicMesh3& UnwrapMesh = bUsePreviews ? *Target->UnwrapPreview->PreviewMesh->GetMesh()
			: *Target->UnwrapCanonical;

		FTransform MeshTransform = Target->AppliedPreview->PreviewMesh->GetTransform();

		auto AppendEdgeLine = [this, &AppliedMesh, &MeshTransform, LivePreviewLineSetPtr](int32 AppliedEid)
		{
			FVector Points[2];
			AppliedMesh.GetEdgeV(AppliedEid, Points[0], Points[1]);
			Points[0] = MeshTransform.TransformPosition(Points[0]);
			Points[1] = MeshTransform.TransformPosition(Points[1]);
			LivePreviewLineSetPtr->AddElement(ToFVector3f(Points[0]),ToFVector3f(Points[1]));
		};

		if (Selection.Type == FUVToolSelection::EType::Triangle)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Triangle);

			LivePreviewLineSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num()*3);
			for (int32 Tid : Selection.SelectedIDs)
			{
				if (!ensure(AppliedMesh.IsTriangle(Tid)))
				{
					continue;
				}

				// Gather the boundary edges for the live preview
				FIndex3i TriEids = AppliedMesh.GetTriEdges(Tid);
				for (int i = 0; i < 3; ++i)
				{
					FIndex2i EdgeTids = AppliedMesh.GetEdgeT(TriEids[i]);
					for (int j = 0; j < 2; ++j)
					{
						if (EdgeTids[j] != Tid && !Selection.SelectedIDs.Contains(EdgeTids[j]))
						{
							AppendEdgeLine(TriEids[i]);
							break;
						}
					}
				}//end for tri edges
			}//end for selection tids
			LivePreviewLineSetPtr->MarkRenderStateDirty();
		}
		else if (Selection.Type == FUVToolSelection::EType::Edge)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Edge);
			LivePreviewLineSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());

			for (int32 UnwrapEid : Selection.SelectedIDs)
			{
				if (!ensure(UnwrapMesh.IsEdge(UnwrapEid)))
				{
					continue;
				}

				FDynamicMesh3::FEdge Edge = UnwrapMesh.GetEdge(UnwrapEid);

				int32 AppliedEid = AppliedMesh.FindEdgeFromTri(
					Target->UnwrapVidToAppliedVid(Edge.Vert.A),
					Target->UnwrapVidToAppliedVid(Edge.Vert.B),
					Edge.Tri.A);

				AppendEdgeLine(AppliedEid);
			}
			LivePreviewLineSetPtr->MarkRenderStateDirty();
		}
		else if (Selection.Type == FUVToolSelection::EType::Vertex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Vertex);

			LivePreviewPointSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());

			for (const int32 UnwrapVid : Selection.SelectedIDs)
			{
				LivePreviewPointSetPtr->AddElement(static_cast<FVector3f>(AppliedMesh.GetVertex(Target->UnwrapVidToAppliedVid(UnwrapVid))));
			}
			LivePreviewPointSetPtr->MarkRenderStateDirty();
		}
	}//end for selection
}

void UUVToolSelectionHighlightMechanic::AppendAppliedHighlight(const TArray<FUVToolSelection>& AppliedSelections, bool bUsePreviews)
{
    using namespace UVToolSelectionHighlightMechanicLocals;
	if (!ensure(LivePreviewGeometryActor))
	{
		return;
	}

	UBasic3DLineSetComponent* LivePreviewLineSetPtr = LivePreviewLineSet.Get();
	UBasic3DPointSetComponent* LivePreviewPointSetPtr = LivePreviewPointSet.Get();

	for (const FUVToolSelection& Selection : AppliedSelections)
	{
		if (!ensure(Selection.Target.IsValid() && Selection.Target->IsValid()))
		{
			return;
		}

		UUVEditorToolMeshInput* Target = Selection.Target.Get();

		const FDynamicMesh3& AppliedMesh = bUsePreviews ? *Target->AppliedPreview->PreviewMesh->GetMesh()
			: *Target->AppliedCanonical;

		FTransform MeshTransform = Target->AppliedPreview->PreviewMesh->GetTransform();

		auto AppendEdgeLine = [this, &AppliedMesh, &MeshTransform, LivePreviewLineSetPtr](int32 AppliedEid)
		{
			FVector Points[2];
			AppliedMesh.GetEdgeV(AppliedEid, Points[0], Points[1]);
			Points[0] = MeshTransform.TransformPosition(Points[0]);
			Points[1] = MeshTransform.TransformPosition(Points[1]);
			LivePreviewLineSetPtr->AddElement(ToFVector3f(Points[0]),ToFVector3f(Points[1]));
		};

		if (Selection.Type == FUVToolSelection::EType::Triangle)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Triangle);

			LivePreviewLineSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num() * 3);
			for (int32 Tid : Selection.SelectedIDs)
			{
				if (!ensure(AppliedMesh.IsTriangle(Tid)))
				{
					continue;
				}

				// Gather the boundary edges for the live preview
				FIndex3i TriEids = AppliedMesh.GetTriEdges(Tid);
				for (int i = 0; i < 3; ++i)
				{
					FIndex2i EdgeTids = AppliedMesh.GetEdgeT(TriEids[i]);
					for (int j = 0; j < 2; ++j)
					{
						if (EdgeTids[j] != Tid && !Selection.SelectedIDs.Contains(EdgeTids[j]))
						{
							AppendEdgeLine(TriEids[i]);
							break;
						}
					}
				}//end for tri edges
			}//end for selection tids
			LivePreviewLineSetPtr->MarkRenderStateDirty();
		}
		else if (Selection.Type == FUVToolSelection::EType::Edge)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Edge);

			LivePreviewLineSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());
			for (int32 Eid : Selection.SelectedIDs)
			{
				if (!ensure(AppliedMesh.IsEdge(Eid)))
				{
					continue;
				}

				AppendEdgeLine(Eid);
			}
			LivePreviewLineSetPtr->MarkRenderStateDirty();
		}
		else if (Selection.Type == FUVToolSelection::EType::Vertex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Vertex);

			LivePreviewPointSetPtr->ReserveAdditionalElements(Selection.SelectedIDs.Num());

			for (const int32 Vid : Selection.SelectedIDs)
			{
				LivePreviewPointSetPtr->AddElement(static_cast<FVector3f>(AppliedMesh.GetVertex(Vid)));
			}
			LivePreviewPointSetPtr->MarkRenderStateDirty();
		}
	}//end for selection
}


void UUVToolSelectionHighlightMechanic::SetEnablePairedEdgeHighlights(bool bEnable)
{
	bPairedEdgeHighlightsEnabled = bEnable;
	SewEdgePairingLeftLineSet->SetVisibility(bEnable);
	SewEdgePairingRightLineSet->SetVisibility(bEnable);
	SewEdgeUnselectedPairingLineSet->SetVisibility(bEnable);
}

void UUVToolSelectionHighlightMechanic::SetColor(const FColor Color) const
{
	LivePreviewLineSet->SetColor(Color);
	LivePreviewPointSet->SetColor(Color);
}

void UUVToolSelectionHighlightMechanic::SetLineThickness(const float LineSize) const
{
	LivePreviewLineSet->SetLineThickness(LineSize);
}

void UUVToolSelectionHighlightMechanic::SetPointSize(const float PointSize) const
{
	LivePreviewPointSet->SetPointSize(PointSize);
}

#undef LOCTEXT_NAMESPACE
