// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/UVEditorToolMeshInput.h"

#include "Drawing/MeshElementsVisualizer.h" // for wireframe display
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h" // FDynamicMeshChange
#include "DynamicMesh/MeshIndexUtil.h"
#include "MeshOpPreviewHelpers.h"
#include "ModelingToolTargetUtil.h"
#include "Parameterization/UVUnwrapMeshUtil.h"
#include "TargetInterfaces/AssetBackedTarget.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "ToolSetupUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorToolMeshInput)

using namespace UE::Geometry;

namespace UVEditorToolMeshInputLocals
{
	const TArray<int32> EmptyArray;

	void NotifyUnwrapPreviewDeferredEditCompleted(UUVEditorToolMeshInput* InputObject,
		const TArray<int32>* ChangedVids, const TArray<int32>* ChangedConnectivityTids, 
		const TArray<int32>* FastRenderUpdateTids, EMeshRenderAttributeFlags ChangedAttribs)
	{
		// Currently there's no way to do a partial or region update if the connectivity changed in any way.
		if (ChangedConnectivityTids == nullptr || !ChangedConnectivityTids->IsEmpty())
		{
			InputObject->UnwrapPreview->PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FullUpdate, ChangedAttribs, true);
		}
		else if (FastRenderUpdateTids) 
		{
			InputObject->UnwrapPreview->PreviewMesh->NotifyRegionDeferredEditCompleted(*FastRenderUpdateTids, ChangedAttribs);
		}
		else
		{
			InputObject->UnwrapPreview->PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, ChangedAttribs, true);
		}
	}

	void NotifyAppliedPreviewDeferredEditCompleted(UUVEditorToolMeshInput* InputObject,
		const TArray<int32>* ChangedVids, const TArray<int32>* FastRenderUpdateTids, EMeshRenderAttributeFlags ChangedAttribs)
	{
		if (FastRenderUpdateTids) 
		{
			InputObject->AppliedPreview->PreviewMesh->NotifyRegionDeferredEditCompleted(*FastRenderUpdateTids, ChangedAttribs);
		}
		else
		{
			InputObject->AppliedPreview->PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, ChangedAttribs, false);
		}
	}
}

const TArray<int32>* const UUVEditorToolMeshInput::NONE_CHANGED_ARG = &UVEditorToolMeshInputLocals::EmptyArray;

bool UUVEditorToolMeshInput::AreMeshesValid() const
{
	return UnwrapPreview && UnwrapPreview->IsValidLowLevel()
		&& AppliedPreview && AppliedPreview->IsValidLowLevel()
		&& UnwrapCanonical && AppliedCanonical;
}

bool UUVEditorToolMeshInput::IsToolTargetValid() const
{
	return SourceTarget && SourceTarget->IsValid();
}

bool UUVEditorToolMeshInput::IsValid() const
{
	return AreMeshesValid()
		&& IsToolTargetValid()
		&& UVLayerIndex >= 0;
}

bool UUVEditorToolMeshInput::InitializeMeshes(UToolTarget* Target, 
	TSharedPtr<FDynamicMesh3> AppliedCanonicalIn, UMeshOpPreviewWithBackgroundCompute* AppliedPreviewIn,
	int32 AssetIDIn, int32 UVLayerIndexIn, UWorld* UnwrapWorld, UWorld* LivePreviewWorld,
	UMaterialInterface* WorkingMaterialIn,
	TFunction<FVector3d(const FVector2f&)> UVToVertPositionFuncIn,
	TFunction<FVector2f(const FVector3d&)> VertPositionToUVFuncIn)
{
	// TODO: The ModelingToolTargetUtil.h doesn't currently have all the proper functions we want
	// to access the tool target (for instance, to get a dynamic mesh without a mesh description).
	// We'll need to update this function once they exist.
	using namespace UE::ToolTarget;

	SourceTarget = Target;
	AssetID = AssetIDIn;
	UVLayerIndex = UVLayerIndexIn;
	UVToVertPosition = UVToVertPositionFuncIn;
	VertPositionToUV = VertPositionToUVFuncIn;

	// We are given the preview- i.e. the mesh with the uv layer applied.
	AppliedCanonical = AppliedCanonicalIn;

	if (!AppliedCanonical->HasAttributes()
		|| UVLayerIndex >= AppliedCanonical->Attributes()->NumUVLayers())
	{
		return false;
	}

	AppliedPreview = AppliedPreviewIn;

	// Set up the unwrapped mesh
	UnwrapCanonical = MakeShared<FDynamicMesh3>();
	UVUnwrapMeshUtil::GenerateUVUnwrapMesh(
		*AppliedCanonical->Attributes()->GetUVLayer(UVLayerIndex),
		*UnwrapCanonical, UVToVertPosition);
	UnwrapCanonical->SetShapeChangeStampEnabled(true);

	// Set up the unwrap preview
	UnwrapPreview = NewObject<UMeshOpPreviewWithBackgroundCompute>();
	UnwrapPreview->Setup(UnwrapWorld);
	UnwrapPreview->PreviewMesh->UpdatePreview(UnwrapCanonical.Get());

	return true;
}


void UUVEditorToolMeshInput::Shutdown()
{
	if (WireframeDisplay)
	{
		WireframeDisplay->Disconnect();
		WireframeDisplay = nullptr;
	}

	UnwrapCanonical = nullptr;
	UnwrapPreview->Shutdown();
	UnwrapPreview = nullptr;
	AppliedCanonical = nullptr;
	// Can't shut down AppliedPreview because it is owned by mode
	AppliedPreview = nullptr;

	SourceTarget = nullptr;

	OnCanonicalModified.Clear();
}

void UUVEditorToolMeshInput::UpdateUnwrapPreviewOverlayFromPositions(const TArray<int32>* ChangedVids, 
	const TArray<int32>* ChangedConnectivityTids, const TArray<int32>* FastRenderUpdateTids)
{
	using namespace UVEditorToolMeshInputLocals;

	UnwrapPreview->PreviewMesh->DeferredEditMesh([this, ChangedVids, ChangedConnectivityTids](FDynamicMesh3& Mesh)
	{
		FDynamicMeshUVOverlay* DestOverlay = Mesh.Attributes()->PrimaryUV();
		UVUnwrapMeshUtil::UpdateUVOverlayFromUnwrapMesh(Mesh, *DestOverlay, VertPositionToUV,
			ChangedVids, ChangedConnectivityTids);
	}, false);

	// We only updated UV's, but this update usually comes after a change to unwrap vert positions, and we
	// can assume that the user hasn't issued a notification yet. So we do a normal unwrap preview update.
	NotifyUnwrapPreviewDeferredEditCompleted(this, ChangedVids, ChangedConnectivityTids, FastRenderUpdateTids, GetUnwrapUpdateAttributeFlags());

	if (WireframeDisplay)
	{
		WireframeDisplay->NotifyMeshChanged();
	}
}

void UUVEditorToolMeshInput::UpdateUnwrapCanonicalOverlayFromPositions(const TArray<int32>* ChangedVids, 
	const TArray<int32>* ChangedConnectivityTids)
{
	FDynamicMeshUVOverlay* DestOverlay = UnwrapCanonical->Attributes()->PrimaryUV();
	UVUnwrapMeshUtil::UpdateUVOverlayFromUnwrapMesh(*UnwrapCanonical, *DestOverlay, VertPositionToUV,
		ChangedVids, ChangedConnectivityTids);
}

void UUVEditorToolMeshInput::UpdateAppliedPreviewFromUnwrapPreview(const TArray<int32>* ChangedVids, 
	const TArray<int32>* ChangedConnectivityTids, const TArray<int32>* FastRenderUpdateTids)
{
	using namespace UVEditorToolMeshInputLocals;

	const FDynamicMesh3* SourceUnwrapMesh = UnwrapPreview->PreviewMesh->GetMesh();

	// We need to update the overlay in AppliedPreview. Assuming that the overlay in UnwrapPreview is updated, we can
	// just copy that overlay (using our function that doesn't copy element parent pointers)
	AppliedPreview->PreviewMesh->DeferredEditMesh([this, SourceUnwrapMesh, ChangedVids, ChangedConnectivityTids](FDynamicMesh3& Mesh)
	{
			const FDynamicMeshUVOverlay* SourceOverlay = SourceUnwrapMesh->Attributes()->PrimaryUV();
			FDynamicMeshUVOverlay* DestOverlay = Mesh.Attributes()->GetUVLayer(UVLayerIndex);

			UVUnwrapMeshUtil::UpdateOverlayFromOverlay(*SourceOverlay, *DestOverlay, false, ChangedVids, ChangedConnectivityTids);
	}, false);

	NotifyAppliedPreviewDeferredEditCompleted(this, ChangedVids, FastRenderUpdateTids, GetAppliedUpdateAttributeFlags());
}

void UUVEditorToolMeshInput::UpdateUnwrapPreviewFromAppliedPreview(const TArray<int32>* ChangedElementIDs, 
	const TArray<int32>* ChangedConnectivityTids, const TArray<int32>* FastRenderUpdateTids)
{
	using namespace UVEditorToolMeshInputLocals;

	// Convert the AppliedPreview UV overlay to positions in UnwrapPreview
	const FDynamicMeshUVOverlay* SourceOverlay = AppliedPreview->PreviewMesh->GetMesh()->Attributes()->GetUVLayer(UVLayerIndex);
	UnwrapPreview->PreviewMesh->DeferredEditMesh([this, SourceOverlay, ChangedElementIDs, ChangedConnectivityTids](FDynamicMesh3& Mesh)
	{
		UVUnwrapMeshUtil::UpdateUVUnwrapMesh(*SourceOverlay, Mesh, UVToVertPosition, ChangedElementIDs, ChangedConnectivityTids);

		// Also copy the actual overlay
		FDynamicMeshUVOverlay* DestOverlay = Mesh.Attributes()->PrimaryUV();
		UVUnwrapMeshUtil::UpdateOverlayFromOverlay(*SourceOverlay, *DestOverlay, false, ChangedElementIDs, ChangedConnectivityTids);
	}, false);

	NotifyUnwrapPreviewDeferredEditCompleted(this, ChangedElementIDs, ChangedConnectivityTids, FastRenderUpdateTids, GetUnwrapUpdateAttributeFlags());

	if (WireframeDisplay)
	{
		WireframeDisplay->NotifyMeshChanged();
	}
}

void UUVEditorToolMeshInput::UpdateCanonicalFromPreviews(const TArray<int32>* ChangedVids, 
	const TArray<int32>* ChangedConnectivityTids, bool bBroadcast)
{
	// Update UnwrapCanonical from UnwrapPreview
	UVUnwrapMeshUtil::UpdateUVUnwrapMesh(*UnwrapPreview->PreviewMesh->GetMesh(), *UnwrapCanonical, ChangedVids, ChangedConnectivityTids);
	
	// Update the overlay in AppliedCanonical from overlay in AppliedPreview
	const FDynamicMeshUVOverlay* SourceOverlay = AppliedPreview->PreviewMesh->GetMesh()->Attributes()->GetUVLayer(UVLayerIndex);
	FDynamicMeshUVOverlay* DestOverlay = AppliedCanonical->Attributes()->GetUVLayer(UVLayerIndex);
	UVUnwrapMeshUtil::UpdateOverlayFromOverlay(*SourceOverlay, *DestOverlay, true, ChangedVids, ChangedConnectivityTids);

	if (bBroadcast)
	{
		OnCanonicalModified.Broadcast(this, FCanonicalModifiedInfo());
	}
}

void UUVEditorToolMeshInput::UpdatePreviewsFromCanonical(const TArray<int32>* ChangedVids, const TArray<int32>* ChangedConnectivityTids, const TArray<int32>* FastRenderUpdateTids)
{
	using namespace UVEditorToolMeshInputLocals;

	// Update UnwrapPreview from UnwrapCanonical
	UnwrapPreview->PreviewMesh->DeferredEditMesh([this, ChangedVids, ChangedConnectivityTids](FDynamicMesh3& Mesh)
	{
		UVUnwrapMeshUtil::UpdateUVUnwrapMesh(*UnwrapCanonical, Mesh, ChangedVids, ChangedConnectivityTids);
	}, false);
	NotifyUnwrapPreviewDeferredEditCompleted(this, ChangedVids, ChangedConnectivityTids, FastRenderUpdateTids, GetUnwrapUpdateAttributeFlags());

	if (WireframeDisplay)
	{
		WireframeDisplay->NotifyMeshChanged();
	}

	// Update AppliedPreview from AppliedCanonical
	AppliedPreview->PreviewMesh->DeferredEditMesh([this, ChangedVids, ChangedConnectivityTids](FDynamicMesh3& Mesh)
	{
		const FDynamicMeshUVOverlay* SourceOverlay = AppliedCanonical->Attributes()->GetUVLayer(UVLayerIndex);
		FDynamicMeshUVOverlay* DestOverlay = Mesh.Attributes()->GetUVLayer(UVLayerIndex);
		UVUnwrapMeshUtil::UpdateOverlayFromOverlay(*SourceOverlay, *DestOverlay, true, ChangedVids, ChangedConnectivityTids);
	}, false);
	NotifyAppliedPreviewDeferredEditCompleted(this, ChangedVids, FastRenderUpdateTids, GetAppliedUpdateAttributeFlags());
}

void UUVEditorToolMeshInput::UpdateAllFromUnwrapPreview(const TArray<int32>* ChangedVids, 
	const TArray<int32>* ChangedConnectivityTids, const TArray<int32>* FastRenderUpdateTids, bool bBroadcast)
{
	UpdateAppliedPreviewFromUnwrapPreview(ChangedVids, ChangedConnectivityTids, FastRenderUpdateTids);
	UpdateCanonicalFromPreviews(ChangedVids, ChangedConnectivityTids, false);

	if (bBroadcast)
	{
		OnCanonicalModified.Broadcast(this, FCanonicalModifiedInfo());
	}
}

void UUVEditorToolMeshInput::UpdateAllFromUnwrapCanonical(
	const TArray<int32>* ChangedVids, const TArray<int32>* ChangedConnectivityTids, 
	const TArray<int32>* FastRenderUpdateTids, bool bBroadcast)
{
	// Update AppliedCanonical
	FDynamicMeshUVOverlay* SourceOverlay = UnwrapCanonical->Attributes()->PrimaryUV();
	FDynamicMeshUVOverlay* DestOverlay = AppliedCanonical->Attributes()->GetUVLayer(UVLayerIndex);
	UVUnwrapMeshUtil::UpdateOverlayFromOverlay(*SourceOverlay, *DestOverlay, false, ChangedVids, ChangedConnectivityTids);

	UpdatePreviewsFromCanonical(ChangedVids, ChangedConnectivityTids, FastRenderUpdateTids);

	if (bBroadcast)
	{
		OnCanonicalModified.Broadcast(this, FCanonicalModifiedInfo());
	}
}

void UUVEditorToolMeshInput::UpdateAllFromAppliedCanonical(
	const TArray<int32>* ChangedElementIDs, const TArray<int32>* ChangedConnectivityTids, 
	const TArray<int32>* FastRenderUpdateTids, bool bBroadcast)
{
	// Update UnwrapCanonical
	const FDynamicMeshUVOverlay* SourceOverlay = AppliedCanonical->Attributes()->GetUVLayer(UVLayerIndex);
	UVUnwrapMeshUtil::UpdateUVUnwrapMesh(*SourceOverlay, *UnwrapCanonical, UVToVertPosition, ChangedElementIDs, ChangedConnectivityTids);

	UpdatePreviewsFromCanonical(ChangedElementIDs, ChangedConnectivityTids, FastRenderUpdateTids);

	if (bBroadcast)
	{
		OnCanonicalModified.Broadcast(this, FCanonicalModifiedInfo());
	}
}

void UUVEditorToolMeshInput::UpdateAllFromAppliedPreview(
	const TArray<int32>* ChangedElementIDs, const TArray<int32>* ChangedConnectivityTids, 
	const TArray<int32>* FastRenderUpdateTids, bool bBroadcast)
{
	UpdateUnwrapPreviewFromAppliedPreview(ChangedElementIDs, ChangedConnectivityTids, FastRenderUpdateTids);
	UpdateCanonicalFromPreviews(ChangedElementIDs, ChangedConnectivityTids, false);

	if (bBroadcast)
	{
		OnCanonicalModified.Broadcast(this, FCanonicalModifiedInfo());
	}
}

void UUVEditorToolMeshInput::UpdateFromCanonicalUnwrapUsingMeshChange(
	const FDynamicMeshChange& UnwrapCanonicalMeshChange, bool bBroadcast)
{
	// Get the changed triangle list out of the mesh change. Unfortunately, we need to
	//  get both the initial and new triangles and lump them together in case unwrap triangles
	//  got created or destroyed via UV setting and unsetting.
	// We could maybe get around this because in most cases we know that no triangles got
	//  created or destroyed, or at least that none got destroyed (because we don't typically
	//  allow unsetting of triangles), but it would require passing that knowledge down, and
	//  in the end, it's not worth the trouble for something that only runs on undo/redo.
	TArray<int32> ChangedTids;
	UnwrapCanonicalMeshChange.GetSavedTriangleList(ChangedTids, true);
	TSet<int32> ChangedTidSet(ChangedTids);

	ChangedTids.Reset();
	UnwrapCanonicalMeshChange.GetSavedTriangleList(ChangedTids, false);
	ChangedTidSet.Append(ChangedTids);

	ChangedTids = ChangedTidSet.Array();

	TArray<int32> ChangedVids;
	TriangleToVertexIDs(UnwrapCanonical.Get(), ChangedTids, ChangedVids);

	TSet<int32> RenderUpdateTidsSet;
	VertexToTriangleOneRing(UnwrapCanonical.Get(), ChangedVids, RenderUpdateTidsSet);
	TArray<int32> RenderUpdateTids = RenderUpdateTidsSet.Array();

	UpdateAllFromUnwrapCanonical(&ChangedVids, &ChangedTids, &RenderUpdateTids, false);
	
	if (bBroadcast)
	{
		OnCanonicalModified.Broadcast(this, FCanonicalModifiedInfo());
	}
}

int32 UUVEditorToolMeshInput::UnwrapVidToAppliedVid(int32 UnwrapVid)
{
	const FDynamicMeshUVOverlay* CanonicalOverlay = AppliedCanonical->Attributes()->GetUVLayer(UVLayerIndex);
	return CanonicalOverlay->GetParentVertex(UnwrapVid);
}

void UUVEditorToolMeshInput::AppliedVidToUnwrapVids(int32 AppliedVid, TArray<int32>& UnwrapVidsOut)
{
	const FDynamicMeshUVOverlay* CanonicalOverlay = AppliedCanonical->Attributes()->GetUVLayer(UVLayerIndex);
	CanonicalOverlay->GetVertexElements(AppliedVid, UnwrapVidsOut);
}

EMeshRenderAttributeFlags UUVEditorToolMeshInput::GetUnwrapUpdateAttributeFlags() const
{
	if (bEnableTriangleVertexColors)
	{
		return EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexUVs | EMeshRenderAttributeFlags::VertexColors;
	}
	else
	{
		return EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexUVs;
	}
}

EMeshRenderAttributeFlags UUVEditorToolMeshInput::GetAppliedUpdateAttributeFlags() const
{
	return EMeshRenderAttributeFlags::VertexUVs;
}