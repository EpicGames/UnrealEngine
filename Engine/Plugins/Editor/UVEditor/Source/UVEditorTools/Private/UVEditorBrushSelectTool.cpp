// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorBrushSelectTool.h"

#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "BaseGizmos/GizmoMath.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "ContextObjectStore.h"
#include "InputCoreTypes.h" // EKey
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Selection/UVEditorMeshSelectionMechanic.h"
#include "Selections/MeshConnectedComponents.h"
#include "Templates/Function.h"
#include "UVEditorUXSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorBrushSelectTool)

#define LOCTEXT_NAMESPACE "UUVEditorBrushSelectTool"

namespace UVEditorBrushSelectToolLocals
{
	const FString BrushIndicatorGizmoType = TEXT("BrushIndicatorGizmoType");

	const float MinUnwrapBrushRadius = 1.0 / (1 << 6);
	const float MinLivePreviewBrushRadius = 100 / (1 << 5);

	void UpdateSelections(TArray<UUVToolSelectionAPI::FUVToolSelection>& Selections, TSet<int32> Tids,
		UUVEditorToolMeshInput* Target, bool bSubtract)
	{
		if (Tids.IsEmpty())
		{
			return;
		}

		int32 ExistingSelectionIndex = Selections.IndexOfByPredicate([Target](const UUVToolSelectionAPI::FUVToolSelection& ExistingSelection)
		{
			return ExistingSelection.Target == Target;
		});

		if (ExistingSelectionIndex >= 0)
		{
			UUVToolSelectionAPI::FUVToolSelection& ExistingSelection = Selections[ExistingSelectionIndex];
			if (bSubtract)
			{
				ExistingSelection.SelectedIDs = ExistingSelection.SelectedIDs.Difference(Tids);
				if (ExistingSelection.SelectedIDs.IsEmpty())
				{
					Selections.RemoveAtSwap(ExistingSelectionIndex);
				}
			}
			else
			{
				ExistingSelection.SelectedIDs.Append(Tids);
			}
		}
		// If no existing selection
		else if (!bSubtract)
		{
			UUVToolSelectionAPI::FUVToolSelection& NewSelection = Selections.Emplace_GetRef();
			NewSelection.Target = Target;
			NewSelection.Type = UUVToolSelectionAPI::FUVToolSelection::EType::Triangle;
			NewSelection.SelectedIDs = Tids;
		}
	}
}

void UUVEditorBrushSelectTool::Setup()
{
	using namespace UVEditorBrushSelectToolLocals;
	using namespace UE::Geometry;

	Super::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "Brush Select Tool"));


	Settings = NewObject<UUVEditorBrushSelectToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();
	EmitChangeAPI = ContextStore->FindContext<UUVToolEmitChangeAPI>();
	SelectionAPI = ContextStore->FindContext<UUVToolSelectionAPI>();
	LivePreviewAPI = ContextStore->FindContext<UUVToolLivePreviewAPI>();

	if (!ensure(SelectionAPI))
	{
		GetToolManager()->DeactivateTool(EToolSide::Mouse, EToolShutdownType::Completed);
	}

	SelectionMechanic = SelectionAPI->GetSelectionMechanic();

	UUVToolSelectionAPI::FHighlightOptions HighlightOptions;
	HighlightOptions.bAutoUpdateUnwrap = true;
	HighlightOptions.bAutoUpdateApplied = true;
	SelectionAPI->SetHighlightOptions(HighlightOptions);
	SelectionAPI->SetHighlightVisible(true, true);

	// Hook up to the input routers
	ULocalClickDragInputBehavior* UnwrapClickDragBehavior = NewObject<ULocalClickDragInputBehavior>();
	UnwrapClickDragBehavior->Initialize();
	UnwrapClickDragBehavior->CanBeginClickDragFunc = [](const FInputDeviceRay&) 
	{
		// Accept all hits
		return FInputRayHit(0.0);
	};
	UnwrapClickDragBehavior->OnClickPressFunc = [this, UnwrapClickDragBehavior](const FInputDeviceRay& DragPos)
	{
		SelectionAPI->BeginChange();
		if (Settings->bClearSelectionOnEachDrag && !bShiftToggle && !bCtrlToggle)
		{
			ClearSelections();
		}
		bCurrentStrokeIsSubtracting = bCtrlToggle && !bShiftToggle;
		// See TODO in header file
		//bCurrentStrokeIsInverting = bCtrlToggle && bShiftToggle;

		UnwrapClickDragBehavior->OnClickDragFunc(DragPos);
		bHaveInteracted = true;
	};
	UnwrapClickDragBehavior->OnClickDragFunc = [this](const FInputDeviceRay& DragPos)
	{
		bool bIntersects;
		FVector3d IntersectionPoint;
		GizmoMath::RayPlaneIntersectionPoint(FVector::Zero(), FVector::ZAxisVector,
			DragPos.WorldRay.Origin, DragPos.WorldRay.Direction, bIntersects, IntersectionPoint);
		PendingUnwrapHits.Add(FVector2d(IntersectionPoint));
		UpdateViewportStateFromHoverOrDragEvent(/*bFromUnwrap*/ true, /*bIsEndEvent*/ false, /*bDragging*/ true);
		// Hover behaviors are terminated during drag, so update hover
		UnwrapBrushIndicator->Update(Settings->UnwrapBrushRadius * FUVEditorUXSettings::UVMeshScalingFactor, 
			IntersectionPoint, FVector::ZAxisVector, /*Falloff*/ 0.0f, /*Strength*/ 1.0f);
	};
	UnwrapClickDragBehavior->OnClickReleaseFunc = [this, UnwrapClickDragBehavior](const FInputDeviceRay&)
	{
		SelectionAPI->EndChangeAndEmitIfModified(true);
		UnwrapClickDragBehavior->OnTerminateFunc();
	};
	UnwrapClickDragBehavior->OnTerminateFunc = [this]()
	{
		PendingUnwrapHits.Reset();
		UpdateViewportStateFromHoverOrDragEvent(/*bFromUnwrap*/ true, /*bIsEndEvent*/ true, /*bDragging*/ true);
	};
	AddInputBehavior(UnwrapClickDragBehavior);

	ULocalMouseHoverBehavior* UnwrapHoverBehavior = NewObject<ULocalMouseHoverBehavior>();
	UnwrapHoverBehavior->Initialize();
	UnwrapHoverBehavior->BeginHitTestFunc = [](const FInputDeviceRay& DragPos) {
		// Accept all hits
		return FInputRayHit(0.0);
	};
	UnwrapHoverBehavior->OnBeginHoverFunc = [this, UnwrapHoverBehavior](const FInputDeviceRay& DragPos)
	{
		UnwrapHoverBehavior->OnUpdateHover(DragPos);
	};
	UnwrapHoverBehavior->OnUpdateHoverFunc = [this](const FInputDeviceRay& DragPos)
	{
		bool bIntersects;
		FVector3d IntersectionPoint;
		GizmoMath::RayPlaneIntersectionPoint(FVector::Zero(), FVector::ZAxisVector,
			DragPos.WorldRay.Origin, DragPos.WorldRay.Direction, bIntersects, IntersectionPoint);
		UpdateViewportStateFromHoverOrDragEvent(/*bFromUnwrap*/ true, /*bIsEndEvent*/ false, /*bDragging*/ false);
		UnwrapBrushIndicator->Update(Settings->UnwrapBrushRadius * FUVEditorUXSettings::UVMeshScalingFactor, 
			IntersectionPoint, FVector::ZAxisVector, /*Falloff*/ 0.0f, /*Strength*/ 1.0f);

		// Should always be true, actually, since we're shooting a ray down into the plane. But might
		//  as well return this bool so that if something is incorrectly set up, we end hover.
		return bIntersects;
	};
	UnwrapHoverBehavior->OnEndHoverFunc = [this]()
	{
		UpdateViewportStateFromHoverOrDragEvent(/*bFromUnwrap*/ true, /*bIsEndEvent*/ true, /*bDragging*/ false);
	};
	UnwrapHoverBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	UnwrapHoverBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	auto UpdateModifiers = [this](int ModifierID, bool bIsOn) 
	{
		if (ModifierID == ShiftModifierID)
		{
			bShiftToggle = bIsOn;
		}
		else if (ModifierID == CtrlModifierID)
		{
			bCtrlToggle = bIsOn;
		}
	};
	UnwrapHoverBehavior->OnUpdateModifierStateFunc = UpdateModifiers;
	AddInputBehavior(UnwrapHoverBehavior);

	ULocalClickDragInputBehavior* LivePreviewClickDragBehavior = NewObject<ULocalClickDragInputBehavior>();
	LivePreviewClickDragBehavior->Initialize();
	LivePreviewClickDragBehavior->CanBeginClickDragFunc = [this](const FInputDeviceRay& DragPos)
	{
		UUVEditorMeshSelectionMechanic::FRaycastResult HitResult;
		if (SelectionMechanic.IsValid() && SelectionMechanic->RaycastCanonicals(DragPos.WorldRay, false, false, HitResult))
		{
			return FInputRayHit(DragPos.WorldRay.GetParameter(HitResult.HitPosition));
		}
		return FInputRayHit();
	};
	LivePreviewClickDragBehavior->OnClickPressFunc = [this, LivePreviewClickDragBehavior](const FInputDeviceRay& DragPos)
	{
		SelectionAPI->BeginChange();
		if (Settings->bClearSelectionOnEachDrag && !bShiftToggle && !bCtrlToggle)
		{
			ClearSelections();
		}
		bCurrentStrokeIsSubtracting = bCtrlToggle && !bShiftToggle;
		// See TODO in header
		//bCurrentStrokeIsInverting = bCtrlToggle && bShiftToggle;

		LivePreviewClickDragBehavior->OnClickDragFunc(DragPos);
		bHaveInteracted = true;
	};
	LivePreviewClickDragBehavior->OnClickDragFunc = [this](const FInputDeviceRay& DragPos)
	{
		UUVEditorMeshSelectionMechanic::FRaycastResult HitResult;
		if (SelectionMechanic.IsValid() && SelectionMechanic->RaycastCanonicals(DragPos.WorldRay, false, false, HitResult))
		{
			PendingLivePreviewHits.Add(HitResult);

			FVector3d Normal = Targets[HitResult.AssetID]->AppliedCanonical->GetTriNormal(HitResult.Tid);
			UpdateViewportStateFromHoverOrDragEvent(/*bFromUnwrap*/ false, /*bIsEndEvent*/ false, /*bDragging*/ true);
			// Hover behaviors are terminated during drag, so update hover
			LivePreviewBrushIndicator->Update(Settings->LivePreviewBrushRadius, 
				HitResult.HitPosition, Normal, /*Falloff*/ 0.0f, /*Strength*/ 1.0f);
		}
	};
	LivePreviewClickDragBehavior->OnClickReleaseFunc = [this, LivePreviewClickDragBehavior](const FInputDeviceRay& DragPos)
	{
		SelectionAPI->EndChangeAndEmitIfModified(true);
		LivePreviewClickDragBehavior->OnTerminateFunc();
	};
	LivePreviewClickDragBehavior->OnTerminateFunc = [this]()
	{
		PendingLivePreviewHits.Reset();
		UpdateViewportStateFromHoverOrDragEvent(/*bFromUnwrap*/ false, /*bIsEndEvent*/ true, /*bDragging*/ true);
	};

	ULocalMouseHoverBehavior* LivePreviewHoverBehavior = NewObject<ULocalMouseHoverBehavior>();
	LivePreviewHoverBehavior->Initialize();
	LivePreviewHoverBehavior->BeginHitTestFunc = [this](const FInputDeviceRay& DragPos)
	{
		UUVEditorMeshSelectionMechanic::FRaycastResult HitResult;
		if (SelectionMechanic.IsValid() && SelectionMechanic->RaycastCanonicals(DragPos.WorldRay, false, false, HitResult))
		{
			return FInputRayHit(DragPos.WorldRay.GetParameter(HitResult.HitPosition));
		}
		return FInputRayHit();
	};
	LivePreviewHoverBehavior->OnBeginHoverFunc = [this, LivePreviewHoverBehavior](const FInputDeviceRay& DragPos)
	{
		LivePreviewHoverBehavior->OnUpdateHover(DragPos);
	};
	LivePreviewHoverBehavior->OnUpdateHoverFunc = [this](const FInputDeviceRay& DragPos)
	{
		UUVEditorMeshSelectionMechanic::FRaycastResult HitResult;
		if (SelectionMechanic.IsValid() && SelectionMechanic->RaycastCanonicals(DragPos.WorldRay, false, false, HitResult))
		{
			FVector3d Normal = Targets[HitResult.AssetID]->AppliedCanonical->GetTriNormal(HitResult.Tid);
			UpdateViewportStateFromHoverOrDragEvent(/*bFromUnwrap*/ false, /*bIsEndEvent*/ false, /*bDragging*/ false);
			LivePreviewBrushIndicator->Update(Settings->LivePreviewBrushRadius, 
				HitResult.HitPosition, Normal, /*Falloff*/ 0.0f, /*Strength*/ 1.0f);
			return true;
		}
		return false;
	};
	LivePreviewHoverBehavior->OnEndHoverFunc = [this]()
	{
		UpdateViewportStateFromHoverOrDragEvent(/*bFromUnwrap*/ false, /*bIsEndEvent*/ true, /*bDragging*/ false);
	};
	LivePreviewHoverBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	LivePreviewHoverBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	LivePreviewHoverBehavior->OnUpdateModifierStateFunc = UpdateModifiers;
	
	if (LivePreviewAPI)
	{
		LivePreviewInputRouter = LivePreviewAPI->GetLivePreviewInputRouter();
		LivePreviewBehaviorSet = NewObject<UInputBehaviorSet>();
		LivePreviewBehaviorSource = NewObject<ULocalInputBehaviorSource>();
		LivePreviewBehaviorSource->GetInputBehaviorsFunc = [this]() { return LivePreviewBehaviorSet; };
		LivePreviewBehaviorSet->Add(LivePreviewClickDragBehavior, this);
		LivePreviewBehaviorSet->Add(LivePreviewHoverBehavior, this);

		if (ensure(LivePreviewInputRouter.IsValid()))
		{
			LivePreviewInputRouter->RegisterSource(LivePreviewBehaviorSource);
		}

		// register and spawn brush indicator gizmo for live preview
		UInteractiveGizmoManager* LivePreviewGizmoManager = LivePreviewAPI->GetGizmoManager();
		if (ensure(LivePreviewGizmoManager))
		{
			LivePreviewGizmoManager->RegisterGizmoType(BrushIndicatorGizmoType, NewObject<UBrushStampIndicatorBuilder>());
			LivePreviewBrushIndicator = LivePreviewGizmoManager->CreateGizmo<UBrushStampIndicator>(
				BrushIndicatorGizmoType, FString(), this);
		}
	}
	
	// register and spawn brush indicator gizmo for unwrap
	GetToolManager()->GetPairedGizmoManager()->RegisterGizmoType(BrushIndicatorGizmoType, NewObject<UBrushStampIndicatorBuilder>());
	UnwrapBrushIndicator = GetToolManager()->GetPairedGizmoManager()->CreateGizmo<UBrushStampIndicator>(BrushIndicatorGizmoType, FString(), this);
	
	GetToolManager()->DisplayMessage(LOCTEXT("StatusBarMessage",
		"Shift adds to selection, Ctrl subtracts, both together toggle. [ and ] change brush size."),
		EToolMessageLevel::UserNotification);
}

void UUVEditorBrushSelectTool::Shutdown(EToolShutdownType ShutdownType)
{
	using namespace UVEditorBrushSelectToolLocals;

	if (LivePreviewInputRouter.IsValid())
	{
		// TODO: Arguably the live preview input router should do this for us before Shutdown(), but
		// we don't currently have support for that...
		LivePreviewInputRouter->DeregisterSource(LivePreviewBehaviorSource);
	}

	GetToolManager()->GetPairedGizmoManager()->DestroyGizmo(UnwrapBrushIndicator);
	GetToolManager()->GetPairedGizmoManager()->DeregisterGizmoType(BrushIndicatorGizmoType);
	UnwrapBrushIndicator = nullptr;

	if (LivePreviewAPI)
	{
		if (UInteractiveGizmoManager* LivePreviewGizmoManager = LivePreviewAPI->GetGizmoManager())
		{
			LivePreviewGizmoManager->DestroyGizmo(LivePreviewBrushIndicator);
			LivePreviewGizmoManager->DeregisterGizmoType(BrushIndicatorGizmoType);
		}
	}
	LivePreviewBrushIndicator = nullptr;

	ProcessPendingUnwrapHits();
	ProcessPendingLivePreviewHits();
	SelectionAPI->EndChangeAndEmitIfModified(true);

	LivePreviewBehaviorSet->RemoveAll();
	LivePreviewBehaviorSet = nullptr;
	LivePreviewBehaviorSource = nullptr;

	Settings->SaveProperties(this);
	Settings = nullptr;

	SelectionAPI = nullptr;
	EmitChangeAPI = nullptr;
	LivePreviewAPI = nullptr;

	Super::Shutdown(ShutdownType);
}

void UUVEditorBrushSelectTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

void UUVEditorBrushSelectTool::OnTick(float DeltaTime)
{
	ProcessPendingUnwrapHits();
	ProcessPendingLivePreviewHits();
}

void UUVEditorBrushSelectTool::ClearSelections(bool bBroadcastAndEmit)
{
	if (bBroadcastAndEmit)
	{
		EmitChangeAPI->BeginUndoTransaction(LOCTEXT("ClearSelectionTransaction", "Clear Selection"));
	}
	SelectionAPI->ClearUnsetElementAppliedMeshSelections(bBroadcastAndEmit, bBroadcastAndEmit);
	SelectionAPI->ClearSelections(bBroadcastAndEmit, bBroadcastAndEmit);
	if (bBroadcastAndEmit)
	{
		EmitChangeAPI->EndUndoTransaction();
	}
}

void UUVEditorBrushSelectTool::ProcessPendingUnwrapHits()
{
	using namespace UE::Geometry;
	using namespace UVEditorBrushSelectToolLocals;

	if (!SelectionMechanic.IsValid() || PendingUnwrapHits.IsEmpty())
	{
		return;
	}

	TArray<FUVToolSelection> SelectionsToSet = SelectionAPI->GetSelections();

	for (const FVector2d& WorldHitPoint : PendingUnwrapHits)
	{
		for (FUVToolSelection& Selection : SelectionMechanic->GetAllCanonicalTrianglesInUnwrapRadius(
			WorldHitPoint, Settings->UnwrapBrushRadius * FUVEditorUXSettings::UVMeshScalingFactor))
		{
			if (Settings->bExpandToIslands)
			{
				TArray<int32> ExpandedTids;
				FMeshConnectedComponents::GrowToConnectedTriangles(
					Selection.Target->UnwrapCanonical.Get(), 
					Selection.SelectedIDs.Array(), ExpandedTids, &TempROIBuffer);
				Selection.SelectedIDs.Append(ExpandedTids);
			}

			UpdateSelections(SelectionsToSet, Selection.SelectedIDs, Selection.Target.Get(), 
				bCurrentStrokeIsSubtracting);
		}
	}

	SelectionAPI->SetSelections(SelectionsToSet, false, false);

	PendingUnwrapHits.Reset();
}

void UUVEditorBrushSelectTool::ProcessPendingLivePreviewHits()
{
	using namespace UE::Geometry;
	using namespace UVEditorBrushSelectToolLocals;

	if (!SelectionMechanic.IsValid() || PendingLivePreviewHits.IsEmpty())
	{
		return;
	}

	TArray<FUVToolSelection> SelectionsToSet = SelectionAPI->GetSelections();
	TArray<FUVToolSelection> UnsetSelectionsToSet = SelectionAPI->GetUnsetElementAppliedMeshSelections();

	for (const UUVEditorMeshSelectionMechanic::FRaycastResult& Hit : PendingLivePreviewHits)
	{
		const FDynamicMesh3* Mesh = Targets[Hit.AssetID]->AppliedCanonical.Get();
		const FDynamicMeshUVOverlay* UVOverlay = Mesh->Attributes()->GetUVLayer(Targets[Hit.AssetID]->UVLayerIndex);

		if (Mesh->IsTriangle(Hit.Tid))
		{
			double RadiusSquared = Settings->LivePreviewBrushRadius * Settings->LivePreviewBrushRadius;

			TFunction<bool(int32, int32)> GrowPredicate = [Mesh, &Hit, RadiusSquared](int t1, int t2)
			{
				return (Mesh->GetTriCentroid(t2) - Hit.HitPosition).SquaredLength() <= RadiusSquared;
			};
			if (Settings->bExpandToIslands)
			{
				GrowPredicate = [Mesh, UVOverlay, &Hit, RadiusSquared](int t1, int t2)
				{
					int32 Edge = Mesh->FindEdgeFromTriPair(t1, t2);
					return !UVOverlay->IsSeamEdge(Edge) 
						|| (Mesh->GetTriCentroid(t2) - Hit.HitPosition).SquaredLength() <= RadiusSquared;
				};
			}

			TArray<int32> StartSet { Hit.Tid };
			TSet<int32> GrownAppliedTids;
			FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartSet, GrownAppliedTids, &TempROIBuffer, GrowPredicate);

			const FDynamicMesh3* UnwrapMesh = Targets[Hit.AssetID]->UnwrapCanonical.Get();
			TSet<int32> UnwrapTids;
			TSet<int32> UnsetTids;
			for (int32 Tid : GrownAppliedTids)
			{
				if (UnwrapMesh->IsTriangle(Tid))
				{
					UnwrapTids.Add(Tid);
				}
				else
				{
					UnsetTids.Add(Tid);
				}
			}

			UpdateSelections(SelectionsToSet, UnwrapTids, Targets[Hit.AssetID], 
				bCurrentStrokeIsSubtracting);
			UpdateSelections(UnsetSelectionsToSet, UnsetTids, Targets[Hit.AssetID], 
				bCurrentStrokeIsSubtracting);
		}
	}

	SelectionAPI->SetSelections(SelectionsToSet, false, false);
	SelectionAPI->SetUnsetElementAppliedMeshSelections(UnsetSelectionsToSet, false, false);

	PendingLivePreviewHits.Reset();
}

void UUVEditorBrushSelectTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	int32 ActionID = (int32)EStandardToolActions::BaseClientDefinedActionID + 1;
	
	// TODO: We could/should use EStandardToolActions::IncreaseBrushSize and EStandardToolActions::DecreaseBrushSize
	//  as the action IDs here so that the same brush resizing hotkeys can act everywhere. The problem is that
	//  there is currently a bug where if we don't have at least one action that is BaseClientDefinedActionID or
	//  above, we will get a crash in our tool, see UE-221911.
	ActionSet.RegisterAction(this, ActionID++,
		TEXT("IncreaseRadius"),
		LOCTEXT("IncreaseRadius", "Increase Radius"),
		LOCTEXT("IncreaseRadiusTooltip", "Increase Brush Radius"),
		EModifierKey::None, EKeys::RightBracket,
		[this]() { IncreaseBrushRadiusAction(); });

	ActionSet.RegisterAction(this, ActionID++,
		TEXT("DecreaseRadius"),
		LOCTEXT("DecreaseRadius", "Decrease Radius"),
		LOCTEXT("DecreaseRadiusTooltip", "Decrease Brush Radius"),
		EModifierKey::None, EKeys::LeftBracket,
		[this]() { DecreaseBrushRadiusAction(); });
}

// We track which viewport (unwrap or live preview) we're hovering so that we can display the relevant brush
//  and so that we can route hotkey presses to update the correct brush. This turns out to be a little tricky
//  because we don't actually get an end hover event when we go from one viewport to the other, and because we
//  DO get an end hover event AFTER getting a drag start event...
void UUVEditorBrushSelectTool::UpdateViewportStateFromHoverOrDragEvent(bool bFromUnwrap, bool bIsEndEvent, bool bDragging)
{
	if (!bIsEndEvent)
	{
		// If we're getting a non-end event, then we definitely know that we're over one viewport
		//  and not the other.
		bHoveringUnwrap = bFromUnwrap;
		bHoveringLivePreview = !bFromUnwrap;
		if (bDragging)
		{
			bDraggingUnwrap = bFromUnwrap;
			bDraggingLivePreview = !bFromUnwrap;
		}
	}
	// If this is an end event, then it's trickier. If we're ending a drag, we can safely say
	//  that we're not over that viewport (if we're still over it, our next hover begin event
	//  will let us know). However if it's a hover end event, that frequently comes after a
	//  begin drag event, so we don't want to mark ourselves as not being over that viewport.
	else if (bDragging)
	{
		if (bFromUnwrap)
		{
			bDraggingUnwrap = false;
			bHoveringUnwrap = false;
		}
		else
		{
			bDraggingLivePreview = false;
			bHoveringLivePreview = false;
		}
	}
	else // end hover event
	{
		if (bFromUnwrap)
		{
			bHoveringUnwrap = bDraggingUnwrap;
		}
		else
		{
			bHoveringLivePreview = bDraggingLivePreview;
		}
	}

	// Update our brushes
	if (UnwrapBrushIndicator)
	{
		UnwrapBrushIndicator->bVisible = bHoveringUnwrap;
	}
	if (LivePreviewBrushIndicator)
	{
		LivePreviewBrushIndicator->bVisible = bHoveringLivePreview;
	}
}

// TODO: Probably will want to rework the increase/decrease actions. Might want the live preview radius in
//  particular to be responsive to the bounds of the mesh, like most of our brush tools.
void UUVEditorBrushSelectTool::IncreaseBrushRadiusAction()
{
	using namespace UVEditorBrushSelectToolLocals;

	if (bHoveringLivePreview)
	{
		Settings->LivePreviewBrushRadius = Settings->LivePreviewBrushRadius <= 0 ? MinLivePreviewBrushRadius
			: Settings->LivePreviewBrushRadius * 2;
	}
	if (bHoveringUnwrap)
	{
		Settings->UnwrapBrushRadius = Settings->UnwrapBrushRadius <= 0 ? MinUnwrapBrushRadius
			: Settings->UnwrapBrushRadius * 2;
	}
}
void UUVEditorBrushSelectTool::DecreaseBrushRadiusAction()
{
	using namespace UVEditorBrushSelectToolLocals;

	if (bHoveringLivePreview)
	{
		Settings->LivePreviewBrushRadius = Settings->LivePreviewBrushRadius < MinLivePreviewBrushRadius + KINDA_SMALL_NUMBER ? 0.0
			: Settings->LivePreviewBrushRadius / 2;
	}
	if (bHoveringUnwrap)
	{
		Settings->UnwrapBrushRadius = Settings->UnwrapBrushRadius < MinUnwrapBrushRadius + KINDA_SMALL_NUMBER ? 0.0
			: Settings->UnwrapBrushRadius / 2;
	}
}

bool UUVEditorBrushSelectTool::CanCurrentlyNestedCancel()
{
	return bHaveInteracted && (SelectionAPI->HaveSelections() 
		|| SelectionAPI->HaveUnsetElementAppliedMeshSelections());
}

bool UUVEditorBrushSelectTool::ExecuteNestedCancelCommand()
{
	if (CanCurrentlyNestedCancel())
	{
		ClearSelections(true);
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
