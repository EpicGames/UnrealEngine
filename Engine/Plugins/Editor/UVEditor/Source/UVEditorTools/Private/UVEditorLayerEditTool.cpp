// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorLayerEditTool.h"

#include "ContextObjectStore.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "ParameterizationOps/UVLayoutOp.h"
#include "Properties/UVLayoutProperties.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "ModelingToolTargetUtil.h"
#include "Components.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "EngineAnalytics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorLayerEditTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UUVChannelEditTool"

namespace UUVEditorChannelEditLocals
{
	const FText UVChannelAddTransactionName = LOCTEXT("UVChannelAddTransactionName", "Add UV Channel");
	const FText UVChannelCopyTransactionName = LOCTEXT("UVChannelCopyTransactionName", "Copy UV Channel");
	const FText UVChannelDeleteTransactionName = LOCTEXT("UVChannelDeleteTransactionName", "Delete UV Channel");

	void DeleteChannel(UUVEditorToolMeshInput* Target, int32 DeletedUVChannelIndex, 
		UUVToolAssetAndChannelAPI* AssetAndChannelAPI, bool bClearInstead)
	{
		FDynamicMeshUVEditor DynamicMeshUVEditor(Target->AppliedCanonical.Get(), DeletedUVChannelIndex, false);

		int32 NewChannelIndex = DeletedUVChannelIndex;
		if (bClearInstead)
		{
			DynamicMeshUVEditor.SetPerTriangleUVs(0.0, nullptr);
		}
		else
		{
			NewChannelIndex = DynamicMeshUVEditor.RemoveUVLayer();
		}

		if (NewChannelIndex != Target->UVLayerIndex)
		{
			// The change of displayed layer will perform the needed update for us
			AssetAndChannelAPI->NotifyOfAssetChannelCountChange(Target->AssetID);
			TArray<int32> ChannelPerAsset = AssetAndChannelAPI->GetCurrentChannelVisibility();
			ChannelPerAsset[Target->AssetID] = NewChannelIndex;
			AssetAndChannelAPI->RequestChannelVisibilityChange(ChannelPerAsset, false);
		}
		else
		{
			// We're showing the same layer index, but it's now the next layer over, actually,
			// so we need to update it.
			Target->UpdateAllFromAppliedCanonical();
		}
	}

	class FInputObjectUVChannelAdd : public FToolCommandChange
	{
	public:
		FInputObjectUVChannelAdd(TObjectPtr<UUVEditorToolMeshInput>& TargetIn, int32 AddedUVChannelIndexIn)
			: Target(TargetIn)
			, AddedUVChannelIndex(AddedUVChannelIndexIn)			
		{
		}

		virtual void Apply(UObject* Object) override
		{
			UInteractiveToolManager* InteractiveToolManager = Cast<UUVEditorChannelEditTool>(Object)->GetToolManager();
			if (!ensure(InteractiveToolManager))
			{
				return;
			}
			UUVToolAssetAndChannelAPI* AssetAndChannelAPI = InteractiveToolManager->GetContextObjectStore()->FindContext<UUVToolAssetAndChannelAPI>();

			FDynamicMeshUVEditor DynamicMeshUVEditor(Target->AppliedCanonical.Get(), AddedUVChannelIndex-1, false);
			int32 NewChannelIndex = DynamicMeshUVEditor.AddUVLayer();
			check(NewChannelIndex == AddedUVChannelIndex);

			if (NewChannelIndex != -1)
			{
				Target->AppliedPreview->PreviewMesh->UpdatePreview(Target->AppliedCanonical.Get());

				AssetAndChannelAPI->NotifyOfAssetChannelCountChange(Target->AssetID);
				Target->OnCanonicalModified.Broadcast(Target.Get(), UUVEditorToolMeshInput::FCanonicalModifiedInfo());
			}

		}

		virtual void Revert(UObject* Object) override
		{
			UInteractiveToolManager* InteractiveToolManager = Cast<UUVEditorChannelEditTool>(Object)->GetToolManager();
			if (!ensure(InteractiveToolManager))
			{
				return;
			}
			UUVToolAssetAndChannelAPI* AssetAndChannelAPI = InteractiveToolManager->GetContextObjectStore()->FindContext<UUVToolAssetAndChannelAPI>();

			FDynamicMeshUVEditor DynamicMeshUVEditor(Target->AppliedCanonical.Get(), AddedUVChannelIndex, false);
			int32 NewChannelIndex = DynamicMeshUVEditor.RemoveUVLayer();
			Target->AppliedPreview->PreviewMesh->UpdatePreview(Target->AppliedCanonical.Get());
			AssetAndChannelAPI->NotifyOfAssetChannelCountChange(Target->AssetID);
			Target->OnCanonicalModified.Broadcast(Target.Get(), UUVEditorToolMeshInput::FCanonicalModifiedInfo());
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			return !(Target.IsValid() && Target->IsValid());
		}

		virtual FString ToString() const override
		{
			return TEXT("UVEditorModeLocals::FInputObjectUVChannelAdd");
		}

	protected:
		TWeakObjectPtr<UUVEditorToolMeshInput> Target;
		int32 AddedUVChannelIndex;		
	};

	class FInputObjectUVChannelCopy : public FToolCommandChange
	{
	public:
		FInputObjectUVChannelCopy(const TObjectPtr<UUVEditorToolMeshInput>& TargetIn, int32 SourceUVChannelIndexIn, int32 TargetUVChannelIndexIn,
		                          const FDynamicMeshUVOverlay& OriginalUVChannelIn)
			: Target(TargetIn)			
			, SourceUVChannelIndex(SourceUVChannelIndexIn)
			, TargetUVChannelIndex(TargetUVChannelIndexIn)
			, OriginalUVChannel(OriginalUVChannelIn)
		{
		}

		virtual void Apply(UObject* Object) override
		{
			UInteractiveToolManager* InteractiveToolManager = Cast<UUVEditorChannelEditTool>(Object)->GetToolManager();
			if (!ensure(InteractiveToolManager))
			{
				return;
			}

			FDynamicMeshUVEditor DynamicMeshUVEditor(Target->AppliedCanonical.Get(), TargetUVChannelIndex, false);
			DynamicMeshUVEditor.CopyUVLayer(Target->AppliedCanonical->Attributes()->GetUVLayer(SourceUVChannelIndex));
			Target->UpdateAllFromAppliedCanonical();
		}

		virtual void Revert(UObject* Object) override
		{
			UInteractiveToolManager* InteractiveToolManager = Cast<UUVEditorChannelEditTool>(Object)->GetToolManager();
			if (!ensure(InteractiveToolManager))
			{
				return;
			}

			FDynamicMeshUVEditor DynamicMeshUVEditor(Target->AppliedCanonical.Get(), TargetUVChannelIndex, false);
			DynamicMeshUVEditor.CopyUVLayer(&OriginalUVChannel);
			Target->UpdateAllFromAppliedCanonical();
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			return !(Target.IsValid() && Target->IsValid());
		}

		virtual FString ToString() const override
		{
			return TEXT("UVEditorModeLocals::FInputObjectUVChannelCopy");
		}

	protected:
		TWeakObjectPtr<UUVEditorToolMeshInput> Target;
		int32 SourceUVChannelIndex;
		int32 TargetUVChannelIndex;
		FDynamicMeshUVOverlay OriginalUVChannel;
	};

	class FInputObjectUVChannelDelete : public FToolCommandChange
	{
	public:
		FInputObjectUVChannelDelete(const TObjectPtr<UUVEditorToolMeshInput>& TargetIn, int32 DeletedUVChannelIndexIn, 
			const FDynamicMeshUVOverlay& OriginalUVChannelIn, bool bClearedInsteadIn)
			: Target(TargetIn)			
			, DeletedUVChannelIndex(DeletedUVChannelIndexIn)
			, OriginalUVChannel(OriginalUVChannelIn)
			, bClearedInstead(bClearedInsteadIn)
		{
		}

		virtual void Apply(UObject* Object) override
		{
			UInteractiveToolManager* InteractiveToolManager = Cast<UUVEditorChannelEditTool>(Object)->GetToolManager();
			if (!ensure(InteractiveToolManager))
			{
				return;
			}
			UUVToolAssetAndChannelAPI* AssetAndChannelAPI = InteractiveToolManager->GetContextObjectStore()->FindContext<UUVToolAssetAndChannelAPI>();

			DeleteChannel(Target.Get(), DeletedUVChannelIndex, AssetAndChannelAPI, bClearedInstead);
		}

		virtual void Revert(UObject* Object) override
		{
			UInteractiveToolManager* InteractiveToolManager = Cast<UUVEditorChannelEditTool>(Object)->GetToolManager();
			if (!ensure(InteractiveToolManager))
			{
				return;
			}
			UUVToolAssetAndChannelAPI* AssetAndChannelAPI = InteractiveToolManager->GetContextObjectStore()->FindContext<UUVToolAssetAndChannelAPI>();

			FDynamicMeshUVEditor DynamicMeshUVEditor(Target->AppliedCanonical.Get(), 0, false);
			if (!bClearedInstead)
			{
				int32 NewChannelIndex = DynamicMeshUVEditor.AddUVLayer();

				// Shift the new layer to the correct index.
				// TODO: This is slightly wasteful since we should just be allowed to swap pointers in the underlying indirect array
				for (int32 ChannelIndex = NewChannelIndex; ChannelIndex > DeletedUVChannelIndex; --ChannelIndex)
				{
					DynamicMeshUVEditor.SwitchActiveLayer(ChannelIndex);
					DynamicMeshUVEditor.CopyUVLayer(Target->AppliedCanonical->Attributes()->GetUVLayer(ChannelIndex - 1));
				}
			}

			// Fill the layer
			DynamicMeshUVEditor.SwitchActiveLayer(DeletedUVChannelIndex);
			DynamicMeshUVEditor.CopyUVLayer(&OriginalUVChannel);

			if (DeletedUVChannelIndex != Target->UVLayerIndex)
			{
				// The change of displayed layer will perform the needed update for us
				AssetAndChannelAPI->NotifyOfAssetChannelCountChange(Target->AssetID);
				TArray<int32> ChannelPerAsset = AssetAndChannelAPI->GetCurrentChannelVisibility();
				ChannelPerAsset[Target->AssetID] = DeletedUVChannelIndex;
				AssetAndChannelAPI->RequestChannelVisibilityChange(ChannelPerAsset, false);
			}
			else
			{
				// We're showing the same layer index, but it's actually the previously deleted layer
				Target->UpdateAllFromAppliedCanonical();
			}
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			return !(Target.IsValid() && Target->IsValid());
		}

		virtual FString ToString() const override
		{
			return TEXT("UVEditorModeLocals::FInputObjectUVChannelDelete");
		}

	protected:
		TWeakObjectPtr<UUVEditorToolMeshInput> Target;		
		int32 DeletedUVChannelIndex;
		FDynamicMeshUVOverlay OriginalUVChannel;
		bool bClearedInstead;
	};

}


bool UUVEditorChannelEditToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return Targets && Targets->Num() > 0;
}

UInteractiveTool* UUVEditorChannelEditToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UUVEditorChannelEditTool* NewTool = NewObject<UUVEditorChannelEditTool>(SceneState.ToolManager);
	NewTool->SetTargets(*Targets);

	return NewTool;
}

void UUVEditorChannelEditTargetProperties::Initialize(
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn,
	bool bInitializeSelection)
{
	UVAssetNames.Reset(TargetsIn.Num());
	NumUVChannelsPerAsset.Reset(TargetsIn.Num());

	for (int i = 0; i < TargetsIn.Num(); ++i)
	{
		UVAssetNames.Add(UE::ToolTarget::GetHumanReadableName(TargetsIn[i]->SourceTarget));

		int32 NumUVChannels = TargetsIn[i]->AppliedCanonical->HasAttributes() ?
			TargetsIn[i]->AppliedCanonical->Attributes()->NumUVLayers() : 0;

		NumUVChannelsPerAsset.Add(NumUVChannels);
	}

	if (bInitializeSelection)
	{
		Asset = UVAssetNames[0];
		GetUVChannelNames();
		TargetChannel = UVChannelNames.Num() > 0 ? UVChannelNames[0] : TEXT("");
		ReferenceChannel = UVChannelNames.Num() > 0 ? UVChannelNames[0] : TEXT("");
	}
}

const TArray<FString>& UUVEditorChannelEditTargetProperties::GetAssetNames()
{
	return UVAssetNames;
}

const TArray<FString>& UUVEditorChannelEditTargetProperties::GetUVChannelNames()
{
	int32 FoundAssetIndex = UVAssetNames.IndexOfByKey(Asset);

	if (FoundAssetIndex == INDEX_NONE)
	{
		UVChannelNames.Reset();
		return UVChannelNames;
	}

	if (UVChannelNames.Num() != NumUVChannelsPerAsset[FoundAssetIndex])
	{
		UVChannelNames.Reset();
		for (int32 i = 0; i < NumUVChannelsPerAsset[FoundAssetIndex]; ++i)
		{
			UVChannelNames.Add(FString::Printf(TEXT("UV%d"), i));
		}
	}

	return UVChannelNames;
}

bool UUVEditorChannelEditTargetProperties::ValidateUVAssetSelection(bool bUpdateIfInvalid)
{
	int32 FoundIndex = UVAssetNames.IndexOfByKey(Asset);
	if (FoundIndex == INDEX_NONE)
	{
		if (bUpdateIfInvalid)
		{
			Asset = (UVAssetNames.Num() > 0) ? UVAssetNames[0] : TEXT("");
		}
		return false;
	}
	return true;
}

bool UUVEditorChannelEditTargetProperties::ValidateUVChannelSelection(bool bUpdateIfInvalid)
{
	bool bValid = ValidateUVAssetSelection(bUpdateIfInvalid);

	int32 FoundAssetIndex = UVAssetNames.IndexOfByKey(Asset);
	if (FoundAssetIndex == INDEX_NONE)
	{
		if (bUpdateIfInvalid)
		{
			TargetChannel = TEXT("");
			ReferenceChannel = TEXT("");
		}
		return false;
	}

	auto CheckAndUpdateChannel = [this, FoundAssetIndex, bUpdateIfInvalid](FString& UVChannel)
	{
		int32 FoundIndex = UVChannelNames.IndexOfByKey(UVChannel);
		if (FoundIndex >= NumUVChannelsPerAsset[FoundAssetIndex])
		{
			FoundIndex = INDEX_NONE;
		}

		if (FoundIndex == INDEX_NONE)
		{
			if (bUpdateIfInvalid)
			{
				UVChannel = (NumUVChannelsPerAsset[FoundAssetIndex] > 0) ? UVChannelNames[0] : TEXT("");
			}
			return false;
		}
		return true;
	};

	bValid &= CheckAndUpdateChannel(TargetChannel);

	return bValid;
}

int32 UUVEditorChannelEditTargetProperties::GetSelectedAssetID()
{
	int32 FoundIndex = UVAssetNames.IndexOfByKey(Asset);
	if (FoundIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return FoundIndex;
}

int32 UUVEditorChannelEditTargetProperties::GetSelectedChannelIndex(bool bForceToZeroOnFailure, bool bUseReference)
{
	int32 FoundAssetIndex = UVAssetNames.IndexOfByKey(Asset);
	if (FoundAssetIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FString UVChannel = bUseReference ? ReferenceChannel : TargetChannel;

	int32 FoundUVIndex = UVChannelNames.IndexOfByKey(UVChannel);
	if (!ensure(FoundUVIndex < NumUVChannelsPerAsset[FoundAssetIndex]))
	{
		return INDEX_NONE;
	}
	return FoundUVIndex;
}

void UUVEditorChannelEditTargetProperties::SetUsageFlags(EChannelEditToolAction Action) {
	switch (Action)
	{
		case EChannelEditToolAction::Add:
			bActionNeedsAsset = true;
			bActionNeedsReference = false;
			bActionNeedsTarget = false;
			break;
		case EChannelEditToolAction::Copy:
			bActionNeedsAsset = true;
			bActionNeedsReference = true;
			bActionNeedsTarget = true;
			break;
		case EChannelEditToolAction::Delete:
			bActionNeedsAsset = true;
			bActionNeedsReference = false;
			bActionNeedsTarget = true;
			break;
		default:
			bActionNeedsAsset = false;
			bActionNeedsReference = false;
			bActionNeedsTarget = false;
			break;
	}
}




// Tool property functions

void  UUVEditorChannelEditToolActionPropertySet::Apply()
{
	if (ParentTool.IsValid())
	{
		PostAction(ParentTool->ActiveAction());
		return;
	}	
	PostAction(EChannelEditToolAction::NoAction);
}


void UUVEditorChannelEditToolActionPropertySet::PostAction(EChannelEditToolAction Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}



void UUVEditorChannelEditTool::Setup()
{
	check(Targets.Num() > 0);
	
	ToolStartTimeAnalytics = FDateTime::UtcNow();

	UInteractiveTool::Setup();

	EmitChangeAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolEmitChangeAPI>();

	ActionSelectionProperties = NewObject<UUVEditorChannelEditSettings>(this);
	ActionSelectionProperties->WatchProperty(ActionSelectionProperties->Action, [this](EChannelEditToolAction Action) {
		if (SourceChannelProperties)
		{
			SourceChannelProperties->SetUsageFlags(Action);
			NotifyOfPropertyChangeByTool(SourceChannelProperties);
		}
	});
	AddToolPropertySource(ActionSelectionProperties);

	SourceChannelProperties = NewObject<UUVEditorChannelEditTargetProperties>(this);
	SourceChannelProperties->Initialize(Targets,true);
	SourceChannelProperties->WatchProperty(SourceChannelProperties->Asset, [this](FString UVAsset) {ApplyVisbleChannelChange(); });
	SourceChannelProperties->WatchProperty(SourceChannelProperties->TargetChannel, [this](FString UVChannel) {ApplyVisbleChannelChange(); });


	AddToolPropertySource(SourceChannelProperties);

	AddActionProperties = NewObject<UUVEditorChannelEditAddProperties>(this);
	AddToolPropertySource(AddActionProperties);

	CopyActionProperties = NewObject<UUVEditorChannelEditCopyProperties>(this);
	AddToolPropertySource(CopyActionProperties);

	DeleteActionProperties = NewObject<UUVEditorChannelEditDeleteProperties>(this);
	AddToolPropertySource(DeleteActionProperties);

	ToolActions = NewObject<UUVEditorChannelEditToolActionPropertySet>(this);
	ToolActions->Initialize(this);
	AddToolPropertySource(ToolActions);

	SetToolDisplayName(LOCTEXT("ToolName", "UV Channel Edit"));
	GetToolManager()->DisplayMessage(LOCTEXT("OnStartUVChannelEditTool", "Add, copy or delete UV channels"),
		EToolMessageLevel::UserNotification);

	for (int32 i = 0; i < Targets.Num(); ++i)
	{
		Targets[i]->OnCanonicalModified.AddWeakLambda(this, [this, i]
		(UUVEditorToolMeshInput* InputObject, const UUVEditorToolMeshInput::FCanonicalModifiedInfo&) {
			UpdateChannelSelectionProperties(i);
		});
	}

	// Analytics
	InputTargetAnalytics = UVEditorAnalytics::CollectTargetAnalytics(Targets);
}

void UUVEditorChannelEditTool::Shutdown(EToolShutdownType ShutdownType)
{
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->OnCanonicalModified.RemoveAll(this);
	}

	Targets.Empty();

	// Analytics
	RecordAnalytics();
}

void UUVEditorChannelEditTool::UpdateChannelSelectionProperties(int32 ChangingAsset)
{
	SourceChannelProperties->Initialize(Targets, false);

	UUVToolAssetAndChannelAPI* AssetAndChannelAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolAssetAndChannelAPI>();
	if (AssetAndChannelAPI)
	{
		ActiveChannel = AssetAndChannelAPI->GetCurrentChannelVisibility()[ChangingAsset];

		SourceChannelProperties->Asset = SourceChannelProperties->UVAssetNames[ChangingAsset];
		SourceChannelProperties->TargetChannel = SourceChannelProperties->GetUVChannelNames()[ActiveChannel];
		SourceChannelProperties->ValidateUVChannelSelection(true);
		SourceChannelProperties->SilentUpdateWatched();
	}
}

void UUVEditorChannelEditTool::OnTick(float DeltaTime)
{
	if (PendingAction != EChannelEditToolAction::NoAction)
	{
		ActiveAsset = SourceChannelProperties->GetSelectedAssetID();
		ActiveChannel = SourceChannelProperties->GetSelectedChannelIndex(true, false);
		ReferenceChannel = SourceChannelProperties->GetSelectedChannelIndex(true, true);

		switch (PendingAction)
		{
		case EChannelEditToolAction::Add:
		{
			AddChannel();
		}
		break;

		case EChannelEditToolAction::Copy:
		{
			CopyChannel();
		}
		break;

		case EChannelEditToolAction::Delete:
		{
			DeleteChannel();
		}
		break;

		default:
			checkSlow(false);
		}

		PendingAction = EChannelEditToolAction::NoAction;
	}
}

void UUVEditorChannelEditTool::ApplyVisbleChannelChange()
{
	SourceChannelProperties->ValidateUVAssetSelection(true);
	SourceChannelProperties->ValidateUVChannelSelection(true);

	ActiveAsset = SourceChannelProperties->GetSelectedAssetID();
	ActiveChannel = SourceChannelProperties->GetSelectedChannelIndex(true, false);

	UUVToolAssetAndChannelAPI* AssetAndChannelAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolAssetAndChannelAPI>();
	if (AssetAndChannelAPI)
	{
		TArray<int32> ChannelPerAsset = AssetAndChannelAPI->GetCurrentChannelVisibility();
		ChannelPerAsset[ActiveAsset] = ActiveChannel;
		AssetAndChannelAPI->RequestChannelVisibilityChange(ChannelPerAsset, true);
	}
}

void UUVEditorChannelEditTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	GetToolManager()->DisplayMessage(LOCTEXT("OnStartUVChannelEditTool", "Add, copy or delete UV channels"),
		EToolMessageLevel::UserNotification);
}

EChannelEditToolAction UUVEditorChannelEditTool::ActiveAction() const
{
	if (ActionSelectionProperties)
	{
		return ActionSelectionProperties->Action;
	}
	return EChannelEditToolAction::NoAction;
}

void UUVEditorChannelEditTool::RequestAction(EChannelEditToolAction ActionType)
{
	PendingAction = ActionType;
}

void UUVEditorChannelEditTool::AddChannel()
{
	UUVToolAssetAndChannelAPI* AssetAndChannelAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolAssetAndChannelAPI>();
	FDynamicMeshUVEditor DynamicMeshUVEditor(Targets[ActiveAsset]->AppliedCanonical.Get(), ActiveChannel, false);
	int32 NewChannelIndex = DynamicMeshUVEditor.AddUVLayer();

	if (NewChannelIndex != -1)
	{
		// Analytics
		FActionHistoryItem Item;
		Item.ActionType = EChannelEditToolAction::Add;
		Item.FirstOperandIndex = NewChannelIndex;
		AnalyticsActionHistory.Add(Item);

		Targets[ActiveAsset]->AppliedPreview->PreviewMesh->UpdatePreview(Targets[ActiveAsset]->AppliedCanonical.Get());

		EmitChangeAPI->BeginUndoTransaction(UUVEditorChannelEditLocals::UVChannelAddTransactionName);
		EmitChangeAPI->EmitToolIndependentChange(this,
			MakeUnique<UUVEditorChannelEditLocals::FInputObjectUVChannelAdd>(Targets[ActiveAsset], NewChannelIndex),
			UUVEditorChannelEditLocals::UVChannelAddTransactionName);

		AssetAndChannelAPI->NotifyOfAssetChannelCountChange(ActiveAsset);
		TArray<int32> ChannelPerAsset = AssetAndChannelAPI->GetCurrentChannelVisibility();
		ChannelPerAsset[ActiveAsset] = NewChannelIndex;
		AssetAndChannelAPI->RequestChannelVisibilityChange(ChannelPerAsset, true);

		EmitChangeAPI->EndUndoTransaction();

		SourceChannelProperties->Initialize(Targets, false);
		SourceChannelProperties->TargetChannel = SourceChannelProperties->GetUVChannelNames()[NewChannelIndex];
	}

	GetToolManager()->DisplayMessage(LOCTEXT("AddChannelNotification", "New UV Channel added."), EToolMessageLevel::UserNotification);
}

void UUVEditorChannelEditTool::CopyChannel()
{
	// Analytics
	FActionHistoryItem Item;
	Item.ActionType = EChannelEditToolAction::Copy;
	Item.FirstOperandIndex = ReferenceChannel;
	Item.SecondOperandIndex = ActiveChannel;
	AnalyticsActionHistory.Add(Item);

	EmitChangeAPI->EmitToolIndependentChange(this,
		MakeUnique<UUVEditorChannelEditLocals::FInputObjectUVChannelCopy>(Targets[ActiveAsset], ReferenceChannel, ActiveChannel, 
			*Targets[ActiveAsset]->AppliedCanonical->Attributes()->GetUVLayer(ActiveChannel)),
		UUVEditorChannelEditLocals::UVChannelCopyTransactionName);

	FDynamicMeshUVEditor DynamicMeshUVEditor(Targets[ActiveAsset]->AppliedCanonical.Get(), ActiveChannel, false);
	DynamicMeshUVEditor.CopyUVLayer(Targets[ActiveAsset]->AppliedCanonical->Attributes()->GetUVLayer(ReferenceChannel));
	Targets[ActiveAsset]->UpdateAllFromAppliedCanonical();

	SourceChannelProperties->Initialize(Targets, false);

	GetToolManager()->DisplayMessage(LOCTEXT("CopyChannelNotification", "UV Channel copied."), EToolMessageLevel::UserNotification);
}

void UUVEditorChannelEditTool::DeleteChannel()
{
	UUVToolAssetAndChannelAPI* AssetAndChannelAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolAssetAndChannelAPI>();

	int32 TotalLayerCount = Targets[ActiveAsset]->AppliedCanonical->Attributes()->NumUVLayers();
	bool bClearInstead = TotalLayerCount == 1;
	
	// Analytics
	FActionHistoryItem Item;
	Item.ActionType = EChannelEditToolAction::Delete;
	Item.FirstOperandIndex = ActiveChannel;
	Item.bDeleteActionWasActuallyClear = bClearInstead;
	AnalyticsActionHistory.Add(Item);

	EmitChangeAPI->EmitToolIndependentChange(this,
		MakeUnique<UUVEditorChannelEditLocals::FInputObjectUVChannelDelete>(Targets[ActiveAsset], ActiveChannel, 
			*Targets[ActiveAsset]->AppliedCanonical->Attributes()->GetUVLayer(ActiveChannel), bClearInstead),
		UUVEditorChannelEditLocals::UVChannelDeleteTransactionName);

	UUVEditorChannelEditLocals::DeleteChannel(Targets[ActiveAsset], ActiveChannel, AssetAndChannelAPI, bClearInstead);

	SourceChannelProperties->Initialize(Targets, false);
	SourceChannelProperties->TargetChannel = SourceChannelProperties->GetUVChannelNames()[Targets[ActiveAsset]->UVLayerIndex];

	GetToolManager()->DisplayMessage(LOCTEXT("DeleteChannelNotification", "UV Channel deleted."), EToolMessageLevel::UserNotification);
}

void UUVEditorChannelEditTool::RecordAnalytics()
{
	using namespace UVEditorAnalytics;
	
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));
	
	// Tool inputs
	InputTargetAnalytics.AppendToAttributes(Attributes, "Input");

	// Tool stats
	auto MaybeAppendStatsToAttributes = [this, &Attributes](EChannelEditToolAction ActionType)
	{
		// For now we only care about the number of times actions were issued
		int32 NumActions = 0;
		
		for (const FActionHistoryItem& Item : AnalyticsActionHistory)
		{
			if (Item.ActionType == ActionType) {
				NumActions += 1;
			}
		}
		
		if (NumActions > 0)
		{
			const FString ActionName = StaticEnum<EChannelEditToolAction>()->GetNameStringByIndex(static_cast<int>(ActionType));
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Stats.%sAction.NumActions"), *ActionName), NumActions));
		}
	};
	MaybeAppendStatsToAttributes(EChannelEditToolAction::Add);
	MaybeAppendStatsToAttributes(EChannelEditToolAction::Delete);
	MaybeAppendStatsToAttributes(EChannelEditToolAction::Copy);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.ToolActiveDuration"), (FDateTime::UtcNow() - ToolStartTimeAnalytics).ToString()));
	
	FEngineAnalytics::GetProvider().RecordEvent(UVEditorAnalyticsEventName(TEXT("ChannelEditTool")), Attributes);

	if constexpr (false)
	{
		for (const FAnalyticsEventAttribute& Attr : Attributes)
		{
			UE_LOGF(LogGeometry, Log, "Debug %ls.%ls = %ls", *UVEditorAnalyticsEventName(TEXT("ChannelEditTool")), *Attr.GetName(), *Attr.GetValue());
		}
	}
}

#undef LOCTEXT_NAMESPACE
