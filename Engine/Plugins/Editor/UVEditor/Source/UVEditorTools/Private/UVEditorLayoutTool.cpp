// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorLayoutTool.h"

#include "ContextObjectStore.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "ParameterizationOps/UVLayoutOp.h"
#include "Properties/UVLayoutProperties.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "EngineAnalytics.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "UVEditorUXSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorLayoutTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UUVEditorLayoutTool"

// Tool builder
// TODO: Could consider sharing some of the tool builder boilerplate for UV editor tools in a common base class.

bool UUVEditorLayoutToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return Targets && Targets->Num() > 0;
}

UInteractiveTool* UUVEditorLayoutToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UUVEditorLayoutTool* NewTool = NewObject<UUVEditorLayoutTool>(SceneState.ToolManager);
	NewTool->SetTargets(*Targets);

	return NewTool;
}


void UUVEditorLayoutTool::Setup()
{
	check(Targets.Num() > 0);
	
	ToolStartTimeAnalytics = FDateTime::UtcNow();

	UInteractiveTool::Setup();

	Settings = NewObject<UUVLayoutProperties>(this);
	Settings->RestoreProperties(this);
	Settings->bUDIMCVAREnabled = (FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport.GetValueOnGameThread() > 0);
	AddToolPropertySource(Settings);

	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();
	UVToolSelectionAPI = ContextStore->FindContext<UUVToolSelectionAPI>();

	UVTool2DViewportAPI = ContextStore->FindContext<UUVTool2DViewportAPI>();

	UUVToolSelectionAPI::FHighlightOptions HighlightOptions;
	HighlightOptions.bBaseHighlightOnPreviews = true;
	HighlightOptions.bAutoUpdateUnwrap = true;
	UVToolSelectionAPI->SetHighlightOptions(HighlightOptions);
	UVToolSelectionAPI->SetHighlightVisible(true, false, true);

	auto SetupOpFactory = [this](UUVEditorToolMeshInput& Target, const FUVToolSelection* Selection)
	{
		TObjectPtr<UUVLayoutOperatorFactory> Factory = NewObject<UUVLayoutOperatorFactory>();
		Factory->TargetTransform = Target.AppliedPreview->PreviewMesh->GetTransform();
		Factory->Settings = Settings;
		Factory->OriginalMesh = Target.AppliedCanonical;
		Factory->GetSelectedUVChannel = [&Target]() { return Target.UVLayerIndex; };
		if (Selection)
		{
			Factory->Selection.Emplace(Selection->GetConvertedSelection(*Selection->Target->UnwrapCanonical,
				                                                        UE::Geometry::FUVToolSelection::EType::Triangle).SelectedIDs);
		}

		if (UVTool2DViewportAPI)
		{
			TMap<int32, int32> TextureResolutionPerUDIM;
			for (const FUDIMBlock& UDIM : UVTool2DViewportAPI->GetUDIMBlocks())
			{
				TextureResolutionPerUDIM.Add(UDIM.UDIM, UDIM.TextureResolution);
			}
			Factory->TextureResolutionPerUDIM = TextureResolutionPerUDIM;
		}

		Target.AppliedPreview->ChangeOpFactory(Factory);
		Target.AppliedPreview->OnMeshUpdated.AddWeakLambda(this, [this, &Target](UMeshOpPreviewWithBackgroundCompute* Preview) {
			Target.UpdateUnwrapPreviewFromAppliedPreview();

			this->UVToolSelectionAPI->RebuildUnwrapHighlight(Preview->PreviewMesh->GetTransform());
			});

		Target.AppliedPreview->InvalidateResult();
		return Factory;
	};

	if (UVToolSelectionAPI->HaveSelections())
	{
		Factories.Reserve(UVToolSelectionAPI->GetSelections().Num());
		for (FUVToolSelection Selection : UVToolSelectionAPI->GetSelections())
		{
			Factories.Add(SetupOpFactory(*Selection.Target, &Selection));
		}
	}
	else
	{
		Factories.Reserve(Targets.Num());
		for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
		{
			Factories.Add(SetupOpFactory(*Targets[TargetIndex], nullptr));
		}
	}

	SetToolDisplayName(LOCTEXT("ToolName", "UV Layout"));
	GetToolManager()->DisplayMessage(LOCTEXT("OnStartUVLayoutTool", "Translate, rotate or scale existing UV Charts using various strategies"),
		EToolMessageLevel::UserNotification);
	
	// Analytics
	InputTargetAnalytics = UVEditorAnalytics::CollectTargetAnalytics(Targets);
}

void UUVEditorLayoutTool::Shutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->OnMeshUpdated.RemoveAll(this);
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		UUVToolEmitChangeAPI* ChangeAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolEmitChangeAPI>();
		const FText TransactionName(LOCTEXT("LayoutTransactionName", "Layout Tool"));
		ChangeAPI->BeginUndoTransaction(TransactionName);

		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Set things up for undo. 
			// TODO: It's not entirely clear whether it would be safe to use a FMeshVertexChange instead... It seems like
			// when bAllowFlips is true, we would end up with changes to the tris of the unwrap. Also, if we stick to saving
			// all the tris and verts, should we consider using the new dynamic mesh serialization?
			FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
			ChangeTracker.BeginChange();
			
			for (int32 Tid : Target->UnwrapCanonical->TriangleIndicesItr())
			{
				ChangeTracker.SaveTriangle(Tid, true);
			}

			// TODO: Again, it's not clear whether we need to update the entire triangle topology...
			Target->UpdateCanonicalFromPreviews();

			ChangeAPI->EmitToolIndependentUnwrapCanonicalChange(Target, ChangeTracker.EndChange(), LOCTEXT("ApplyLayoutTool", "Layout Tool"));
		}

		ChangeAPI->EndUndoTransaction();

		// Analytics
		RecordAnalytics();
	}
	else
	{
		// Reset the inputs
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			Target->UpdatePreviewsFromCanonical();
		}
	}


	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->ClearOpFactory();
	}

	for (int32 FactoryIndex = 0; FactoryIndex < Factories.Num(); ++FactoryIndex)
	{
		Factories[FactoryIndex] = nullptr;
	}

	Settings = nullptr;
	Targets.Empty();
}

void UUVEditorLayoutTool::OnTick(float DeltaTime)
{
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->Tick(DeltaTime);
	}
}



void UUVEditorLayoutTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->InvalidateResult();
	}
}

bool UUVEditorLayoutTool::CanAccept() const
{
	if (UVToolSelectionAPI->HaveSelections())
	{
		for (FUVToolSelection Selection : UVToolSelectionAPI->GetSelections())
		{
			if (!Selection.Target->AppliedPreview->HaveValidResult())
			{
				return false;
			}
		}
	}
	else
	{
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			if (!Target->AppliedPreview->HaveValidResult())
			{
				return false;
			}
		}
	}
	return true;
}

void UUVEditorLayoutTool::RecordAnalytics()
{
	using namespace UVEditorAnalytics;
	
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));

	// Tool inputs
	InputTargetAnalytics.AppendToAttributes(Attributes, "Input");

	// Tool stats
	if (CanAccept())
	{
		TArray<double> PerAssetValidResultComputeTimes;
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Note: This would log -1 if the result was invalid, but checking CanAccept above ensures results are valid
			PerAssetValidResultComputeTimes.Add(Target->AppliedPreview->GetValidResultComputeTime());
		}
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.PerAsset.ComputeTimeSeconds"), PerAssetValidResultComputeTimes));
	}
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.ToolActiveDuration"), (FDateTime::UtcNow() - ToolStartTimeAnalytics).ToString()));

	// Tool settings chosen by the user
	Attributes.Add(AnalyticsEventAttributeEnum(TEXT("Settings.LayoutType"), Settings->LayoutType));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.TextureResolution"), Settings->TextureResolution));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Scale"), Settings->Scale));
	const TArray<FVector2D::FReal> TranslationArray({Settings->Translation.X, Settings->Translation.Y});
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Translation"), TranslationArray));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.AllowFlips"), Settings->bAllowFlips));

	FEngineAnalytics::GetProvider().RecordEvent(UVEditorAnalyticsEventName(TEXT("LayoutTool")), Attributes);

	if constexpr (false)
	{
		for (const FAnalyticsEventAttribute& Attr : Attributes)
		{
			UE_LOGF(LogGeometry, Log, "Debug %ls.%ls = %ls", *UVEditorAnalyticsEventName(TEXT("LayoutTool")), *Attr.GetName(), *Attr.GetValue());
		}
	}
}

#undef LOCTEXT_NAMESPACE

