// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorLogging.h"

DEFINE_LOG_CATEGORY(LogUVEditor);
