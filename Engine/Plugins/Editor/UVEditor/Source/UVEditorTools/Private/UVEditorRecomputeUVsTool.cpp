// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorRecomputeUVsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Polygroups/PolygroupUtil.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "ContextObjectStore.h"
#include "MeshOpPreviewHelpers.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "EngineAnalytics.h"
#include "ParameterizationOps/RecomputeUVsOp.h"
#include "Properties/RecomputeUVsProperties.h"
#include "UVEditorUXSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorRecomputeUVsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UUVEditorRecomputeUVsTool"


/*
 * ToolBuilder
 */

bool UUVEditorRecomputeUVsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return Targets && Targets->Num() > 0;
}

UInteractiveTool* UUVEditorRecomputeUVsToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UUVEditorRecomputeUVsTool* NewTool = NewObject<UUVEditorRecomputeUVsTool>(SceneState.ToolManager);
	NewTool->SetTargets(*Targets);

	return NewTool;
}


/*
 * Tool
 */


void UUVEditorRecomputeUVsTool::Setup()
{
	ToolStartTimeAnalytics = FDateTime::UtcNow();
	
	UInteractiveTool::Setup();

	// initialize our properties

	Settings = NewObject<URecomputeUVsToolProperties>(this);
	Settings->RestoreProperties(this);
	Settings->bUDIMCVAREnabled = (FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport.GetValueOnGameThread() > 0);
	AddToolPropertySource(Settings);

	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();
	UVToolSelectionAPI = ContextStore->FindContext<UUVToolSelectionAPI>();

	UUVToolSelectionAPI::FHighlightOptions HighlightOptions;
	HighlightOptions.bBaseHighlightOnPreviews = true;
	HighlightOptions.bAutoUpdateUnwrap = true;

	UVToolSelectionAPI->SetHighlightOptions(HighlightOptions);
	UVToolSelectionAPI->SetHighlightVisible(true, false, true);


	if (Targets.Num() == 1)
	{
		PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
		PolygroupLayerProperties->RestoreProperties(this, TEXT("UVEditorRecomputeUVsTool"));
		PolygroupLayerProperties->InitializeGroupLayers(Targets[0]->AppliedCanonical.Get());
		PolygroupLayerProperties->WatchProperty(PolygroupLayerProperties->ActiveGroupLayer, [&](FName) { OnSelectedGroupLayerChanged(); });
		AddToolPropertySource(PolygroupLayerProperties);		
	}
	else
	{
		Settings->bEnablePolygroupSupport = false;
		Settings->IslandGeneration = ERecomputeUVsPropertiesIslandMode::ExistingUVs;
	}
	UpdateActiveGroupLayer(false);  /* Don't update factories that don't exist yet. */

	auto SetupOpFactory = [this](UUVEditorToolMeshInput& Target, const FUVToolSelection* Selection)
	{
		TObjectPtr<URecomputeUVsOpFactory> Factory = NewObject<URecomputeUVsOpFactory>();
		Factory->TargetTransform = (FTransform3d)Target.AppliedPreview->PreviewMesh->GetTransform();
		Factory->Settings = Settings;
		Factory->OriginalMesh = Target.AppliedCanonical;
		Factory->InputGroups = ActiveGroupSet;
		Factory->GetSelectedUVChannel = [&Target]() { return Target.UVLayerIndex; };
		if (Selection)
		{
			Factory->Selection.Emplace(Selection->GetConvertedSelection(*Selection->Target->UnwrapCanonical,
				                                                        UE::Geometry::FUVToolSelection::EType::Triangle).SelectedIDs);
		}

		Target.AppliedPreview->ChangeOpFactory(Factory);
		Target.AppliedPreview->OnMeshUpdated.AddWeakLambda(this, [this, &Target](UMeshOpPreviewWithBackgroundCompute* Preview) {
			if (Preview->HaveValidNonEmptyResult())
			{
				Target.UpdateUnwrapPreviewFromAppliedPreview();

				this->UVToolSelectionAPI->RebuildUnwrapHighlight(Preview->PreviewMesh->GetTransform());
			}
			});		

		Target.AppliedPreview->InvalidateResult();
		return Factory;
	};

	if (UVToolSelectionAPI->HaveSelections())
	{
		Factories.Reserve(UVToolSelectionAPI->GetSelections().Num());
		for (FUVToolSelection Selection : UVToolSelectionAPI->GetSelections())
		{
			Factories.Add(SetupOpFactory(*Selection.Target, &Selection));
		}
	}
	else
	{
		Factories.Reserve(Targets.Num());
		for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
		{
			Factories.Add(SetupOpFactory(*Targets[TargetIndex], nullptr));
		}
	}
	

	SetToolDisplayName(LOCTEXT("ToolNameLocal", "UV Unwrap"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool_Regions", "Generate UVs for PolyGroups or existing UV islands of the mesh using various strategies."),
		EToolMessageLevel::UserNotification);
	
	// Analytics
	InputTargetAnalytics = UVEditorAnalytics::CollectTargetAnalytics(Targets);
}


void UUVEditorRecomputeUVsTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PropertySet == Settings )
	{
		// One of the UV generation properties must have changed.  Dirty the result to force a recompute
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			Target->AppliedPreview->InvalidateResult();
		}
	}
}


void UUVEditorRecomputeUVsTool::Shutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);
	if (PolygroupLayerProperties) {
		PolygroupLayerProperties->RestoreProperties(this, TEXT("UVEditorRecomputeUVsTool"));
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		UUVToolEmitChangeAPI* ChangeAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolEmitChangeAPI>();

		const FText TransactionName(LOCTEXT("RecomputeUVsTransactionName", "Recompute UVs"));
		ChangeAPI->BeginUndoTransaction(TransactionName);
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Set things up for undo. 
			FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
			ChangeTracker.BeginChange();

			for (int32 Tid : Target->UnwrapCanonical->TriangleIndicesItr())
			{
				ChangeTracker.SaveTriangle(Tid, true);
			}

			Target->UpdateCanonicalFromPreviews();

			ChangeAPI->EmitToolIndependentUnwrapCanonicalChange(Target, ChangeTracker.EndChange(), LOCTEXT("ApplyRecomputeUVsTool", "Unwrap Tool"));
		}

		ChangeAPI->EndUndoTransaction();

		// Analytics
		RecordAnalytics();
	}
	else
	{
		// Reset the inputs
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			Target->UpdatePreviewsFromCanonical();
		}
	}

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->ClearOpFactory();
	}
	for (int32 FactoryIndex = 0; FactoryIndex < Factories.Num(); ++FactoryIndex)
	{
		Factories[FactoryIndex] = nullptr;
	}
	Settings = nullptr;
	Targets.Empty();
}

void UUVEditorRecomputeUVsTool::OnTick(float DeltaTime)
{
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->Tick(DeltaTime);
	}
}

void UUVEditorRecomputeUVsTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}



bool UUVEditorRecomputeUVsTool::CanAccept() const
{
	if (UVToolSelectionAPI->HaveSelections())
	{
		for (FUVToolSelection Selection : UVToolSelectionAPI->GetSelections())
		{
			if (!Selection.Target->AppliedPreview->HaveValidResult())
			{
				return false;
			}
		}
	}
	else
	{
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			if (!Target->AppliedPreview->HaveValidResult())
			{
				return false;
			}
		}
	}
	return true;
}


void UUVEditorRecomputeUVsTool::OnSelectedGroupLayerChanged()
{
	UpdateActiveGroupLayer();
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->InvalidateResult();
	}
}


void UUVEditorRecomputeUVsTool::UpdateActiveGroupLayer(bool bUpdateFactories)
{
	if (Targets.Num() == 1)
	{
		if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
		{
			ActiveGroupSet = MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(Targets[0]->AppliedCanonical.Get());
		}
		else
		{
			FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
			FDynamicMeshPolygroupAttribute* FoundAttrib = UE::Geometry::FindPolygroupLayerByName(*Targets[0]->AppliedCanonical, SelectedName);
			ensureMsgf(FoundAttrib, TEXT("Selected attribute not found! Falling back to Default group layer."));
			ActiveGroupSet = MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(Targets[0]->AppliedCanonical.Get(), FoundAttrib);
		}
	}
	else
	{
		ActiveGroupSet = nullptr;
	}

	if (bUpdateFactories)
	{
		for (int32 FactoryIdx = 0; FactoryIdx < Factories.Num(); ++FactoryIdx)
		{
			Factories[FactoryIdx]->InputGroups = ActiveGroupSet;
		}
	}
}


void UUVEditorRecomputeUVsTool::RecordAnalytics()
{
	using namespace UVEditorAnalytics;
	
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));

	// Tool inputs
	InputTargetAnalytics.AppendToAttributes(Attributes, "Input");

	// Tool outputs	
	const FTargetAnalytics OutputTargetAnalytics = CollectTargetAnalytics(Targets);
	OutputTargetAnalytics.AppendToAttributes(Attributes, "Output");
	
	// Tool stats
	if (CanAccept())
	{
		TArray<double> PerAssetValidResultComputeTimes;
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Note: This would log -1 if the result was invalid, but checking CanAccept above ensures results are valid
			PerAssetValidResultComputeTimes.Add(Target->AppliedPreview->GetValidResultComputeTime());
		}
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.PerAsset.ComputeTimeSeconds"), PerAssetValidResultComputeTimes));
	}
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.ToolActiveDuration"), (FDateTime::UtcNow() - ToolStartTimeAnalytics).ToString()));
	
	// Tool settings chosen by the user (Volatile! Sync with EditCondition meta-tags in URecomputeUVsToolProperties)
	
	Attributes.Add(AnalyticsEventAttributeEnum(TEXT("Settings.IslandGeneration"), Settings->IslandGeneration));
	Attributes.Add(AnalyticsEventAttributeEnum(TEXT("Settings.AutoRotation"), Settings->AutoRotation));
	
	Attributes.Add(AnalyticsEventAttributeEnum(TEXT("Settings.UnwrapType"), Settings->UnwrapType));
	if (Settings->UnwrapType == ERecomputeUVsPropertiesUnwrapType::IslandMerging || Settings->UnwrapType == ERecomputeUVsPropertiesUnwrapType::ExpMap)
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.SmoothingSteps"), Settings->SmoothingSteps));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.SmoothingAlpha"), Settings->SmoothingAlpha));
	}
	if (Settings->UnwrapType == ERecomputeUVsPropertiesUnwrapType::IslandMerging)
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.MergingDistortionThreshold"), Settings->MergingDistortionThreshold));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.MergingAngleThreshold"), Settings->MergingAngleThreshold));
	}
	
	Attributes.Add(AnalyticsEventAttributeEnum(TEXT("Settings.LayoutType"), Settings->LayoutType));
	if (Settings->LayoutType == ERecomputeUVsPropertiesLayoutType::Repack)
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.TextureResolution"), Settings->TextureResolution));
	}
	if (Settings->LayoutType == ERecomputeUVsPropertiesLayoutType::NormalizeToBounds || Settings->LayoutType == ERecomputeUVsPropertiesLayoutType::NormalizeToWorld)
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.NormalizeScale"), Settings->NormalizeScale));
	}

	FEngineAnalytics::GetProvider().RecordEvent(UVEditorAnalyticsEventName(TEXT("UnwrapTool")), Attributes);

	if constexpr (false)
	{
		for (const FAnalyticsEventAttribute& Attr : Attributes)
		{
			UE_LOGF(LogGeometry, Log, "Debug %ls.%ls = %ls", *UVEditorAnalyticsEventName(TEXT("UnwrapTool")), *Attr.GetName(), *Attr.GetValue());
		}
	}
}

#undef LOCTEXT_NAMESPACE

