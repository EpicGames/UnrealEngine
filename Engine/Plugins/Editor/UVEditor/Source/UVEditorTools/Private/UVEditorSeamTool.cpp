// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorSeamTool.h"

#include "Algo/Reverse.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ContextObjectStore.h"
#include "Drawing/PreviewGeometryActor.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "InputBehaviorSet.h"
#include "InputRouter.h"
#include "MeshOpPreviewHelpers.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/MeshDijkstra.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "ToolSceneQueriesUtil.h"
#include "UVEditorUXSettings.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "EngineAnalytics.h"
#include "Algo/Unique.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorSeamTool)

#define LOCTEXT_NAMESPACE "UUVEditorSeamTool"

using namespace UE::Geometry;

namespace UVEditorSeamToolLocals
{
	const FText EditSeamTransactionName(LOCTEXT("EditSeamTransactionName", "Edit Working Seam"));
	const FText ApplySeamTransactionName(LOCTEXT("ApplySeamTransactionName", "Apply Seam"));

	const FString& HoverPointSetID(TEXT("HoverPointSet"));
	const FString& HoverLineSetID(TEXT("HoverLineSet"));

	const FString& LockedPointSetID(TEXT("LockedPointSet"));
	const FString& LockedLineSetID(TEXT("LockedLineSet"));

	const FString& StartPointSetID(TEXT("StartPointSet"));
	
	const FString& ExistingSeamsID(TEXT("SeamLineSet"));

	void AddDisplayedPoints(UUVEditorToolMeshInput* InputObject,
		UPointSetComponent* UnwrapPointSet, UPointSetComponent* AppliedPointSet,
		int32 AppliedVid, const TArray<int32>& UnwrapVids, const FColor& Color, float DepthBias)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AddDisplayedPoints);

		const FTransform AppliedTransform = InputObject->AppliedPreview->PreviewMesh->GetTransform();
		AppliedPointSet->AddPoint(FRenderablePoint(
			AppliedTransform.TransformPosition(InputObject->AppliedCanonical->GetVertex(AppliedVid)), Color, FUVEditorUXSettings::ToolPointSize, DepthBias));
		for (const int32 UnwrapVid : UnwrapVids)
		{
			UnwrapPointSet->AddPoint(FRenderablePoint(
				InputObject->UnwrapCanonical->GetVertex(UnwrapVid), Color, FUVEditorUXSettings::ToolPointSize, DepthBias));
		}
	}

	void AddDisplayedPath(UUVEditorToolMeshInput* InputObject, 
		ULineSetComponent* UnwrapLineSet, ULineSetComponent* AppliedLineSet,
		const TArray<int32>& NewPathVids, bool bPathIsFromUnwrap, bool bForceDisplaySeamPaths,
		const FColor& Color, float Thickness, float DepthBias)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AddDisplayedPath);

		FTransform AppliedTransform = InputObject->AppliedPreview->PreviewMesh->GetTransform();
		if (NewPathVids.Num() < 2)
		{
			return;
		}

		// Adds a path by simply walking through new path vids and using the given vid to position function.
		auto AddSimplePath = [&NewPathVids, Color, Thickness, DepthBias](ULineSetComponent* LineSet,
			TFunctionRef<FVector3d(int32)> VidToPosition)
		{
			FVector3d Vec1, Vec2;
			FVector3d* EndPoint1 = &Vec1;
			FVector3d* EndPoint2 = &Vec2;

			*EndPoint1 = VidToPosition(NewPathVids[0]);
			for (int32 i = 1; i < NewPathVids.Num(); ++i)
			{
				*EndPoint2 = VidToPosition(NewPathVids[i]);
				LineSet->AddLine(FRenderableLine((FVector)*EndPoint1, (FVector)*EndPoint2,
					Color, Thickness, DepthBias));
				Swap(EndPoint1, EndPoint2);
			}
		};

		// Add the path into the live preview.

		TUniqueFunction<FVector3d(int32)> VidToAppliedMeshPosition = [InputObject, &AppliedTransform](int32 Vid) {
			return AppliedTransform.TransformPosition(InputObject->AppliedCanonical->GetVertex(Vid));
		};
		if (bPathIsFromUnwrap)
		{
			VidToAppliedMeshPosition = [InputObject, &AppliedTransform](int32 Vid) {
				return AppliedTransform.TransformPosition(
					InputObject->AppliedCanonical->GetVertex(
						InputObject->UnwrapVidToAppliedVid(Vid)));
			};
		}
		AddSimplePath(AppliedLineSet, VidToAppliedMeshPosition);

		// The work we have to do for the unwrap is actually very different depending on whether
		// the path came from the unwrap or the applied mesh, because an applied mesh vertex may
		// correspond to multiple unwrap vertices.
		if (bPathIsFromUnwrap)
		{
			// The easy case, same as for live preview
			TUniqueFunction<FVector3d(int32)> VidToUnwrapPosition = [InputObject](int32 Vid) {
				return InputObject->UnwrapCanonical->GetVertex(Vid);
			};
			AddSimplePath(UnwrapLineSet, VidToUnwrapPosition);
		}
		else
		{
			// Since the tool supports both cutting and joining seams, we want to display all matching edges, so we
			// can highlight seams in the unwrap mesh when selecting on the applied mesh.
			auto GetUnwrapEids = [InputObject](const TArray<int32>& Vids1, const TArray<int32>& Vids2)
			{
				TArray<int32> Eids;
				for (int32 Vid1 : Vids1)
				{
					for (int32 Vid2 : Vids2)
					{
						int32 Eid = InputObject->UnwrapCanonical->FindEdge(Vid1, Vid2);
						if (Eid != IndexConstants::InvalidID)
						{
							Eids.Add(Eid);
						}
					}
				}
				Eids.Sort();
				Eids.SetNum(Algo::Unique(Eids));
				return Eids;
			};

			TArray<int32> Vids1, Vids2;
			TArray<int32>* EndPoints1 = &Vids1;
			TArray<int32>* EndPoints2 = &Vids2;

			InputObject->AppliedVidToUnwrapVids(NewPathVids[0], *EndPoints1);
			for (int32 i = 1; i < NewPathVids.Num(); ++i)
			{
				InputObject->AppliedVidToUnwrapVids(NewPathVids[i], *EndPoints2);
				TArray<int32> UnwrapEids = GetUnwrapEids(*EndPoints1, *EndPoints2);

				// Normally we want to just want to render unwrap edges if there's a one to one relationship
				// between the applied mesh, but in other cases, like when doing seam joining, we want to
				// render all replicas to show where joins will happen.
				if (bForceDisplaySeamPaths || UnwrapEids.Num() == 1)
				{
					for (int32 UnwrapEid : UnwrapEids)
					{
						FIndex2i EdgeVids = InputObject->UnwrapCanonical->GetEdgeV(UnwrapEid);
						UnwrapLineSet->AddLine(FRenderableLine(
							(FVector)InputObject->UnwrapCanonical->GetVertex(EdgeVids.A),
							(FVector)InputObject->UnwrapCanonical->GetVertex(EdgeVids.B),
							Color, Thickness, DepthBias));
					}
				}
				Swap(EndPoints1, EndPoints2);
			}
		}//end if unwrap path is not from unwrap
	}//end AddDisplayedPath()

	/**
	 * Applies/reverts changes to LockedPath.
	 */
	class FPathChange : public FToolCommandChange
	{
	public:
		/**
		 * @param bIgnoreFirstPathVid If true, first vid in PathVidsIn is not considered. Useful
		 *  because in cases where we're adding to an existing path, the first vert in the new
		 *  path exists in the previous path.
		 */
		FPathChange(const TArray<int32> PathVidsIn, bool bRemovedIn, int32 MeshIndexIn, 
			bool bIgnoreFirstPathVid = false)
			: bRemoved(bRemovedIn)
			, MeshIndex(MeshIndexIn)
		{
			for (int32 i = bIgnoreFirstPathVid ? 1 : 0; i < PathVidsIn.Num(); ++i)
			{
				PathVids.Add(PathVidsIn[i]);
			}
		}

		virtual void Apply(UObject* Object) override
		{
			UUVEditorSeamTool* Tool = Cast<UUVEditorSeamTool>(Object);
			if (bRemoved)
			{
				Tool->EditLockedPath([this](TArray<int32>& Path) {
					if (ensure(Path.Num() >= PathVids.Num()))
					{
						Path.SetNum(Path.Num() - PathVids.Num());
					}
				}, MeshIndex);
			}
			else
			{
				Tool->EditLockedPath([this](TArray<int32>& Path) {
					Path.Append(PathVids);
				}, MeshIndex);
			}
		}

		virtual void Revert(UObject* Object) override
		{
			UUVEditorSeamTool* Tool = Cast<UUVEditorSeamTool>(Object);
			if (bRemoved)
			{
				Tool->EditLockedPath([this](TArray<int32>& Path) {
					Path.Append(PathVids);
				}, MeshIndex);
			}
			else
			{
				Tool->EditLockedPath([this](TArray<int32>& Path) {
					if (ensure(Path.Num() >= PathVids.Num()))
					{
						Path.SetNum(Path.Num() - PathVids.Num());
					}
				}, MeshIndex);
			}
		}

		virtual FString ToString() const override
		{
			return TEXT("UVEditorSeamToolLocals::FPathChange");
		}

	protected:
		TArray<int32> PathVids;
		bool bRemoved = false;
		int32 MeshIndex = IndexConstants::InvalidID;
	};
}

bool UUVEditorSeamToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return Targets && Targets->Num() > 0;
}

UInteractiveTool* UUVEditorSeamToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UUVEditorSeamTool* NewTool = NewObject<UUVEditorSeamTool>(SceneState.ToolManager);
	NewTool->SetTargets(*Targets);

	return NewTool;
}

void UUVEditorSeamTool::SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
{
	Targets = TargetsIn;
}

void UUVEditorSeamTool::Setup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorSeamTool_Setup);
	
	ToolStartTimeAnalytics = FDateTime::UtcNow();

	using namespace UVEditorSeamToolLocals;

	UInteractiveTool::Setup();

	// We cache the bounding box max dimensions because we use these to adjust our
	//  path similarity metric when balancing against path length.
	UnwrapMaxDims.SetNum(Targets.Num());
	AppliedMeshMaxDims.SetNum(Targets.Num());
	for (int32 i = 0; i < Targets.Num(); ++i)
	{
		UnwrapMaxDims[i] = Targets[i]->UnwrapCanonical->GetBounds(true).MaxDim();
		AppliedMeshMaxDims[i] = Targets[i]->AppliedCanonical->GetBounds(true).MaxDim();
	}

	Settings = NewObject<UUVEditorSeamToolProperties>();
	Settings->RestoreProperties(this);
	Settings->WatchProperty(Settings->Mode, [&](EUVEditorSeamMode) { OnSeamModeChanged(); });
	AddToolPropertySource(Settings);

	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();
	EmitChangeAPI = ContextStore->FindContext<UUVToolEmitChangeAPI>();
	LivePreviewAPI = ContextStore->FindContext<UUVToolLivePreviewAPI>();
	checkSlow(EmitChangeAPI && LivePreviewAPI);

	// Set things up for being able to add behaviors to live preview.
	LivePreviewBehaviorSet = NewObject<UInputBehaviorSet>();
	LivePreviewBehaviorSource = NewObject<ULocalInputBehaviorSource>();
	LivePreviewBehaviorSource->GetInputBehaviorsFunc = [this]() { return LivePreviewBehaviorSet; };

	// Set up click and hover behaviors both for the 2d (unwrap) viewport and the 3d 
	// (applied/live preview) viewport
	ULocalSingleClickInputBehavior* UnwrapClickBehavior = NewObject<ULocalSingleClickInputBehavior>();
	UnwrapClickBehavior->Initialize();
	UnwrapClickBehavior->IsHitByClickFunc = [this](const FInputDeviceRay& InputRay) {
		FInputRayHit HitResult;
		HitResult.bHit = Get2DHitVertex(InputRay.WorldRay) != IndexConstants::InvalidID;
		// We don't bother with the depth since there aren't competing click behaviors
		return HitResult;
	};
	UnwrapClickBehavior->OnClickedFunc = [this](const FInputDeviceRay& ClickPos) {
		int32 IndexOfMesh = IndexConstants::InvalidID;
		int32 Vid = Get2DHitVertex(ClickPos.WorldRay, &IndexOfMesh);
		if (Vid != IndexConstants::InvalidID)
		{
			OnMeshVertexClicked(Vid, IndexOfMesh, true);
		}
	};
	AddInputBehavior(UnwrapClickBehavior);

	ULocalMouseHoverBehavior* UnwrapHoverBehavior = NewObject<ULocalMouseHoverBehavior>();
	UnwrapHoverBehavior->Initialize();
	UnwrapHoverBehavior->BeginHitTestFunc = [this](const FInputDeviceRay& PressPos) {
		FInputRayHit HitResult;
		HitResult.bHit = Get2DHitVertex(PressPos.WorldRay) != IndexConstants::InvalidID;
		// We don't bother with the depth since there aren't competing hover behaviors
		return HitResult;
	};
	UnwrapHoverBehavior->OnUpdateHoverFunc = [this](const FInputDeviceRay& DevicePos)
	{
		int32 IndexOfMesh = IndexConstants::InvalidID;
		int32 Vid = Get2DHitVertex(DevicePos.WorldRay, &IndexOfMesh);
		if (Vid != IndexConstants::InvalidID)
		{
			OnMeshVertexHovered(Vid, IndexOfMesh, true);
			return true;
		}
		return false;
	};
	UnwrapHoverBehavior->OnEndHoverFunc = [this]()
	{
		ClearHover();
	};
	UnwrapHoverBehavior->Modifiers.RegisterModifier(TemporaryModeToggleModifierID, FInputDeviceState::IsShiftKeyDown);
	UnwrapHoverBehavior->OnUpdateModifierStateFunc = [this](int ModifierID, bool bIsOn) {
		if (ModifierID == TemporaryModeToggleModifierID && bModeIsTemporarilyToggled != bIsOn)
		{
			bModeIsTemporarilyToggled = bIsOn;
			ClearHover();
			UpdateHover();
			ResetPreviewColors();		
		}
	};
	AddInputBehavior(UnwrapHoverBehavior);

	// Now the applied/live preview viewport...
	ULocalSingleClickInputBehavior* LivePreviewClickBehavior = NewObject<ULocalSingleClickInputBehavior>();
	LivePreviewClickBehavior->Initialize();
	LivePreviewClickBehavior->IsHitByClickFunc = [this](const FInputDeviceRay& InputRay) {
		FInputRayHit HitResult;
		HitResult.bHit = Get3DHitVertex(InputRay.WorldRay) != IndexConstants::InvalidID;
		// We don't bother with the depth since there aren't competing click behaviors
		return HitResult;
	};
	LivePreviewClickBehavior->OnClickedFunc = [this](const FInputDeviceRay& ClickPos) {
		int32 IndexOfMesh = IndexConstants::InvalidID;
		int32 Vid = Get3DHitVertex(ClickPos.WorldRay, &IndexOfMesh);
		if (Vid != IndexConstants::InvalidID)
		{
			OnMeshVertexClicked(Vid, IndexOfMesh, false);
		}
	};

	LivePreviewBehaviorSet->Add(LivePreviewClickBehavior, this);

	ULocalMouseHoverBehavior* LivePreviewHoverBehavior = NewObject<ULocalMouseHoverBehavior>();
	LivePreviewHoverBehavior->Initialize();
	LivePreviewHoverBehavior->BeginHitTestFunc = [this](const FInputDeviceRay& PressPos) {
		FInputRayHit HitResult;
		HitResult.bHit = Get3DHitVertex(PressPos.WorldRay) != IndexConstants::InvalidID;
		// We don't bother with the depth since there aren't competing hover behaviors
		return HitResult;
	};
	LivePreviewHoverBehavior->OnUpdateHoverFunc = [this](const FInputDeviceRay& DevicePos)
	{
		int32 IndexOfMesh = IndexConstants::InvalidID;
		int32 Vid = Get3DHitVertex(DevicePos.WorldRay, &IndexOfMesh);
		if (Vid != IndexConstants::InvalidID)
		{
			OnMeshVertexHovered(Vid, IndexOfMesh, false);
			return true;
		}
		return false;
	};
	LivePreviewHoverBehavior->OnEndHoverFunc = [this]()
	{
		ClearHover();
	};
	LivePreviewHoverBehavior->Modifiers.RegisterModifier(TemporaryModeToggleModifierID, FInputDeviceState::IsShiftKeyDown);
	LivePreviewHoverBehavior->OnUpdateModifierStateFunc = [this](int ModifierID, bool bIsOn)
	{
		if (ModifierID == TemporaryModeToggleModifierID && bModeIsTemporarilyToggled != bIsOn)
		{
			bModeIsTemporarilyToggled = bIsOn;
			ClearHover();
			UpdateHover();
			ResetPreviewColors();
		}
	};
	LivePreviewBehaviorSet->Add(LivePreviewHoverBehavior, this);

	// Give the live preview behaviors to the live preview input router
	if (LivePreviewAPI)
	{
		LivePreviewInputRouter = LivePreviewAPI->GetLivePreviewInputRouter();
		LivePreviewInputRouter->RegisterSource(LivePreviewBehaviorSource);
	}

	// Set up all the components we need to visualize things.
	UnwrapGeometry = NewObject<UPreviewGeometry>();
	UnwrapGeometry->CreateInWorld(Targets[0]->UnwrapPreview->GetWorld(), FTransform::Identity);
	
	LivePreviewGeometry = NewObject<UPreviewGeometry>();
	LivePreviewGeometry->CreateInWorld(Targets[0]->AppliedPreview->GetWorld(), FTransform::Identity);

	// These visualize the locked-in portion of the current seam
	UnwrapGeometry->AddPointSet(LockedPointSetID);
	UnwrapGeometry->AddLineSet(LockedLineSetID);
	LivePreviewGeometry->AddPointSet(LockedPointSetID);
	LivePreviewGeometry->AddLineSet(LockedLineSetID);

	// These visualize the start points of the locked-in seam. It's handy to have them
	// separate so that they don't get cleared when the last endpoints get updated.
	UnwrapGeometry->AddPointSet(StartPointSetID);
	LivePreviewGeometry->AddPointSet(StartPointSetID);

	// These visualize the currently hovered (preview) portion of the seam
	UnwrapGeometry->AddPointSet(HoverPointSetID);
	UnwrapGeometry->AddLineSet(HoverLineSetID);
	LivePreviewGeometry->AddPointSet(HoverPointSetID);
	LivePreviewGeometry->AddLineSet(HoverLineSetID);

	// These visualize the existing seams on the 3d preview. These can get initialized now.
	LivePreviewGeometry->AddLineSet(ExistingSeamsID);
	ReconstructExistingSeamsVisualization();


	UUVToolAABBTreeStorage* TreeStore = ContextStore->FindContext<UUVToolAABBTreeStorage>();
	if (!TreeStore)
	{
		TreeStore = NewObject<UUVToolAABBTreeStorage>();
		ContextStore->AddContextObject(TreeStore);
	}

	// Initialize the AABB trees from cached values, or make new ones.
	for (int32 i = 0; i < Targets.Num(); ++i)
	{
		TSharedPtr<FDynamicMeshAABBTree3> Tree2d = TreeStore->Get(Targets[i]->UnwrapCanonical.Get());
		if (!Tree2d)
		{
			Tree2d = MakeShared<FDynamicMeshAABBTree3>();
			Tree2d->SetMesh(Targets[i]->UnwrapCanonical.Get());
			TreeStore->Set(Targets[i]->UnwrapCanonical.Get(), Tree2d, Targets[i]);
		}
		Spatials2D.Add(Tree2d);

		TSharedPtr<FDynamicMeshAABBTree3> Tree3d = TreeStore->Get(Targets[i]->AppliedCanonical.Get());
		if (!Tree3d)
		{
			Tree3d = MakeShared<FDynamicMeshAABBTree3>();
			Tree3d->SetMesh(Targets[i]->AppliedCanonical.Get());
			TreeStore->Set(Targets[i]->AppliedCanonical.Get(), Tree3d, Targets[i]);
		}
		Spatials3D.Add(Tree3d);

		// Make sure we update the seam visualization after mesh undo/redo
		Targets[i]->OnCanonicalModified.AddWeakLambda(this, [this]
		(UUVEditorToolMeshInput* InputObject, const UUVEditorToolMeshInput::FCanonicalModifiedInfo& Info) {
			ReconstructExistingSeamsVisualization();
		});
	}

	UpdateToolMessage();

	// Analytics
	InputTargetAnalytics = UVEditorAnalytics::CollectTargetAnalytics(Targets);
}

void UUVEditorSeamTool::ReconstructExistingSeamsVisualization()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorSeamTool_ReconstructExistingSeamsVisualization);

	using namespace UVEditorSeamToolLocals;

	ULineSetComponent* SeamLines = LivePreviewGeometry->FindLineSet(ExistingSeamsID);
	SeamLines->Clear();
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		FDynamicMesh3* Mesh = Target->AppliedCanonical.Get();
		FDynamicMeshUVOverlay* Overlay = Target->AppliedCanonical->Attributes()->GetUVLayer(Target->UVLayerIndex);
		FTransform Transform = Target->AppliedPreview->PreviewMesh->GetTransform();
		for (int32 Eid : Mesh->EdgeIndicesItr())
		{
			if (Overlay->IsSeamEdge(Eid))
			{
				FIndex2i EdgeVids = Mesh->GetEdgeV(Eid);
				SeamLines->AddLine((FVector)Transform.TransformPosition(Mesh->GetVertex(EdgeVids.A)),
					(FVector)Transform.TransformPosition(Mesh->GetVertex(EdgeVids.B)),
					FUVEditorUXSettings::LivePreviewExistingSeamColor,
					FUVEditorUXSettings::LivePreviewExistingSeamThickness,
					FUVEditorUXSettings::LivePreviewExistingSeamDepthBias);
			}
		}
	}
}

void UUVEditorSeamTool::ReconstructLockedPathVisualization()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorSeamTool_ReconstructLockedPathVisualization);

	using namespace UVEditorSeamToolLocals;

	UPointSetComponent* UnwrapPointSet = UnwrapGeometry->FindPointSet(LockedPointSetID);
	UPointSetComponent* AppliedPointSet = LivePreviewGeometry->FindPointSet(LockedPointSetID);
	ULineSetComponent* UnwrapLineSet = UnwrapGeometry->FindLineSet(LockedLineSetID);
	ULineSetComponent* AppliedLineSet = LivePreviewGeometry->FindLineSet(LockedLineSetID);
	UPointSetComponent* UnwrapStartPoints = UnwrapGeometry->FindPointSet(StartPointSetID);
	UPointSetComponent* AppliedStartPoints = LivePreviewGeometry->FindPointSet(StartPointSetID);

	UnwrapLineSet->Clear();
	AppliedLineSet->Clear();
	UnwrapPointSet->Clear();
	AppliedPointSet->Clear();
	UnwrapStartPoints->Clear();
	AppliedStartPoints->Clear();

	if (ClickedMeshIndex != IndexConstants::InvalidID)
	{
		UUVEditorToolMeshInput* ClickedTarget = Targets[ClickedMeshIndex];

		AddDisplayedPath(ClickedTarget, UnwrapLineSet, AppliedLineSet, LockedPath,
			false, IsInJoinMode(), GetLockedPathColor(),
			FUVEditorUXSettings::ToolLockedPathThickness,
			FUVEditorUXSettings::ToolLockedPathDepthBias);

		TArray<int32> UnwrapVids;
		ClickedTarget->AppliedVidToUnwrapVids(LastLockedAppliedVid, UnwrapVids);
		AddDisplayedPoints(ClickedTarget, UnwrapPointSet, AppliedPointSet, LastLockedAppliedVid, UnwrapVids, 
			GetLockedPathColor(), FUVEditorUXSettings::ToolLockedPathDepthBias);

		UnwrapVids.Reset();
		ClickedTarget->AppliedVidToUnwrapVids(SeamStartAppliedVid, UnwrapVids);
		AddDisplayedPoints(ClickedTarget, UnwrapStartPoints, AppliedStartPoints, SeamStartAppliedVid, UnwrapVids,
			GetLockedPathColor(), FUVEditorUXSettings::ToolLockedPathDepthBias);
	}
}

void UUVEditorSeamTool::UpdateToolMessage()
{
	switch (State)
	{
	case EState::WaitingToStart:
		GetToolManager()->DisplayMessage(LOCTEXT("StartSeamMessage", "Click to start a seam in the 2D or 3D viewport."),
			EToolMessageLevel::UserNotification);
		break;
	case EState::SeamInProgress:
		GetToolManager()->DisplayMessage(LOCTEXT("CompleteSeamMessage", "To complete the seam, press Enter or click either on the first or the last point. Press Esc to cancel."),
			EToolMessageLevel::UserNotification);
		break;
	default:
		ensure(false);
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserNotification);
		break;
	}
}

void UUVEditorSeamTool::Shutdown(EToolShutdownType ShutdownType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorSeamTool_Shutdown);

	if (LivePreviewInputRouter.IsValid())
	{
		// TODO: Arguably the live preview input router should do this for us before Shutdown(), but
		// we don't currently have support for that...
		LivePreviewInputRouter->DeregisterSource(LivePreviewBehaviorSource);
	}

	// Apply any pending seam if needed
	if (ShutdownType != EToolShutdownType::Cancel && LockedPath.Num() > 0)
	{		
		ApplySeam(LockedPath);
	}

	UInteractiveTool::Shutdown(ShutdownType);

	EmitChangeAPI = nullptr;
	LivePreviewAPI = nullptr;

	LivePreviewBehaviorSet->RemoveAll();
	LivePreviewBehaviorSet = nullptr;
	LivePreviewBehaviorSource = nullptr;

	UnwrapGeometry->Disconnect();
	LivePreviewGeometry->Disconnect();
	UnwrapGeometry = nullptr;
	LivePreviewGeometry = nullptr;

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->OnCanonicalModified.RemoveAll(this);
	}

	Spatials2D.Reset();
	Spatials3D.Reset();

	// Analytics
	RecordAnalytics();
}

void UUVEditorSeamTool::OnTick(float DeltaTime)
{
	using namespace UVEditorSeamToolLocals;

	if (LivePreviewAPI)
	{
		LivePreviewAPI->GetLivePreviewCameraState(LivePreviewCameraState);
	}

	// This clears hover if necessary, etc. Includes checks to avoid redoing work, so
	// no checks needed here.
	UpdateHover();

	// Apply click if there was one (signaled by a non-InvalidId vid)
	if (ClickedVid != IndexConstants::InvalidID)
	{
		ApplyClick();
		ClickedVid = IndexConstants::InvalidID;
	}
}

void UUVEditorSeamTool::UpdateHover()
{
	using namespace UVEditorSeamToolLocals;

	if (bLastHoverVidWasFromUnwrap == bHoverVidIsFromUnwrap
		&& LastHoverMeshIndex == HoverMeshIndex
		&& LastHoverVid == HoverVid)
	{
		return;
	}

	bLastHoverVidWasFromUnwrap = bHoverVidIsFromUnwrap;
	LastHoverMeshIndex = HoverMeshIndex;
	LastHoverVid = HoverVid;

	ClearHover(false);

	if (HoverVid == IndexConstants::InvalidID)
	{
		return;
	}

	int32 AppliedVid = bHoverVidIsFromUnwrap ? Targets[HoverMeshIndex]->UnwrapVidToAppliedVid(HoverVid) : HoverVid;
	TArray<int32> UnwrapVids;
	Targets[HoverMeshIndex]->AppliedVidToUnwrapVids(AppliedVid, UnwrapVids);

	UPointSetComponent* UnwrapPointSet = UnwrapGeometry->FindPointSet(HoverPointSetID);
	UPointSetComponent* AppliedPointSet = LivePreviewGeometry->FindPointSet(HoverPointSetID);

	if (State == EState::WaitingToStart)
	{
		// Just draw the point and finish
		AddDisplayedPoints(Targets[HoverMeshIndex], UnwrapPointSet, AppliedPointSet,
			AppliedVid, UnwrapVids, GetExtendPathColor(), FUVEditorUXSettings::ToolExtendPathDepthBias);
		return;
	}

	// Otherwise, try to find a path.
	TArray<int32> NewPathVids;
	if (AppliedVid != LastLockedAppliedVid)
	{
		if (bHoverVidIsFromUnwrap)
		{
			TArray<int32> LastLockedUnwrapVids;
			Targets[HoverMeshIndex]->AppliedVidToUnwrapVids(LastLockedAppliedVid, LastLockedUnwrapVids);
			GetVidPath(*Targets[HoverMeshIndex], bHoverVidIsFromUnwrap, 
				LastLockedUnwrapVids, HoverVid, NewPathVids);
		}
		else
		{
			ensure(LastLockedAppliedVid != IndexConstants::InvalidID);
			GetVidPath(*Targets[HoverMeshIndex], bHoverVidIsFromUnwrap, 
				{ LastLockedAppliedVid }, HoverVid, NewPathVids);
		}
	}

	// Draw the path, if found.
	if (NewPathVids.Num() > 0)
	{
		ULineSetComponent* UnwrapLineSet = UnwrapGeometry->FindLineSet(HoverLineSetID);
		ULineSetComponent* AppliedLineSet = LivePreviewGeometry->FindLineSet(HoverLineSetID);
		AddDisplayedPath(Targets[HoverMeshIndex], UnwrapLineSet, AppliedLineSet,
			NewPathVids, bHoverVidIsFromUnwrap, IsInJoinMode(), GetExtendPathColor(),
			FUVEditorUXSettings::ToolExtendPathThickness, FUVEditorUXSettings::ToolExtendPathDepthBias);
	}

	// See if we would apply the seam on click
	if (AppliedVid == LastLockedAppliedVid || AppliedVid == SeamStartAppliedVid)
	{
		bCompletionColorOverride = true;
		ResetPreviewColors();
	}
	else if (NewPathVids.Num() > 0)
	{
		// Otherwise draw the new point we're hovering. Don't do this if we didn't find a path
		// there because that turns out to be confusing, esp when we have overlapping islands and
		// the point is on another island.
		AddDisplayedPoints(Targets[HoverMeshIndex], UnwrapPointSet, AppliedPointSet,
			AppliedVid, UnwrapVids, GetExtendPathColor(), FUVEditorUXSettings::ToolExtendPathDepthBias);
	}
}

void UUVEditorSeamTool::ClearHover(bool bClearHoverInfo)
{
	using namespace UVEditorSeamToolLocals;

	UnwrapGeometry->FindPointSet(HoverPointSetID)->Clear();
	LivePreviewGeometry->FindPointSet(HoverPointSetID)->Clear();
	UnwrapGeometry->FindLineSet(HoverLineSetID)->Clear();
	LivePreviewGeometry->FindLineSet(HoverLineSetID)->Clear();

	if (bClearHoverInfo)
	{
		HoverVid = IndexConstants::InvalidID;
		HoverMeshIndex = IndexConstants::InvalidID;
		LastHoverVid = HoverVid;
		LastHoverMeshIndex = HoverMeshIndex;
	}

	if (bCompletionColorOverride)
	{
		bCompletionColorOverride = false;
		ResetPreviewColors();
	}
}

void UUVEditorSeamTool::ResetPreviewColors()
{
	using namespace UVEditorSeamToolLocals;

	FColor Color = bCompletionColorOverride ? FUVEditorUXSettings::ToolCompletionPathColor 
		: GetExtendPathColor();

	UnwrapGeometry->FindPointSet(HoverPointSetID)->SetAllPointsColor(Color);
	LivePreviewGeometry->FindPointSet(HoverPointSetID)->SetAllPointsColor(Color);
	UnwrapGeometry->FindLineSet(HoverLineSetID)->SetAllLinesColor(Color);
	LivePreviewGeometry->FindLineSet(HoverLineSetID)->SetAllLinesColor(Color);

	Color = bCompletionColorOverride ? FUVEditorUXSettings::ToolCompletionPathColor 
		: GetLockedPathColor();

	UnwrapGeometry->FindPointSet(LockedPointSetID)->SetAllPointsColor(Color);
	LivePreviewGeometry->FindPointSet(LockedPointSetID)->SetAllPointsColor(Color);
	UnwrapGeometry->FindLineSet(LockedLineSetID)->SetAllLinesColor(Color);
	LivePreviewGeometry->FindLineSet(LockedLineSetID)->SetAllLinesColor(Color);
	UnwrapGeometry->FindPointSet(StartPointSetID)->SetAllPointsColor(Color);
	LivePreviewGeometry->FindPointSet(StartPointSetID)->SetAllPointsColor(Color);
}

void UUVEditorSeamTool::ApplyClick()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorSeamTool_ApplyClick);

	using namespace UVEditorSeamToolLocals;

	ClearHover();

	UUVEditorToolMeshInput* ClickedTarget = Targets[ClickedMeshIndex];
	FTransform AppliedTransform = ClickedTarget->AppliedPreview->PreviewMesh->GetTransform();
	
	int32 AppliedVid = bClickWasInUnwrap ? ClickedTarget->UnwrapVidToAppliedVid(ClickedVid) : ClickedVid;
	TArray<int32> UnwrapVids;
	ClickedTarget->AppliedVidToUnwrapVids(AppliedVid, UnwrapVids);

	if (State == EState::WaitingToStart)
	{
		SeamStartAppliedVid = AppliedVid;
		LastLockedAppliedVid = SeamStartAppliedVid;

		UPointSetComponent* UnwrapPointSet = UnwrapGeometry->FindPointSet(StartPointSetID);
		UPointSetComponent* AppliedPointSet = LivePreviewGeometry->FindPointSet(StartPointSetID);
		AddDisplayedPoints(ClickedTarget, UnwrapPointSet, AppliedPointSet, AppliedVid, UnwrapVids, 
			GetLockedPathColor(), FUVEditorUXSettings::ToolLockedPathDepthBias);

		LockedPath.Add(AppliedVid);
		EmitChangeAPI->EmitToolDependentChange(this, 
			MakeUnique<FPathChange>(LockedPath, false, ClickedMeshIndex),
			EditSeamTransactionName);

		State = EState::SeamInProgress;
		UpdateToolMessage();
		return;
	}

	// Otherwise, we're working on a seam...
	// Find a path for the new section
	TArray<int32> VidPath;
	TArray<int32>* AppliedVidPath = &VidPath;
	TArray<int32> TempStorage; // Only used if we need spare storage for AppliedVids
	if (bClickWasInUnwrap)
	{
		TArray<int32> LastLockedUnwrapVids;
		ClickedTarget->AppliedVidToUnwrapVids(LastLockedAppliedVid, LastLockedUnwrapVids);
		GetVidPath(*ClickedTarget, bClickWasInUnwrap, LastLockedUnwrapVids, ClickedVid, VidPath);
		for (int32 Vid : VidPath)
		{
			TempStorage.Add(ClickedTarget->UnwrapVidToAppliedVid(Vid));
		}
		AppliedVidPath = &TempStorage;
	}
	else
	{
		GetVidPath(*ClickedTarget, bClickWasInUnwrap, { LastLockedAppliedVid }, ClickedVid, VidPath);
	}

	if (VidPath.Num() == 0)
	{
		// TODO: Write an error message complaining that path couldn't be found?
		return;
	}

	// See if we completed a seam
	if (AppliedVid == LastLockedAppliedVid || AppliedVid == SeamStartAppliedVid)
	{
		// We keep LockedPath unchanged and create a separate path copy to pass to ApplyPath
		// so that when we clear LockedPath, the undo transaction would bring us back to the
		// pre-completed state.
		TArray<int32> SeamVids = LockedPath;
		if (ensure(SeamVids.Num() > 0)) {
			// Avoid duplicating the last point
			SeamVids.Pop();
		}
		SeamVids.Append(*AppliedVidPath);

		ApplySeam(SeamVids);
		return;
	}

	// If not completed, need to update path, visualization, and emit change
	if (ensure(LockedPath.Num() > 0)) {
		// Avoid duplicating the last point
		LockedPath.Pop();
	}
	LockedPath.Append(*AppliedVidPath);

	UPointSetComponent* UnwrapPointSet = UnwrapGeometry->FindPointSet(LockedPointSetID);
	UPointSetComponent* AppliedPointSet = LivePreviewGeometry->FindPointSet(LockedPointSetID);
	ULineSetComponent* UnwrapLineSet = UnwrapGeometry->FindLineSet(LockedLineSetID);
	ULineSetComponent* AppliedLineSet = LivePreviewGeometry->FindLineSet(LockedLineSetID);
	AddDisplayedPath(ClickedTarget, UnwrapLineSet, AppliedLineSet, VidPath,
		bClickWasInUnwrap, IsInJoinMode(), GetLockedPathColor(),
		FUVEditorUXSettings::ToolLockedPathThickness,
		FUVEditorUXSettings::ToolLockedPathDepthBias);
	UnwrapPointSet->Clear();
	AppliedPointSet->Clear();
	AddDisplayedPoints(ClickedTarget, UnwrapPointSet, AppliedPointSet, AppliedVid, UnwrapVids, 
		GetLockedPathColor(), FUVEditorUXSettings::ToolLockedPathDepthBias);
	
	EmitChangeAPI->EmitToolDependentChange(this, 
		MakeUnique<FPathChange>(*AppliedVidPath, false, ClickedMeshIndex, true),
		EditSeamTransactionName);

	LastLockedAppliedVid = AppliedVid;
}

void UUVEditorSeamTool::ApplySeam(const TArray<int32>& AppliedVidsIn)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorSeamTool_ApplySeam);

	using namespace UVEditorSeamToolLocals;

	UUVEditorToolMeshInput* ClickedTarget = Targets[ClickedMeshIndex];
	
	FDynamicMeshUVOverlay* UVOverlay = ClickedTarget->AppliedCanonical.Get()->Attributes()->GetUVLayer(ClickedTarget->UVLayerIndex);
	FDynamicMeshUVEditor UVEditor(ClickedTarget->AppliedCanonical.Get(), UVOverlay);

	// The locked path (represented by AppliedVidsIn) is used to create an actual UV seam or join existing
	// seams, depending on the current tool mode.
	
	// Convert vid path into set of eids path in the applied mesh
	TSet<int32> EidSet;
	for (int32 i = 0; i < AppliedVidsIn.Num() - 1; ++i)
	{
		int32 Eid = ClickedTarget->AppliedCanonical->FindEdge(AppliedVidsIn[i], AppliedVidsIn[i + 1]);
		if (ensure(Eid != IndexConstants::InvalidID))
		{
			EidSet.Add(Eid);
		}
	}

	TSet<int32> TidSet;	
	TArray<int32> VidSet;
	if (IsInJoinMode())
	{
		UVEditor.RemoveSeamsAtEdges(EidSet);

		// Don't actually need to update VidSet because no element values changed or were created, only connectivity.
		for (int32 Vid : AppliedVidsIn)
		{
			FDynamicMesh3::FLocalIntArray VertTids;
			ClickedTarget->AppliedCanonical->GetVtxTriangles(Vid, VertTids);
			TidSet.Append(VertTids);
		}
	}
	else
	{
		FUVEditResult UVEditResult;
		UVEditor.CreateSeamsAtEdges(EidSet, &UVEditResult);

		for (int32 UnwrapVid : UVEditResult.NewUVElements)
		{
			FDynamicMesh3::FLocalIntArray VertTids;
			ClickedTarget->AppliedCanonical->GetVtxTriangles(ClickedTarget->UnwrapVidToAppliedVid(UnwrapVid), VertTids);
			TidSet.Append(VertTids);
		}
		VidSet = UVEditResult.NewUVElements;
	}

	FDynamicMeshChangeTracker ChangeTracker(ClickedTarget->UnwrapCanonical.Get());
	ChangeTracker.BeginChange();
	ChangeTracker.SaveTriangles(TidSet, true);

	TArray<int32> AppliedTids = TidSet.Array();
	ClickedTarget->UpdateAllFromAppliedCanonical(&VidSet, &AppliedTids, &AppliedTids);
	Spatials2D[ClickedMeshIndex]->Build();

	// Emit transaction
	EmitChangeAPI->BeginUndoTransaction(ApplySeamTransactionName);
	 // emits locked path change
	ClearLockedPath();
	EmitChangeAPI->EmitToolIndependentUnwrapCanonicalChange(
		ClickedTarget, ChangeTracker.EndChange(), EditSeamTransactionName);
	EmitChangeAPI->EndUndoTransaction();

	ReconstructExistingSeamsVisualization();
}

void UUVEditorSeamTool::ClearLockedPath(bool bEmitChange)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorSeamTool_ClearLockedPath);

	using namespace UVEditorSeamToolLocals;

	if (bEmitChange)
	{
		EmitChangeAPI->EmitToolDependentChange(this,
			MakeUnique<FPathChange>(LockedPath, true, ClickedMeshIndex),
			EditSeamTransactionName);
	}
	LockedPath.Reset();

	UnwrapGeometry->FindPointSet(LockedPointSetID)->Clear();
	LivePreviewGeometry->FindPointSet(LockedPointSetID)->Clear();
	UnwrapGeometry->FindLineSet(LockedLineSetID)->Clear();
	LivePreviewGeometry->FindLineSet(LockedLineSetID)->Clear();
	UnwrapGeometry->FindPointSet(StartPointSetID)->Clear();
	LivePreviewGeometry->FindPointSet(StartPointSetID)->Clear();

	ClearHover();

	ClickedMeshIndex = IndexConstants::InvalidID;
	ClickedVid = IndexConstants::InvalidID;
	SeamStartAppliedVid = IndexConstants::InvalidID;
	LastLockedAppliedVid = IndexConstants::InvalidID;

	State = EState::WaitingToStart;
	UpdateToolMessage();
}

int32 UUVEditorSeamTool::Get2DHitVertex(const FRay& WorldRayIn, int32* IndexOf2DSpatialOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorSeamTool_Get2DHitVertex);

	if (!ensure(WorldRayIn.Direction.Z != 0))
	{
		return IndexConstants::InvalidID;
	}

	// Find plane hit location
	double PlaneHitT = WorldRayIn.Origin.Z / -WorldRayIn.Direction.Z;
	FVector3d PlanePoint = WorldRayIn.PointAt(PlaneHitT);

	int32 Result = IndexConstants::InvalidID;
	if (State == EState::WaitingToStart)
	{
		// For tolerance, we'd like to divide the screen into 90 sections to be similar to the 90 degree
		// FOV angle tolerance we use in 3D. Ray Z should be our distance from ground plane, so width
		// would be 2Z for 90 degree FOV.
		double Tolerance = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD() * WorldRayIn.Origin.Z / 45;

		double MinDistSquared = TNumericLimits<double>::Max();
		for (int32 i = 0; i < Spatials2D.Num(); ++i)
		{
			double DistSquared = TNumericLimits<double>::Max();
			int32 Vid = Spatials2D[i]->FindNearestVertex(PlanePoint, DistSquared, Tolerance);

			if (DistSquared >= 0 && DistSquared < MinDistSquared)
			{
				MinDistSquared = DistSquared;
				Result = Vid;
				if (IndexOf2DSpatialOut)
				{
					*IndexOf2DSpatialOut = i;
				}
			}
		}
	}
	else if (State == EState::SeamInProgress)
	{
		// This part goes about the process slightly differently than above, because in cases
		// of drawing internal seams, we want to pick not just the nearest vertex to the mouse
		// position, but the nearest vertex on the nearest triangle. Otherwise, we might end up
		// picking the vertex on the other side of a recently drawn seam (which shares the same
		// spatial location) and forcing the path finding to route the long way around the mesh
		// to get to it.

		// We could call FindNearestTriangle followed by FindNearestVertex with the triangle set
		// as a filter, but for three vertices, it's probably faster to simply brute force the check.
		
		// We don't provide a tolerance here because we want to snap to nearest triangle and vertex even if 
		// we're far outside the mesh
		double Distance = -1;
		int32 Tid = Spatials2D[ClickedMeshIndex]->FindNearestTriangle(PlanePoint, Distance);
		if (Distance >= 0)
		{
			TArray<FVector3d> VertPositions;
			VertPositions.SetNum(3);
			Targets[ClickedMeshIndex]->UnwrapCanonical->GetTriVertices(Tid, VertPositions[0], VertPositions[1], VertPositions[2]);
			FIndex3i Vids = Targets[ClickedMeshIndex]->UnwrapCanonical->GetTriangle(Tid);
			Distance = TNumericLimits<double>::Max();
			for (uint32 VIndex = 0; VIndex < 3; ++VIndex)
			{
				if (FVector3d::DistSquared(VertPositions[VIndex], PlanePoint) < Distance)
				{
					Distance = FVector3d::DistSquared(VertPositions[VIndex], PlanePoint);
					Result = Vids[VIndex];
				}
			}

			if (IndexOf2DSpatialOut)
			{
				*IndexOf2DSpatialOut = ClickedMeshIndex;
			}
		}
	}

	return Result;
}

int32 UUVEditorSeamTool::Get3DHitVertex(const FRay& WorldRayIn, int32* IndexOf3DSpatialOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorSeamTool_Get3DHitVertex);

	// When we hit a triangle, gives the closest vert in terms of angle to camera (kind of). Tests
	// the three verts of the triangle we hit, and the closest vert to the hit location (to make it
	// easier to hit verts in very thin/degenerate tris).
	auto GetBestSnappedVert = [this](FDynamicMeshAABBTree3& Tree, const FTransform& MeshTransform,
		const FVector3d& HitPoint, int32 Tid, double& BestSnapMetricOut)
	{
		const FDynamicMesh3* Mesh = Tree.GetMesh();
		FIndex3i TriVids = Mesh->GetTriangle(Tid);

		double Unused = 0; // dummy variable for function call
		int32 VidCandidates[4] = { TriVids.A, TriVids.B, TriVids.C, Tree.FindNearestVertex(HitPoint, Unused) };
		
		FVector3d VertCandidates[4];
		Mesh->GetTriVertices(Tid, VertCandidates[0], VertCandidates[1], VertCandidates[2]);
		VertCandidates[3] = Mesh->GetVertex(VidCandidates[3]);

		BestSnapMetricOut = TNumericLimits<double>::Max();
		int32 BestVid = IndexConstants::InvalidID;
		for (int i = 0; i < 3; ++i)
		{
			double SnapMetric = ToolSceneQueriesUtil::PointSnapMetric(
				LivePreviewCameraState, HitPoint, MeshTransform.TransformPosition(VertCandidates[i]));

			if (SnapMetric < BestSnapMetricOut)
			{
				BestSnapMetricOut = SnapMetric;
				BestVid = VidCandidates[i];
			}
		}

		return BestVid;
	};


	if (State == EState::WaitingToStart)
	{
		// We'll hit test all the preview meshes
		int32 IndexOfBestSpatial = IndexConstants::InvalidID;
		double BestRayT = TNumericLimits<double>::Max();
		int32 BestTid = IndexConstants::InvalidID;
		for (int32 i = 0; i < Spatials3D.Num(); ++i)
		{
			FTransform Transform = Targets[i]->AppliedPreview->PreviewMesh->GetTransform();
			FRay3d LocalRay(
				(FVector3d)Transform.InverseTransformPosition(WorldRayIn.Origin),
				(FVector3d)Transform.InverseTransformVector(WorldRayIn.Direction));

			int32 Tid = IndexConstants::InvalidID;
			double RayT = TNumericLimits<double>::Max();
			if (Spatials3D[i]->FindNearestHitTriangle(LocalRay, RayT, Tid))
			{
				if (RayT >= 0 && RayT < BestRayT)
				{
					BestRayT = RayT;
					BestTid = Tid;
					IndexOfBestSpatial = i;
				}
			}
		}

		if (BestTid == IndexConstants::InvalidID)
		{
			// Didn't hit anything
			return IndexConstants::InvalidID;
		}

		// Find vertex that's within some angle tolerance
		double BestSnapMetric = TNumericLimits<double>::Max();
		FTransform Transform = Targets[IndexOfBestSpatial]->AppliedPreview->PreviewMesh->GetTransform();
		int32 BestVid = GetBestSnappedVert(*Spatials3D[IndexOfBestSpatial],
			Transform, WorldRayIn.PointAt(BestRayT), BestTid, BestSnapMetric);

		if (BestSnapMetric <= ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD())
		{
			if (IndexOf3DSpatialOut)
			{
				*IndexOf3DSpatialOut = IndexOfBestSpatial;
			}
			return BestVid;
		}
		return IndexConstants::InvalidID;
	}
	else if (State == EState::SeamInProgress)
	{
		// Only need to hit test the mesh we've already started on, and we always snap to nearest vert
		FTransform Transform = Targets[ClickedMeshIndex]->AppliedPreview->PreviewMesh->GetTransform();
		FRay3d LocalRay(
			(FVector3d)Transform.InverseTransformPosition(WorldRayIn.Origin),
			(FVector3d)Transform.InverseTransformVector(WorldRayIn.Direction));

		double RayT = TNumericLimits<double>::Max();
		int32 Tid = IndexConstants::InvalidID;
		if (Spatials3D[ClickedMeshIndex]->FindNearestHitTriangle(LocalRay, RayT, Tid))
		{
			double BestSnapMetric = TNumericLimits<double>::Max();
			if (IndexOf3DSpatialOut)
			{
				*IndexOf3DSpatialOut = ClickedMeshIndex;
			}
			return GetBestSnappedVert(*Spatials3D[ClickedMeshIndex],
				Transform, WorldRayIn.PointAt(RayT), Tid, BestSnapMetric);
		}
		return IndexConstants::InvalidID;
	}

	// Shouldn't get here: all enum cases should have returns above
	ensure(false);
	return IndexConstants::InvalidID;
}

void UUVEditorSeamTool::OnMeshVertexClicked(int32 Vid, int32 IndexOfMesh, bool bVidIsFromUnwrap)
{
	using namespace UVEditorSeamToolLocals;

	if (ClickedVid != IndexConstants::InvalidID)
	{
		// Haven't yet finished processing the last click
		return;
	}

	// Save the click information. The click will be applied on tick.
	ClickedVid = Vid;
	ClickedMeshIndex = IndexOfMesh;
	bClickWasInUnwrap = bVidIsFromUnwrap;
}

void UUVEditorSeamTool::OnMeshVertexHovered(int32 Vid, int32 IndexOfMesh, bool bVidIsFromUnwrap)
{
	using namespace UVEditorSeamToolLocals;

	// Save the hover information. Hover will be updated on tick
	HoverVid = Vid;
	HoverMeshIndex = IndexOfMesh;
	bHoverVidIsFromUnwrap = bVidIsFromUnwrap;
}

void UUVEditorSeamTool::EditLockedPath(
	TUniqueFunction<void(TArray<int32>& LockedPathInOut)> EditFunction, int32 MeshIndex)
{
	EditFunction(LockedPath);

	bool bPathNotEmpty = LockedPath.Num() > 0;
	State = bPathNotEmpty ? EState::SeamInProgress : EState::WaitingToStart;
	UpdateToolMessage();
	ClickedMeshIndex = bPathNotEmpty ? MeshIndex : IndexConstants::InvalidID;
	LastLockedAppliedVid = bPathNotEmpty ? LockedPath.Last() : IndexConstants::InvalidID;
	SeamStartAppliedVid = bPathNotEmpty ? LockedPath[0] : IndexConstants::InvalidID;

	ReconstructLockedPathVisualization();
	ClearHover();
}

bool UUVEditorSeamTool::CanCurrentlyNestedCancel()
{
	return LockedPath.Num() > 0;
}

bool UUVEditorSeamTool::ExecuteNestedCancelCommand()
{
	if (LockedPath.Num() > 0)
	{
		ClearLockedPath();
		return true;
	}
	return false;
}

bool UUVEditorSeamTool::CanCurrentlyNestedAccept()
{
	return LockedPath.Num() > 0;
}

bool UUVEditorSeamTool::ExecuteNestedAcceptCommand()
{
	if (LockedPath.Num() > 0)
	{
		ApplySeam(LockedPath);
		return true;
	}
	return false;
}

FColor UUVEditorSeamTool::GetLockedPathColor() const
{
	return !IsInJoinMode() ? FUVEditorUXSettings::ToolLockedCutPathColor :
     	                     FUVEditorUXSettings::ToolLockedJoinPathColor;
}


FColor UUVEditorSeamTool::GetExtendPathColor() const
{
	return !IsInJoinMode() ? FUVEditorUXSettings::ToolExtendCutPathColor :
    	                     FUVEditorUXSettings::ToolExtendJoinPathColor;
}

bool UUVEditorSeamTool::IsInJoinMode() const
{
	bool bIsJoinMode = Settings && Settings->Mode == EUVEditorSeamMode::Join;
	return bModeIsTemporarilyToggled ? !bIsJoinMode : bIsJoinMode;
}

void UUVEditorSeamTool::OnSeamModeChanged()
{
	bModeIsTemporarilyToggled = false;
	ResetPreviewColors();
}

void UUVEditorSeamTool::RecordAnalytics()
{
	using namespace UVEditorAnalytics;
	
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));
	
	// Tool inputs
	InputTargetAnalytics.AppendToAttributes(Attributes, "Input");

	// Tool outputs
	const FTargetAnalytics OutputTargetAnalytics = CollectTargetAnalytics(Targets);
	OutputTargetAnalytics.AppendToAttributes(Attributes, "Output");

	// Tool stats
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.ToolActiveDuration"), (FDateTime::UtcNow() - ToolStartTimeAnalytics).ToString()));
	
	FEngineAnalytics::GetProvider().RecordEvent(UVEditorAnalyticsEventName(TEXT("SeamTool")), Attributes);

	if constexpr (false)
	{
		for (const FAnalyticsEventAttribute& Attr : Attributes)
		{
			UE_LOGF(LogGeometry, Log, "Debug %ls.%ls = %ls", *UVEditorAnalyticsEventName(TEXT("SeamTool")), *Attr.GetName(), *Attr.GetValue());
		}
	}
}

void UUVEditorSeamTool::GetVidPath(const UUVEditorToolMeshInput& Target, bool bUnwrapMeshSource, 
	const TArray<int32>& StartVids, int32 EndVid, TArray<int32>& VidPathOut)
{
	VidPathOut.Reset();
	if (StartVids.Contains(EndVid))
	{
		VidPathOut.Add(EndVid);
		return;
	}

	FDynamicMesh3* Mesh = bUnwrapMeshSource ? Target.UnwrapCanonical.Get() : Target.AppliedCanonical.Get();

	UE::Geometry::TMeshDijkstra<FDynamicMesh3> PathFinder(Mesh);
	TArray<TMeshDijkstra<FDynamicMesh3>::FSeedPoint> SeedPoints;
	for (int32 StartVid : StartVids)
	{
		SeedPoints.Add({ StartVid, StartVid, 0 });
	}

	// See if we need a custom distance metric
	if (IsInJoinMode() || Settings->PathSimilarityWeight > 0)
	{
		PathFinder.bEnableDistanceWeighting = true;

		// We'll replace this with one that includes a similarity metric if the weight is nonzero
		TUniqueFunction<double(int32 FromVID, int32 ToVID, int32 SeedVID, double EuclideanDistance)> PathMetric = 
			[](int32 FromVid, int32 ToVid, int32 SeedVid, double EuclideanDistance) 
		{ 
			return EuclideanDistance; 
		};

		double MeshMaxDim = bUnwrapMeshSource ? UnwrapMaxDims[Target.AssetID] : AppliedMeshMaxDims[Target.AssetID];
		if (Settings->PathSimilarityWeight > 0 && MeshMaxDim > 0)
		{
			// Since we have multiple potential sources, we have multiple direct lines from source to destination
			//  that we can try to measure similarity against.
			TArray<FLine3d> DirectLines;
			FVector3d EndPosition = Mesh->GetVertex(EndVid);
			for (int32 StartVid : StartVids)
			{
				DirectLines.Add(FLine3d::FromPoints(Mesh->GetVertex(StartVid), EndPosition));
			}

			// Replace our metric
			PathMetric = [PathSimilarityWeight = Settings->PathSimilarityWeight, Mesh, MeshMaxDim, Lines = MoveTemp(DirectLines)]
				(int32 FromVid, int32 ToVid, int32 SeedVid, double EuclideanDistance)
			{
				// Get the lowest integrated squared distance from the lines
				double SimilarityMetric = TNumericLimits<double>::Max();
				for (const FLine3d& Line : Lines)
				{
					SimilarityMetric = FMath::Min(SimilarityMetric, SquaredDistanceFromLineIntegratedAlongSegment(
						Line, FSegment3d(Mesh->GetVertex(FromVid), Mesh->GetVertex(ToVid))));
				}

				// We want the relative path weights to not change if the mesh is uniformly scaled by a positive scalar s.
				//  Our similarity metric is proportional to s^3, so divide by max dim squared to make it proportional
				//  to s like EuclideanDistance, so that the final result is just scaled by s. We don't do cube root 
				//  because our adjustment needs to be distributive, to avoid being sensitive to path tesselation.
				SimilarityMetric /= (MeshMaxDim * MeshMaxDim);

				return EuclideanDistance + PathSimilarityWeight * SimilarityMetric;
			};
		}

		// If we're joining seams, put a seam discount on whatever our path metric is.
		if (IsInJoinMode())
		{
			constexpr double SeamEuclideanDistanceMultiplier = 0.01;
			if (bUnwrapMeshSource)
			{
				PathFinder.GetWeightedDistanceFunc = [Mesh, PathMetric = MoveTemp(PathMetric)](int32 FromVid, int32 ToVid, int32 SeedVid, double EuclideanDistance)
				{
					double Metric = PathMetric(FromVid, ToVid, SeedVid, EuclideanDistance);
					if (Mesh->IsBoundaryEdge(Mesh->FindEdge(FromVid, ToVid)))
					{
						return Metric * SeamEuclideanDistanceMultiplier;
					}
					return Metric;
				};
			}
			else
			{
				const FDynamicMeshUVOverlay* Overlay = Mesh->Attributes()->GetUVLayer(Target.UVLayerIndex);
				PathFinder.GetWeightedDistanceFunc = [Overlay, Mesh, PathMetric = MoveTemp(PathMetric)](int32 FromVid, int32 ToVid, int32 SeedVid, double EuclideanDistance)
				{
					double Metric = PathMetric(FromVid, ToVid, SeedVid, EuclideanDistance);
					
					if (Overlay->IsSeamEdge(Mesh->FindEdge(FromVid, ToVid)))
					{
						return Metric * SeamEuclideanDistanceMultiplier;
					}
					return Metric;
				};
			}
		}
		else
		{
			PathFinder.GetWeightedDistanceFunc = MoveTemp(PathMetric);
		}
	}

	if (PathFinder.ComputeToTargetPoint(SeedPoints, EndVid))
	{
		PathFinder.FindPathToNearestSeed(EndVid, VidPathOut);
	}
	Algo::Reverse(VidPathOut);
}

#undef LOCTEXT_NAMESPACE

