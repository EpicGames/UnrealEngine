// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorStyle.h"

#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/ToolBarStyle.h"

FName FUVEditorStyle::StyleName("UVStyle");

FUVEditorStyle::FUVEditorStyle()
	: FSlateStyleSet(StyleName)
{
	// Used FFractureEditorStyle as a model

	const FVector2D IconSize(16.0f, 16.0f);
	const FVector2D ToolbarIconSize(20.0f, 20.0f);

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/UVEditor/Content/Icons"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate") );

	Set("UVEditor.OpenUVEditor",				new IMAGE_BRUSH_SVG("UVEditor", IconSize));

	Set("UVEditor.BeginSelectTool",				new FSlateVectorImageBrush(FPaths::EngineContentDir() / TEXT("Slate/Starship/Common/edit.svg"), ToolbarIconSize));
	Set("UVEditor.BeginLayoutTool",             new IMAGE_BRUSH_SVG("UVLayout", ToolbarIconSize));
	Set("UVEditor.BeginParameterizeMeshTool",   new IMAGE_BRUSH_SVG("AutoUnwrap", ToolbarIconSize));
	Set("UVEditor.BeginChannelEditTool",        new IMAGE_BRUSH_SVG("AttributeEditor", ToolbarIconSize));
	Set("UVEditor.BeginSeamTool",               new IMAGE_BRUSH_SVG("ModelingUVSeamEdit", ToolbarIconSize));
	Set("UVEditor.BeginRecomputeUVsTool",       new IMAGE_BRUSH_SVG("GroupUnwrap", ToolbarIconSize));
	Set("UVEditor.BeginTransformTool",          new IMAGE_BRUSH_SVG("TransformUVs", ToolbarIconSize));
	Set("UVEditor.BeginAlignTool",              new IMAGE_BRUSH_SVG("AlignLeft", ToolbarIconSize));
	Set("UVEditor.BeginDistributeTool",         new IMAGE_BRUSH_SVG("DistributeHorizontally", ToolbarIconSize));
	Set("UVEditor.BeginTexelDensityTool",       new IMAGE_BRUSH_SVG("TexelDensity", ToolbarIconSize));
	Set("UVEditor.BeginBrushSelectTool",        new IMAGE_BRUSH("MeshSelect_40x", ToolbarIconSize));
	Set("UVEditor.BeginBrushSelectTool.Small",  new IMAGE_BRUSH("MeshSelect_40x", ToolbarIconSize));
	Set("UVEditor.BeginUVSnapshotTool",		    new IMAGE_BRUSH_SVG("UVSnapshot", ToolbarIconSize));
	
	// Select tool actions
	Set("UVEditor.SewAction", new IMAGE_BRUSH_SVG("UVSew", ToolbarIconSize));
	Set("UVEditor.SplitAction", new IMAGE_BRUSH_SVG("UVCut", ToolbarIconSize));
	Set("UVEditor.MakeIslandAction", new IMAGE_BRUSH_SVG("SelectionIslands", ToolbarIconSize));
	Set("UVEditor.IslandConformalUnwrapAction", new IMAGE_BRUSH_SVG("UVUnwrap", ToolbarIconSize));

	// Top toolbar icons
	Set("UVEditor.ApplyChanges", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Apply", ToolbarIconSize));
	Set("UVEditor.ChannelSettings", new CORE_IMAGE_BRUSH_SVG("Starship/Common/SetDrawUVs", ToolbarIconSize));
	Set("UVEditor.BackgroundSettings", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Sprite", ToolbarIconSize));

	// Viewport icons
	Set("UVEditor.OrbitCamera", new CORE_IMAGE_BRUSH_SVG("Starship/EditorViewport/rotate", ToolbarIconSize));
	Set("UVEditor.FlyCamera", new CORE_IMAGE_BRUSH_SVG("Starship/EditorViewport/camera", ToolbarIconSize));
	Set("UVEditor.FocusCamera", new CORE_IMAGE_BRUSH_SVG("Starship/Actors/snap-view-to-object", ToolbarIconSize));

	Set("UVEditor.ElementSelection", new IMAGE_BRUSH_SVG("MeshSelect_16", ToolbarIconSize));
	Set("UVEditor.VertexSelection", new IMAGE_BRUSH_SVG("SelectionVertices", ToolbarIconSize));
	Set("UVEditor.EdgeSelection", new IMAGE_BRUSH_SVG("SelectionLine", ToolbarIconSize));
	Set("UVEditor.TriangleSelection", new IMAGE_BRUSH_SVG("SelectionTriangle", ToolbarIconSize));
	Set("UVEditor.IslandSelection", new IMAGE_BRUSH_SVG("SelectionIslands", ToolbarIconSize));
	Set("UVEditor.FullMeshSelection", new IMAGE_BRUSH_SVG("SelectionMulti", ToolbarIconSize));

	// Distribute
	Set("UVEditor.DistributeSpaceVertically", new IMAGE_BRUSH_SVG("DistributeVertical", ToolbarIconSize));
	Set("UVEditor.DistributeSpaceHorizontally", new IMAGE_BRUSH_SVG("DistributeHorizontal", ToolbarIconSize));
	Set("UVEditor.DistributeTopEdges", new IMAGE_BRUSH_SVG("DistributeEdgesTop", ToolbarIconSize));
	Set("UVEditor.DistributeBottomEdges", new IMAGE_BRUSH_SVG("DistributeEdgesBottom", ToolbarIconSize));
	Set("UVEditor.DistributeLeftEdges", new IMAGE_BRUSH_SVG("DistributeEdgesLeft", ToolbarIconSize));
	Set("UVEditor.DistributeRightEdges", new IMAGE_BRUSH_SVG("DistributeEdgesRight", ToolbarIconSize));
	Set("UVEditor.DistributeCentersHorizontally", new IMAGE_BRUSH_SVG("DistributeCentersHorizontal", ToolbarIconSize));
	Set("UVEditor.DistributeCentersVertically", new IMAGE_BRUSH_SVG("DistributeCentersVertical", ToolbarIconSize));
	Set("UVEditor.DistributeRemoveOverlap", new IMAGE_BRUSH_SVG("RemoveOverlap", ToolbarIconSize));

	// Align
	Set("UVEditor.AlignDirectionBottomEdges", new IMAGE_BRUSH_SVG("AlignEdgesBottom", ToolbarIconSize));
	Set("UVEditor.AlignDirectionTopEdges", new IMAGE_BRUSH_SVG("AlignEdgesTop", ToolbarIconSize));
	Set("UVEditor.AlignDirectionLeftEdges", new IMAGE_BRUSH_SVG("AlignEdgesLeft", ToolbarIconSize));
	Set("UVEditor.AlignDirectionRightEdges", new IMAGE_BRUSH_SVG("AlignEdgesRight", ToolbarIconSize));
	Set("UVEditor.AlignDirectionCentersHorizontally", new IMAGE_BRUSH_SVG("AlignCentersHorizontal", ToolbarIconSize));
	Set("UVEditor.AlignDirectionCentersVertically", new IMAGE_BRUSH_SVG("AlignCentersVertical", ToolbarIconSize));

	// Style for the toolbar in the SeamTool customization
	{
		// For the selection button toolbar, we want to use something similar to the toolbar we use in the viewport
		Set("SeamTool.ModeToolbar", FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar"));

		// However, increase the size of the buttons a bit
		FCheckBoxStyle ToggleButtonStart = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Start");
		ToggleButtonStart.SetPadding(FMargin(9, 7, 6, 7));
		Set("SeamTool.ModeToolbar.ToggleButton.Start", ToggleButtonStart);

		FCheckBoxStyle ToggleButtonMiddle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Middle");
		ToggleButtonMiddle.SetPadding(FMargin(9, 7, 6, 7));
		Set("SeamTool.ModeToolbar.ToggleButton.Middle", ToggleButtonMiddle);

		FCheckBoxStyle ToggleButtonEnd = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.End");
		ToggleButtonEnd.SetPadding(FMargin(7, 7, 8, 7));
		Set("SeamTool.ModeToolbar.ToggleButton.End", ToggleButtonEnd);
	}

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FUVEditorStyle::~FUVEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FUVEditorStyle& FUVEditorStyle::Get()
{
	static FUVEditorStyle Inst;
	return Inst;
}


