// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorTexelDensityTool.h"

#include "ContextObjectStore.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "ParameterizationOps/TexelDensityOp.h"
#include "InputBehaviorSet.h"
#include "InputRouter.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ToolSceneQueriesUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "ToolSetupUtil.h"
#include "UVEditorUXSettings.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "Parameterization/UVMetrics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorTexelDensityTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UUVEditorTexelDensityTool"

namespace UVEditorTexelDensityToolLocals
{
	const FString& HoverTriangleSetID(TEXT("HoverTriangleSet"));
	const FString& HoverLineSetID(TEXT("HoverLineSet"));
}

bool UUVEditorTexelDensityToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return Targets && Targets->Num() > 0;
}

UInteractiveTool* UUVEditorTexelDensityToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UUVEditorTexelDensityTool* NewTool = NewObject<UUVEditorTexelDensityTool>(SceneState.ToolManager);
	NewTool->SetTargets(*Targets);

	return NewTool;
}

void UUVEditorTexelDensityActionSettings::PostAction(ETexelDensityToolAction Action)
{
	ParentTool->RequestAction(Action);
}


void UUVEditorTexelDensityActionSettings::SampleTexelDensity()
{
	PostAction(ETexelDensityToolAction::BeginSamping);
}


bool UUVEditorTexelDensityToolSettings::InSamplingMode() const
{
	if (ParentTool.IsValid())
	{
		return ParentTool->ActiveAction() == ETexelDensityToolAction::Sampling || ParentTool->ActiveAction() == ETexelDensityToolAction::BeginSamping;
	}
	return true;
}

void UUVEditorTexelDensityTool::Setup()
{
	check(Targets.Num() > 0);

	ToolStartTimeAnalytics = FDateTime::UtcNow();

	UInteractiveTool::Setup();
	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();

	Settings = NewObject<UUVEditorTexelDensityToolSettings>(this);
	ActionSettings = NewObject<UUVEditorTexelDensityActionSettings>(this);
	ActionSettings->Initialize(this);
	Cast<UUVEditorTexelDensityToolSettings>(Settings)->Initialize(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);
	AddToolPropertySource(ActionSettings);

	UVToolSelectionAPI = ContextStore->FindContext<UUVToolSelectionAPI>();
	UVTool2DViewportAPI = ContextStore->FindContext<UUVTool2DViewportAPI>();
	LivePreviewAPI = ContextStore->FindContext<UUVToolLivePreviewAPI>();

	UUVToolSelectionAPI::FHighlightOptions HighlightOptions;
	HighlightOptions.bBaseHighlightOnPreviews = true;
	HighlightOptions.bAutoUpdateUnwrap = true;
	UVToolSelectionAPI->SetHighlightOptions(HighlightOptions);
	UVToolSelectionAPI->SetHighlightVisible(true, false, true);

	auto SetupOpFactory = [this](UUVEditorToolMeshInput& Target, const FUVToolSelection* Selection)
	{
		int32 TargetIndex = Targets.Find(&Target);

		TObjectPtr<UUVTexelDensityOperatorFactory> Factory = NewObject<UUVTexelDensityOperatorFactory>();
		Factory->TargetTransform = Target.AppliedPreview->PreviewMesh->GetTransform();
		Factory->Settings = Settings;
		Factory->OriginalMesh = Target.AppliedCanonical;
		Factory->GetSelectedUVChannel = [&Target]() { return Target.UVLayerIndex; };
		if (Selection)
		{
			Factory->Selection.Emplace(Selection->GetConvertedSelection(*Target.UnwrapCanonical, FUVToolSelection::EType::Triangle).SelectedIDs);
		}

		if (UVTool2DViewportAPI)
		{
			TMap<int32, int32> TextureResolutionPerUDIM;
			for (const FUDIMBlock& UDIM : UVTool2DViewportAPI->GetUDIMBlocks())
			{
				TextureResolutionPerUDIM.Add(UDIM.UDIM, UDIM.TextureResolution);
			}
			Factory->TextureResolutionPerUDIM = TextureResolutionPerUDIM;
		}

		Target.AppliedPreview->ChangeOpFactory(Factory);
		Target.AppliedPreview->OnMeshUpdated.AddWeakLambda(this, [this, &Target](UMeshOpPreviewWithBackgroundCompute* Preview) {
			Target.UpdateUnwrapPreviewFromAppliedPreview();

			this->UVToolSelectionAPI->RebuildUnwrapHighlight(Target.UnwrapPreview->PreviewMesh->GetTransform());
			});

		Target.AppliedPreview->OnOpCompleted.AddWeakLambda(this,
			[this, TargetIndex](const FDynamicMeshOperator* Op)
			{
				RemainingTargetsToProcess--;
			}
		);

		Target.AppliedPreview->InvalidateResult();
		return Factory;
	};

	RemainingTargetsToProcess = 0;
	if (UVToolSelectionAPI->HaveSelections())
	{
		Factories.Reserve(UVToolSelectionAPI->GetSelections().Num());
		for (FUVToolSelection Selection : UVToolSelectionAPI->GetSelections())
		{
			RemainingTargetsToProcess++;
			Factories.Add(SetupOpFactory(*Selection.Target, &Selection));
		}
	}
	else
	{
		Factories.Reserve(Targets.Num());
		for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
		{
			RemainingTargetsToProcess++;
			Factories.Add(SetupOpFactory(*Targets[TargetIndex], nullptr));
		}
	}

	// Set things up for being able to add behaviors to live preview.
	LivePreviewBehaviorSet = NewObject<UInputBehaviorSet>();
	LivePreviewBehaviorSource = NewObject<ULocalInputBehaviorSource>();
	LivePreviewBehaviorSource->GetInputBehaviorsFunc = [this]() { return LivePreviewBehaviorSet; };

	// Set up click and hover behaviors both for the 2d (unwrap) viewport and the 3d 
	// (applied/live preview) viewport
	ULocalSingleClickInputBehavior* UnwrapClickBehavior = NewObject<ULocalSingleClickInputBehavior>();
	UnwrapClickBehavior->Initialize();
	UnwrapClickBehavior->IsHitByClickFunc = [this](const FInputDeviceRay& InputRay) {
		FInputRayHit HitResult;
		HitResult.bHit = Get2DHitTriangle(InputRay.WorldRay) != IndexConstants::InvalidID;
		// We don't bother with the depth since there aren't competing click behaviors
		return HitResult;
	};
	UnwrapClickBehavior->OnClickedFunc = [this](const FInputDeviceRay& ClickPos) {
		int32 IndexOfMesh = IndexConstants::InvalidID;
		int32 Tid = Get2DHitTriangle(ClickPos.WorldRay, &IndexOfMesh);
		if (Tid != IndexConstants::InvalidID)
		{
			OnMeshTriangleClicked(Tid, IndexOfMesh, true);
		}
	};
	AddInputBehavior(UnwrapClickBehavior);

	ULocalMouseHoverBehavior* UnwrapHoverBehavior = NewObject<ULocalMouseHoverBehavior>();
	UnwrapHoverBehavior->Initialize();
	UnwrapHoverBehavior->BeginHitTestFunc = [this](const FInputDeviceRay& PressPos) {
		FInputRayHit HitResult;
		HitResult.bHit = Get2DHitTriangle(PressPos.WorldRay) != IndexConstants::InvalidID;
		// We don't bother with the depth since there aren't competing hover behaviors
		return HitResult;
	};
	UnwrapHoverBehavior->OnUpdateHoverFunc = [this](const FInputDeviceRay& DevicePos)
	{
		int32 IndexOfMesh = IndexConstants::InvalidID;
		int32 Tid = Get2DHitTriangle(DevicePos.WorldRay, &IndexOfMesh);
		if (Tid != IndexConstants::InvalidID)
		{
			OnMeshTriangleHovered(Tid, IndexOfMesh, true);
			return true;
		}
		return false;
	};
	UnwrapHoverBehavior->OnEndHoverFunc = [this]()
	{
		ClearHover();
	};
	AddInputBehavior(UnwrapHoverBehavior);

	// Now the applied/live preview viewport...
	ULocalSingleClickInputBehavior* LivePreviewClickBehavior = NewObject<ULocalSingleClickInputBehavior>();
	LivePreviewClickBehavior->Initialize();
	LivePreviewClickBehavior->IsHitByClickFunc = [this](const FInputDeviceRay& InputRay) {
		FInputRayHit HitResult;
		HitResult.bHit = Get3DHitTriangle(InputRay.WorldRay) != IndexConstants::InvalidID;
		// We don't bother with the depth since there aren't competing click behaviors
		return HitResult;
	};
	LivePreviewClickBehavior->OnClickedFunc = [this](const FInputDeviceRay& ClickPos) {
		int32 IndexOfMesh = IndexConstants::InvalidID;
		int32 Tid = Get3DHitTriangle(ClickPos.WorldRay, &IndexOfMesh);
		if (Tid != IndexConstants::InvalidID)
		{
			OnMeshTriangleClicked(Tid, IndexOfMesh, false);
		}
	};

	LivePreviewBehaviorSet->Add(LivePreviewClickBehavior, this);

	ULocalMouseHoverBehavior* LivePreviewHoverBehavior = NewObject<ULocalMouseHoverBehavior>();
	LivePreviewHoverBehavior->Initialize();
	LivePreviewHoverBehavior->BeginHitTestFunc = [this](const FInputDeviceRay& PressPos) {
		FInputRayHit HitResult;
		HitResult.bHit = Get3DHitTriangle(PressPos.WorldRay) != IndexConstants::InvalidID;
		// We don't bother with the depth since there aren't competing hover behaviors
		return HitResult;
	};
	LivePreviewHoverBehavior->OnUpdateHoverFunc = [this](const FInputDeviceRay& DevicePos)
	{
		int32 IndexOfMesh = IndexConstants::InvalidID;
		int32 Tid = Get3DHitTriangle(DevicePos.WorldRay, &IndexOfMesh);
		if (Tid != IndexConstants::InvalidID)
		{
			OnMeshTriangleHovered(Tid, IndexOfMesh, false);
			return true;
		}
		return false;
	};
	LivePreviewHoverBehavior->OnEndHoverFunc = [this]()
	{
		ClearHover();
	};
	LivePreviewBehaviorSet->Add(LivePreviewHoverBehavior, this);

	// Give the live preview behaviors to the live preview input router
	if (LivePreviewAPI)
	{
		LivePreviewInputRouter = LivePreviewAPI->GetLivePreviewInputRouter();
		LivePreviewInputRouter->RegisterSource(LivePreviewBehaviorSource);
	}

	// Set up all the components we need to visualize things.
	UnwrapGeometry = NewObject<UPreviewGeometry>();
	UnwrapGeometry->CreateInWorld(Targets[0]->UnwrapPreview->GetWorld(), FTransform::Identity);

	LivePreviewGeometry = NewObject<UPreviewGeometry>();
	LivePreviewGeometry->CreateInWorld(Targets[0]->AppliedPreview->GetWorld(), FTransform::Identity);

	// These visualize the currently hovered (preview) portion of the seam
	UnwrapGeometry->AddTriangleSet(UVEditorTexelDensityToolLocals::HoverTriangleSetID);
	UnwrapGeometry->AddLineSet(UVEditorTexelDensityToolLocals::HoverLineSetID);
	LivePreviewGeometry->AddTriangleSet(UVEditorTexelDensityToolLocals::HoverTriangleSetID);
	LivePreviewGeometry->AddLineSet(UVEditorTexelDensityToolLocals::HoverLineSetID);

	TriangleSetMaterial = ToolSetupUtil::GetCustomTwoSidedDepthOffsetMaterial(this->GetToolManager(),
		FUVEditorUXSettings::SelectionHoverTriangleFillColor,
		FUVEditorUXSettings::SelectionHoverTriangleDepthBias,
		FUVEditorUXSettings::SelectionHoverTriangleOpacity);

	UUVToolAABBTreeStorage* TreeStore = ContextStore->FindContext<UUVToolAABBTreeStorage>();
	if (!TreeStore)
	{
		TreeStore = NewObject<UUVToolAABBTreeStorage>();
		ContextStore->AddContextObject(TreeStore);
	}

	// Initialize the AABB trees from cached values, or make new ones.
	for (int32 i = 0; i < Targets.Num(); ++i)
	{
		TSharedPtr<FDynamicMeshAABBTree3> Tree2d = TreeStore->Get(Targets[i]->UnwrapCanonical.Get());
		if (!Tree2d)
		{
			Tree2d = MakeShared<FDynamicMeshAABBTree3>();
			Tree2d->SetMesh(Targets[i]->UnwrapCanonical.Get());
			TreeStore->Set(Targets[i]->UnwrapCanonical.Get(), Tree2d, Targets[i]);
		}
		Spatials2D.Add(Tree2d);

		TSharedPtr<FDynamicMeshAABBTree3> Tree3d = TreeStore->Get(Targets[i]->AppliedCanonical.Get());
		if (!Tree3d)
		{
			Tree3d = MakeShared<FDynamicMeshAABBTree3>();
			Tree3d->SetMesh(Targets[i]->AppliedCanonical.Get());
			TreeStore->Set(Targets[i]->AppliedCanonical.Get(), Tree3d, Targets[i]);
		}
		Spatials3D.Add(Tree3d);

	}


	SetToolDisplayName(LOCTEXT("ToolNameLocal", "Texel Density"));
	UpdateToolMessage();

	// Analytics
	InputTargetAnalytics = UVEditorAnalytics::CollectTargetAnalytics(Targets);
}

int32 UUVEditorTexelDensityTool::Get2DHitTriangle(const FRay& WorldRayIn, int32* IndexOf2DSpatialOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorTexelDensityTool_Get2DHitTriangle);

	if (!ensure(WorldRayIn.Direction.Z != 0))
	{
		return IndexConstants::InvalidID;
	}

	// Find plane hit location
	double PlaneHitT = WorldRayIn.Origin.Z / -WorldRayIn.Direction.Z;
	FVector3d PlanePoint = WorldRayIn.PointAt(PlaneHitT);

	int32 Result = IndexConstants::InvalidID;
	// For tolerance, we'd like to divide the screen into 90 sections to be similar to the 90 degree
	// FOV angle tolerance we use in 3D. Ray Z should be our distance from ground plane, so width
	// would be 2Z for 90 degree FOV.
	double Tolerance = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD() * WorldRayIn.Origin.Z / 45;

	double MinDistSquared = TNumericLimits<double>::Max();
	for (int32 i = 0; i < Spatials2D.Num(); ++i)
	{
		double DistSquared = TNumericLimits<double>::Max();
		int32 Tid = Spatials2D[i]->FindNearestTriangle(PlanePoint, DistSquared);
		if (DistSquared >= 0 && DistSquared < MinDistSquared)
		{
			MinDistSquared = DistSquared;
			Result = Tid;
			if (IndexOf2DSpatialOut)
			{
				*IndexOf2DSpatialOut = i;
			}
		}
	}

	return Result;
}

int32 UUVEditorTexelDensityTool::Get3DHitTriangle(const FRay& WorldRayIn, int32* IndexOf3DSpatialOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorTexelDensityTool_Get3DHitTriangle);
	// We'll hit test all the preview meshes
	double BestRayT = TNumericLimits<double>::Max();
	int32 BestTid = IndexConstants::InvalidID;
	for (int32 i = 0; i < Spatials3D.Num(); ++i)
	{
		FTransform Transform = Targets[i]->AppliedPreview->PreviewMesh->GetTransform();
		FRay3d LocalRay(
			(FVector3d)Transform.InverseTransformPosition(WorldRayIn.Origin),
			(FVector3d)Transform.InverseTransformVector(WorldRayIn.Direction));

		int32 Tid = IndexConstants::InvalidID;
		double RayT = TNumericLimits<double>::Max();
		if (Spatials3D[i]->FindNearestHitTriangle(LocalRay, RayT, Tid))
		{
			if (RayT >= 0 && RayT < BestRayT)
			{
				BestRayT = RayT;
				BestTid = Tid;
				if (IndexOf3DSpatialOut)
				{
					*IndexOf3DSpatialOut = i;
				}
			}
		}
	}

	if (BestTid == IndexConstants::InvalidID)
	{
		// Didn't hit anything
		return IndexConstants::InvalidID;
	}

	return BestTid;
}

void UUVEditorTexelDensityTool::OnMeshTriangleClicked(int32 Tid, int32 IndexOfMesh, bool bTidIsFromUnwrap)
{
	using namespace UVEditorTexelDensityToolLocals;

	if (ClickedTid != IndexConstants::InvalidID)
	{
		// Haven't yet finished processing the last click
		return;
	}

	// Save the click information. The click will be applied on tick.
	ClickedTid = Tid;
	ClickedMeshIndex = IndexOfMesh;
	bClickWasInUnwrap = bTidIsFromUnwrap;
}

void UUVEditorTexelDensityTool::OnMeshTriangleHovered(int32 Tid, int32 IndexOfMesh, bool bTidIsFromUnwrap)
{
	using namespace UVEditorTexelDensityToolLocals;

	// Save the hover information. Hover will be updated on tick
	HoverTid = Tid;
	HoverMeshIndex = IndexOfMesh;
	bHoverTidIsFromUnwrap = bTidIsFromUnwrap;
}

void UUVEditorTexelDensityTool::UpdateHover()
{
	using namespace UVEditorTexelDensityToolLocals;

	if (bLastHoverTidWasFromUnwrap == bHoverTidIsFromUnwrap
		&& LastHoverMeshIndex == HoverMeshIndex
		&& LastHoverTid == HoverTid)
	{
		return;
	}

	bLastHoverTidWasFromUnwrap = bHoverTidIsFromUnwrap;
	LastHoverMeshIndex = HoverMeshIndex;
	LastHoverTid = HoverTid;

	ClearHover(false);

	if (HoverTid == IndexConstants::InvalidID)
	{
		return;
	}

	UTriangleSetComponent* UnwrapTriangleSet = UnwrapGeometry->FindTriangleSet(HoverTriangleSetID);
	UTriangleSetComponent* AppliedTriangleSet = LivePreviewGeometry->FindTriangleSet(HoverTriangleSetID);

	FVector VertA, VertB, VertC;

	Targets[HoverMeshIndex]->UnwrapCanonical->GetTriVertices(HoverTid, VertA, VertB, VertC);
	UnwrapTriangleSet->AddTriangle(FRenderableTriangle(
		TriangleSetMaterial,
		{ VertA, FVector2D(), FVector(0,0,1), FColor() },
		{ VertB, FVector2D(), FVector(0,0,1), FColor() },
		{ VertC, FVector2D(), FVector(0,0,1), FColor() }
	));

	FTransform3d Transform = Targets[HoverMeshIndex]->AppliedPreview->PreviewMesh->GetTransform();

	//UE_LOGF(UVEditorLog, Log, "Mesh: %d; Transform: %f %f %f;", HoverMeshIndex, Transform.GetTranslation().X, Transform.GetTranslation().Y, Transform.GetTranslation().Z);

	Targets[HoverMeshIndex]->AppliedCanonical->GetTriVertices(HoverTid, VertA, VertB, VertC);

	//UE_LOGF(UVEditorLog, Log, "VECA: %f %f %f;", VertA.X, VertA.Y, VertA.Z);

	AppliedTriangleSet->AddTriangle(FRenderableTriangle(
		TriangleSetMaterial,
		{ Transform.TransformPosition(VertA), FVector2D(), FVector(0,0,1), FColor() },
		{ Transform.TransformPosition(VertB), FVector2D(), FVector(0,0,1), FColor() },
		{ Transform.TransformPosition(VertC), FVector2D(), FVector(0,0,1), FColor() }
	));

}

void UUVEditorTexelDensityTool::ClearHover(bool bClearHoverInfo)
{
	using namespace UVEditorTexelDensityToolLocals;

	UnwrapGeometry->FindTriangleSet(HoverTriangleSetID)->Clear();
	LivePreviewGeometry->FindTriangleSet(HoverTriangleSetID)->Clear();
	UnwrapGeometry->FindLineSet(HoverLineSetID)->Clear();
	LivePreviewGeometry->FindLineSet(HoverLineSetID)->Clear();

	if (bClearHoverInfo)
	{
		HoverTid = IndexConstants::InvalidID;
		HoverMeshIndex = IndexConstants::InvalidID;
		LastHoverTid = HoverTid;
		LastHoverMeshIndex = HoverMeshIndex;
	}
}

void UUVEditorTexelDensityTool::ApplyClick()
{
	const TObjectPtr<UUVEditorToolMeshInput>& Target = Targets[ClickedMeshIndex];

	bool bValidUVTid = Target->AppliedCanonical->Attributes()->GetUVLayer(Target->UVLayerIndex)->IsSetTriangle(ClickedTid);

	if (!bValidUVTid)
	{
		return;
	}

	double TexelDensity = FUVMetrics::TexelDensity(*Target->AppliedCanonical, Target->UVLayerIndex, ClickedTid, Settings->TextureResolution);

	Settings->TargetWorldUnits = Settings->TargetPixelCount / TexelDensity;
}


void UUVEditorTexelDensityTool::Shutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);

	if (LivePreviewInputRouter.IsValid())
	{
		// TODO: Arguably the live preview input router should do this for us before Shutdown(), but
		// we don't currently have support for that...
		LivePreviewInputRouter->DeregisterSource(LivePreviewBehaviorSource);
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		UUVToolEmitChangeAPI* ChangeAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolEmitChangeAPI>();

		const FText TransactionName(LOCTEXT("TexelDensityTransactionName", "Texel Density"));
		ChangeAPI->BeginUndoTransaction(TransactionName);
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Set things up for undo. 
			FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
			ChangeTracker.BeginChange();

			for (int32 Tid : Target->UnwrapCanonical->TriangleIndicesItr())
			{
				ChangeTracker.SaveTriangle(Tid, true);
			}

			Target->UpdateCanonicalFromPreviews();

			ChangeAPI->EmitToolIndependentUnwrapCanonicalChange(Target, ChangeTracker.EndChange(), LOCTEXT("ApplyTexelDensityTool", "Texel Density Tool"));
		}

		ChangeAPI->EndUndoTransaction();

		// Analytics
		RecordAnalytics();
	}
	else
	{
		// Reset the inputs
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			Target->UpdatePreviewsFromCanonical();
		}
	}

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->ClearOpFactory();
	}
	for (int32 FactoryIndex = 0; FactoryIndex < Factories.Num(); ++FactoryIndex)
	{
		Factories[FactoryIndex] = nullptr;
	}

	LivePreviewAPI = nullptr;

	LivePreviewBehaviorSet->RemoveAll();
	LivePreviewBehaviorSet = nullptr;
	LivePreviewBehaviorSource = nullptr;

	UnwrapGeometry->Disconnect();
	LivePreviewGeometry->Disconnect();
	UnwrapGeometry = nullptr;
	LivePreviewGeometry = nullptr;

	Spatials2D.Reset();
	Spatials3D.Reset();

	Settings = nullptr;
	Targets.Empty();
}

void UUVEditorTexelDensityTool::OnTick(float DeltaTime)
{
	UToolsContextCursorAPI* ToolsContextCursorAPI = GetToolManager()->GetContextObjectStore()->FindContext<UToolsContextCursorAPI>();

	if (PendingAction == ETexelDensityToolAction::BeginSamping)
	{
		if (ToolsContextCursorAPI)
		{
			ToolsContextCursorAPI->SetCursorOverride(EMouseCursor::EyeDropper);			
		}
		LivePreviewAPI->SetCursorOverride(EMouseCursor::EyeDropper);

		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			Target->UpdatePreviewsFromCanonical();
			this->UVToolSelectionAPI->RebuildUnwrapHighlight(Target->UnwrapPreview->PreviewMesh->GetTransform());
		}

		PendingAction = ETexelDensityToolAction::Sampling;
	}
	else if( PendingAction == ETexelDensityToolAction::Sampling)
	{
		UpdateHover();

		// Apply click if there was one (signaled by a non-InvalidId tid)
		if (ClickedTid != IndexConstants::InvalidID)
		{
			ApplyClick();
			ClickedTid = IndexConstants::InvalidID;

			if (ToolsContextCursorAPI)
			{
				ToolsContextCursorAPI->ClearCursorOverride();
			}
			LivePreviewAPI->ClearCursorOverride();

			PendingAction = ETexelDensityToolAction::NoAction;
			PerformBackgroundScalingTask();
		}
	}
	else
	{
		ClearHover();
	}

	if (PendingAction == ETexelDensityToolAction::Processing && RemainingTargetsToProcess == 0)
	{
		PendingAction = ETexelDensityToolAction::NoAction;
	}

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->Tick(DeltaTime);
	}

	UpdateToolMessage();
}

void UUVEditorTexelDensityTool::Render(IToolsContextRenderAPI* RenderAPI)
{

}

bool UUVEditorTexelDensityTool::CanAccept() const
{
	if (UVToolSelectionAPI->HaveSelections())
	{
		for (FUVToolSelection Selection : UVToolSelectionAPI->GetSelections())
		{
			if (!Selection.Target->AppliedPreview->HaveValidResult())
			{
				return false;
			}
		}
	}
	else
	{
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			if (!Target->AppliedPreview->HaveValidResult())
			{
				return false;
			}
		}
	}
	return true;
}

void UUVEditorTexelDensityTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PendingAction == ETexelDensityToolAction::Sampling)
	{
		PendingAction = ETexelDensityToolAction::NoAction;
		UpdateToolMessage();
	}

	if (PendingAction == ETexelDensityToolAction::NoAction || PendingAction == ETexelDensityToolAction::Processing)
	{
		if (PropertySet == Settings)
		{
			PerformBackgroundScalingTask();
		}
	}
}

void UUVEditorTexelDensityTool::RequestAction(ETexelDensityToolAction ActionType)
{
	if (PendingAction == ETexelDensityToolAction::NoAction)
	{
		PendingAction = ActionType;
	}

	UpdateToolMessage();
}

ETexelDensityToolAction UUVEditorTexelDensityTool::ActiveAction() const
{
	return PendingAction;
}

void UUVEditorTexelDensityTool::PerformBackgroundScalingTask()
{
	PendingAction = ETexelDensityToolAction::Processing;
	UpdateToolMessage();

	RemainingTargetsToProcess = 0;
	if (UVToolSelectionAPI->HaveSelections())
	{
		for (FUVToolSelection Selection : UVToolSelectionAPI->GetSelections())
		{
			RemainingTargetsToProcess++;
			Selection.Target->AppliedPreview->InvalidateResult();
		}
	}
	else
	{
		for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
		{
			RemainingTargetsToProcess++;
			Targets[TargetIndex]->AppliedPreview->InvalidateResult();
		}
	}
}

void UUVEditorTexelDensityTool::UpdateToolMessage()
{
	switch (PendingAction)
	{
	case ETexelDensityToolAction::NoAction:
		GetToolManager()->DisplayMessage(LOCTEXT("OnStartTool_TexelDensity", "Read and rescale UVs based on texel density values."),
			EToolMessageLevel::UserNotification);
		break;
	case ETexelDensityToolAction::Processing:
		GetToolManager()->DisplayMessage(LOCTEXT("OnProcessing_TexelDensity", "Applying texel density scaling. Please wait..."),
			EToolMessageLevel::UserNotification);
		break;
	case ETexelDensityToolAction::BeginSamping:
	case ETexelDensityToolAction::Sampling:
		GetToolManager()->DisplayMessage(LOCTEXT("OnSampling_TexelDensity", "Click on a triangle in the UV or 3D preview viewport to set the tool's current texel density ratio to be equal to that triangle."),
			EToolMessageLevel::UserNotification);
		break;
	default:
		ensure(false);
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserNotification);
		break;
	}
}

void UUVEditorTexelDensityTool::RecordAnalytics()
{
	using namespace UVEditorAnalytics;

	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));

	// Tool inputs
	InputTargetAnalytics.AppendToAttributes(Attributes, "Input");

	// Tool outputs	
	const FTargetAnalytics OutputTargetAnalytics = CollectTargetAnalytics(Targets);
	OutputTargetAnalytics.AppendToAttributes(Attributes, "Output");

	// Tool stats
	if (CanAccept())
	{
		TArray<double> PerAssetValidResultComputeTimes;
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Note: This would log -1 if the result was invalid, but checking CanAccept above ensures results are valid
			PerAssetValidResultComputeTimes.Add(Target->AppliedPreview->GetValidResultComputeTime());
		}
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.PerAsset.ComputeTimeSeconds"), PerAssetValidResultComputeTimes));
	}
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.ToolActiveDuration"), (FDateTime::UtcNow() - ToolStartTimeAnalytics).ToString()));

	FEngineAnalytics::GetProvider().RecordEvent(UVEditorAnalyticsEventName(TEXT("TexelDensityTool")), Attributes);
}

#undef LOCTEXT_NAMESPACE