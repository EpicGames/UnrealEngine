// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorToolAnalyticsUtils.h"

#include "UObject/ObjectPtr.h"


namespace UE
{
namespace Geometry
{
	
namespace UVEditorAnalytics
{

FString UVEditorAnalyticsEventName(const FString& ToolEventName)
{
	return FString::Printf(TEXT("Editor.Usage.UVEditor.%s"), *ToolEventName);
}

void FTargetAnalytics::AppendToAttributes(TArray<FAnalyticsEventAttribute>& Attributes, FString Prefix) const
{
	if (!Prefix.IsEmpty())
	{
		Prefix += ".";
	}
	
	Attributes.Add(FAnalyticsEventAttribute(Prefix + TEXT("AllAssets.NumVertices"), AllAssetsNumVertices));
	Attributes.Add(FAnalyticsEventAttribute(Prefix + TEXT("AllAssets.NumTriangles"), AllAssetsNumTriangles));
	Attributes.Add(FAnalyticsEventAttribute(Prefix + TEXT("AllAssets.NumUVLayers"), AllAssetsNumUVLayers));
	
	Attributes.Add(FAnalyticsEventAttribute(Prefix + TEXT("PerAsset.NumVertices"), PerAssetNumVertices));
	Attributes.Add(FAnalyticsEventAttribute(Prefix + TEXT("PerAsset.NumTriangles"), PerAssetNumTriangles));
	Attributes.Add(FAnalyticsEventAttribute(Prefix + TEXT("PerAsset.NumUVLayers"), PerAssetNumUVLayers));
	Attributes.Add(FAnalyticsEventAttribute(Prefix + TEXT("PerAsset.ActiveUVChannel.Index"), PerAssetActiveUVChannelIndex));
	Attributes.Add(FAnalyticsEventAttribute(Prefix + TEXT("PerAsset.ActiveUVChannel.NumVertices"), PerAssetActiveUVChannelNumVertices));
	Attributes.Add(FAnalyticsEventAttribute(Prefix + TEXT("PerAsset.ActiveUVChannel.NumTriangles"), PerAssetActiveUVChannelNumTriangles));
}

FTargetAnalytics CollectTargetAnalytics(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& Targets)
{
	FTargetAnalytics Result;
	
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		const int NumVertices = Target->AppliedCanonical->VertexCount();
		const int NumTriangles = Target->AppliedCanonical->TriangleCount();
		const int NumUVLayers = Target->AppliedCanonical->HasAttributes() ? Target->AppliedCanonical->Attributes()->NumUVLayers() : 0;
		
		Result.AllAssetsNumVertices += NumVertices;
		Result.AllAssetsNumTriangles += NumTriangles;
		Result.AllAssetsNumUVLayers += NumUVLayers;
		
		Result.PerAssetNumVertices.Add(NumVertices);
		Result.PerAssetNumTriangles.Add(NumTriangles);
		Result.PerAssetNumUVLayers.Add(NumUVLayers);

		Result.PerAssetActiveUVChannelIndex.Add(Target->UVLayerIndex);
		if (Target->UnwrapCanonical)
		{
			const int NumVerticesActiveUVChannel = Target->UnwrapCanonical->VertexCount();
			const int NumTrianglesActiveUVChannel = Target->UnwrapCanonical->TriangleCount();
			Result.PerAssetActiveUVChannelNumVertices.Add(NumVerticesActiveUVChannel);
			Result.PerAssetActiveUVChannelNumTriangles.Add(NumTrianglesActiveUVChannel);
		}
	}

	return Result;
}

} // end namespace UVEditorAnalytics
	
} // end namespace Geometry
} // end namespace UE
