// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorToolBase.h"

#include "InteractiveTool.h"
#include "InteractiveToolManager.h"
#include "ToolContextInterfaces.h" // FToolBuilderState

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorToolBase)

void UGenericUVEditorToolBuilder::Initialize(TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn, TSubclassOf<UInteractiveTool> ToolClassIn)
{
	Targets = &TargetsIn;
	if (ensure(ToolClassIn && ToolClassIn->ImplementsInterface(UUVEditorGenericBuildableTool::StaticClass())))
	{
		ToolClass = ToolClassIn;
	}
}

bool UGenericUVEditorToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return ToolClass && Targets && Targets->Num() > 0;
}

UInteractiveTool* UGenericUVEditorToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UInteractiveTool* NewTool = NewObject<UInteractiveTool>(SceneState.ToolManager, ToolClass.Get());
	IUVEditorGenericBuildableTool* CastTool = Cast<IUVEditorGenericBuildableTool>(NewTool);
	if (ensure(CastTool))
	{
		CastTool->SetTargets(*Targets);
	}
	return NewTool;
}
