// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorTransformTool.h"

#include "ContextObjectStore.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "Operators/UVEditorUVTransformOp.h"
#include "SceneView.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "EngineAnalytics.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "UVEditorUXSettings.h"
#include "CanvasTypes.h"
#include "Engine/Canvas.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorTransformTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UUVEditorTransformTool"

namespace TransformToolLocals
{
	FText ToolName(EUVEditorUVTransformType Mode)
	{
		switch (Mode)
		{
		case EUVEditorUVTransformType::Transform:
			return LOCTEXT("ToolNameTransform", "UV Transform");
		case EUVEditorUVTransformType::Align:
			return LOCTEXT("ToolNameAlign", "UV Align");
		case EUVEditorUVTransformType::Distribute:
			return LOCTEXT("ToolNameDistribute", "UV Distribute");
		default:
			ensure(false);
			return FText();
		}
	}

	FText ToolDescription(EUVEditorUVTransformType Mode)
	{
		switch (Mode)
		{
		case EUVEditorUVTransformType::Transform:
			return LOCTEXT("OnStartToolTransform", "Translate, rotate or scale existing UV Charts using various strategies");
		case EUVEditorUVTransformType::Align:
			return LOCTEXT("OnStartToolAlign", "Align UV elements relative to various positions and with various strategies");
		case EUVEditorUVTransformType::Distribute:
			return LOCTEXT("OnStartToolDistribute", "Distribute UV elements spatially with various strategies");
		default:
			ensure(false);
			return FText();
		}
	}

	FText ToolTransaction(EUVEditorUVTransformType Mode)
	{
		switch (Mode)
		{
		case EUVEditorUVTransformType::Transform:
			return LOCTEXT("TransactionNameTransform", "Transform Tool");
		case EUVEditorUVTransformType::Align:
			return LOCTEXT("TransactionNameAlign", "Align Tool");
		case EUVEditorUVTransformType::Distribute:
			return LOCTEXT("TransactionNameDistribute", "Distribute Tool");
		default:
			ensure(false);
			return FText();
		}
	}

	FText ToolConfirmation(EUVEditorUVTransformType Mode)
	{
		switch (Mode)
		{
		case EUVEditorUVTransformType::Transform:
			return LOCTEXT("ApplyToolTransform", "Transform Tool");
		case EUVEditorUVTransformType::Align:
			return LOCTEXT("ApplyToolAlign", "Align Tool");
		case EUVEditorUVTransformType::Distribute:
			return LOCTEXT("ApplyToolDistribute", "Distribute Tool");
		default:
			ensure(false);
			return FText();
		}
	}
}


// Tool builder
// TODO: Could consider sharing some of the tool builder boilerplate for UV editor tools in a common base class.

bool UUVEditorBaseTransformToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return Targets && Targets->Num() > 0;
}

UInteractiveTool* UUVEditorBaseTransformToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UUVEditorTransformTool* NewTool = NewObject<UUVEditorTransformTool>(SceneState.ToolManager);
	ConfigureTool(NewTool);	
	return NewTool;
}

void UUVEditorBaseTransformToolBuilder::ConfigureTool(UUVEditorTransformTool* NewTool) const
{
	NewTool->SetTargets(*Targets);
}

void UUVEditorTransformToolBuilder::ConfigureTool(UUVEditorTransformTool* NewTool) const
{
	UUVEditorBaseTransformToolBuilder::ConfigureTool(NewTool);
	NewTool->SetToolMode(EUVEditorUVTransformType::Transform);
}

void UUVEditorAlignToolBuilder::ConfigureTool(UUVEditorTransformTool* NewTool) const
{
	UUVEditorBaseTransformToolBuilder::ConfigureTool(NewTool);
	NewTool->SetToolMode(EUVEditorUVTransformType::Align);
}

void UUVEditorDistributeToolBuilder::ConfigureTool(UUVEditorTransformTool* NewTool) const
{
	UUVEditorBaseTransformToolBuilder::ConfigureTool(NewTool);
	NewTool->SetToolMode(EUVEditorUVTransformType::Distribute);
}

void UUVEditorTransformTool::SetToolMode(const EUVEditorUVTransformType& Mode)
{
	ToolMode = Mode;
}

void UUVEditorTransformTool::Setup()
{
	check(Targets.Num() > 0);

	ToolStartTimeAnalytics = FDateTime::UtcNow();

	UInteractiveTool::Setup();
	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();

	switch(ToolMode.Get(EUVEditorUVTransformType::Transform))
	{
		case EUVEditorUVTransformType::Transform:
			Settings = NewObject<UUVEditorUVTransformProperties>(this);
			break;
		case EUVEditorUVTransformType::Align:
			Settings = NewObject<UUVEditorUVAlignProperties>(this);
			break;
		case EUVEditorUVTransformType::Distribute:
			Settings = NewObject<UUVEditorUVDistributeProperties>(this);
			break;
		default:
			ensure(false);
	}
	Settings->RestoreProperties(this);
	//Settings->bUDIMCVAREnabled = (FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport.GetValueOnGameThread() > 0);
	AddToolPropertySource(Settings);

	DisplaySettings = NewObject<UUVEditorTransformToolDisplayProperties>(this);
	DisplaySettings->RestoreProperties(this);
	DisplaySettings->GetOnModified().AddLambda([this](UObject* PropertySetArg, FProperty* PropertyArg)
	{
		OnPropertyModified(PropertySetArg, PropertyArg);
	});
	UUVEditorToolPropertiesAPI* PropertiesAPI = ContextStore->FindContext<UUVEditorToolPropertiesAPI>();
	if (PropertiesAPI)
	{
		PropertiesAPI->SetToolDisplayProperties(DisplaySettings);
	}

	UVToolSelectionAPI = ContextStore->FindContext<UUVToolSelectionAPI>();

	UUVToolSelectionAPI::FHighlightOptions HighlightOptions;
	HighlightOptions.bBaseHighlightOnPreviews = true;
	HighlightOptions.bAutoUpdateUnwrap = true;
	UVToolSelectionAPI->SetHighlightOptions(HighlightOptions);
	UVToolSelectionAPI->SetHighlightVisible(true, false, true);

	PerTargetPivotLocations.SetNum(Targets.Num());

	auto SetupOpFactory = [this](UUVEditorToolMeshInput& Target, const FUVToolSelection* Selection)
	{
		int32 TargetIndex = Targets.Find(&Target);

		TObjectPtr<UUVEditorUVTransformOperatorFactory> Factory = NewObject<UUVEditorUVTransformOperatorFactory>();
		Factory->TargetTransform = Target.UnwrapPreview->PreviewMesh->GetTransform();
		Factory->Settings = Settings;
		Factory->TransformType = ToolMode.Get(EUVEditorUVTransformType::Transform);
		Factory->OriginalMesh = Target.UnwrapCanonical;
		Factory->GetSelectedUVChannel = [&Target]() { return 0; /*Since we're passing in the unwrap mesh, the UV index is always zero.*/ };
		if (Selection)
		{
			// Generate vertex and edge selection sets for the operation. These are needed to differentiate
			// between islands, whether they are composed of triangles, edges or isolated points.
			FUVToolSelection UnwrapVertexSelection;
			FUVToolSelection UnwrapEdgeSelection;
			switch (Selection->Type)
			{
			case FUVToolSelection::EType::Vertex:
				Factory->VertexSelection.Emplace(Selection->SelectedIDs);
				Factory->EdgeSelection.Emplace(Selection->GetConvertedSelection(*Target.UnwrapCanonical, FUVToolSelection::EType::Edge).SelectedIDs);
				break;
			case FUVToolSelection::EType::Edge:
				Factory->VertexSelection.Emplace(Selection->GetConvertedSelection(*Target.UnwrapCanonical, FUVToolSelection::EType::Vertex).SelectedIDs);
				Factory->EdgeSelection.Emplace(Selection->SelectedIDs);
				break;
			case FUVToolSelection::EType::Triangle:
				Factory->VertexSelection.Emplace(Selection->GetConvertedSelection(*Target.UnwrapCanonical, FUVToolSelection::EType::Vertex).SelectedIDs);
				Factory->EdgeSelection.Emplace(Selection->GetConvertedSelection(*Target.UnwrapCanonical, FUVToolSelection::EType::Edge).SelectedIDs);
				break;
			}
		}

		Target.UnwrapPreview->ChangeOpFactory(Factory);
		Target.UnwrapPreview->OnMeshUpdated.AddWeakLambda(this, [this, &Target](UMeshOpPreviewWithBackgroundCompute* Preview) {
			Target.UpdateUnwrapPreviewOverlayFromPositions();
			Target.UpdateAppliedPreviewFromUnwrapPreview();
			

			this->UVToolSelectionAPI->RebuildUnwrapHighlight(Target.UnwrapPreview->PreviewMesh->GetTransform());
			});

		Target.UnwrapPreview->OnOpCompleted.AddWeakLambda(this,
			[this, TargetIndex](const FDynamicMeshOperator* Op)
			{
				const FUVEditorUVTransformBaseOp* TransformBaseOp = (const FUVEditorUVTransformBaseOp*)(Op);
				PerTargetPivotLocations[TargetIndex] = TransformBaseOp->GetPivotLocations();
			}
		);

		Target.UnwrapPreview->InvalidateResult();
		return Factory;
	};

	if (UVToolSelectionAPI->HaveSelections())
	{
		Factories.Reserve(UVToolSelectionAPI->GetSelections().Num());
		for (FUVToolSelection Selection : UVToolSelectionAPI->GetSelections())
		{
			Factories.Add(SetupOpFactory(*Selection.Target, &Selection));
		}
	}
	else
	{
		Factories.Reserve(Targets.Num());
		for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
		{
			Factories.Add(SetupOpFactory(*Targets[TargetIndex], nullptr));
		}
	}

	SetToolDisplayName(TransformToolLocals::ToolName(ToolMode.Get(EUVEditorUVTransformType::Transform)));
	GetToolManager()->DisplayMessage(TransformToolLocals::ToolDescription(ToolMode.Get(EUVEditorUVTransformType::Transform)),
		EToolMessageLevel::UserNotification);

	// Analytics
	InputTargetAnalytics = UVEditorAnalytics::CollectTargetAnalytics(Targets);
}

void UUVEditorTransformTool::Shutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->UnwrapPreview->OnMeshUpdated.RemoveAll(this);
		Target->UnwrapPreview->OnOpCompleted.RemoveAll(this);
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		UUVToolEmitChangeAPI* ChangeAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolEmitChangeAPI>();
		const FText TransactionName(TransformToolLocals::ToolTransaction(ToolMode.Get(EUVEditorUVTransformType::Transform)));
		ChangeAPI->BeginUndoTransaction(TransactionName);

		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Set things up for undo. 
			// TODO: It's not entirely clear whether it would be safe to use a FMeshVertexChange instead... It seems like
			// when bAllowFlips is true, we would end up with changes to the tris of the unwrap. Also, if we stick to saving
			// all the tris and verts, should we consider using the new dynamic mesh serialization?
			FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
			ChangeTracker.BeginChange();

			for (int32 Tid : Target->UnwrapCanonical->TriangleIndicesItr())
			{
				ChangeTracker.SaveTriangle(Tid, true);
			}

			// TODO: Again, it's not clear whether we need to update the entire triangle topology...
			Target->UpdateCanonicalFromPreviews();

			ChangeAPI->EmitToolIndependentUnwrapCanonicalChange(Target, ChangeTracker.EndChange(),
				       TransformToolLocals::ToolConfirmation(ToolMode.Get(EUVEditorUVTransformType::Transform)));
		}

		ChangeAPI->EndUndoTransaction();

		// Analytics
		RecordAnalytics();
	}
	else
	{
		// Reset the inputs
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			Target->UpdatePreviewsFromCanonical();
		}
	}


	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->UnwrapPreview->ClearOpFactory();
	}

	for (int32 FactoryIndex = 0; FactoryIndex < Factories.Num(); ++FactoryIndex)
	{
		Factories[FactoryIndex] = nullptr;
	}

	Settings = nullptr;
	DisplaySettings = nullptr;
	Targets.Empty();
}

void UUVEditorTransformTool::OnTick(float DeltaTime)
{
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->UnwrapPreview->Tick(DeltaTime);
	}
}

void UUVEditorTransformTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PropertySet == DisplaySettings)
	{
		return;
	}

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->UnwrapPreview->InvalidateResult();
	}
}

bool UUVEditorTransformTool::CanAccept() const
{

	if (UVToolSelectionAPI->HaveSelections())
	{
		for (FUVToolSelection Selection : UVToolSelectionAPI->GetSelections())
		{
			if (!Selection.Target->UnwrapPreview->HaveValidResult())
			{
				return false;
			}
		}
	}
	else
	{
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			if (!Target->UnwrapPreview->HaveValidResult())
			{
				return false;
			}
		}
	}
	return true;
}

void UUVEditorTransformTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	// TODO: Add support here for highlighting first selected item for alignment visualization	

	auto ConvertUVToPixel = [RenderAPI](const FVector2D& UVIn, FVector2D& PixelOut)
	{
		FVector WorldPoint = FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition((FVector2f)UVIn);
		FVector4 TestProjectedHomogenous = RenderAPI->GetSceneView()->WorldToScreen(WorldPoint);
		bool bValid = RenderAPI->GetSceneView()->ScreenToPixel(TestProjectedHomogenous, PixelOut);
		return bValid;
	};

	if (DisplaySettings->bDrawPivots)
	{
		for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
		{
			for (const FVector2D& PivotLocation : PerTargetPivotLocations[TargetIndex])
			{
				FVector2D PivotLocationPixel;
				ConvertUVToPixel(PivotLocation, PivotLocationPixel);

				const int32 NumSides = FUVEditorUXSettings::PivotCircleNumSides;
				const float Radius = FUVEditorUXSettings::PivotCircleRadius;
				const float LineThickness = FUVEditorUXSettings::PivotLineThickness;
				const FColor LineColor = FUVEditorUXSettings::PivotLineColor;

				const float	AngleDelta = 2.0f * UE_PI / NumSides;
				FVector2D AxisX(1.f, 0.f);
				FVector2D AxisY(0.f, -1.f);
				FVector2D LastVertex = PivotLocationPixel + AxisX * Radius;

				for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
				{
					const FVector2D Vertex = PivotLocationPixel + (AxisX * FMath::Cos(AngleDelta * (SideIndex + 1)) + AxisY * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
					FCanvasLineItem LineItem(LastVertex, Vertex);
					LineItem.LineThickness = LineThickness;
					LineItem.SetColor(LineColor);
					Canvas->DrawItem(LineItem);
					LastVertex = Vertex;
				}
			}
		}
	}
}


void UUVEditorTransformTool::RecordAnalytics()
{
	using namespace UVEditorAnalytics;

	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));

	// Tool inputs
	InputTargetAnalytics.AppendToAttributes(Attributes, "Input");

	// Tool stats
	if (CanAccept())
	{
		TArray<double> PerAssetValidResultComputeTimes;
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Note: This would log -1 if the result was invalid, but checking CanAccept above ensures results are valid
			PerAssetValidResultComputeTimes.Add(Target->UnwrapPreview->GetValidResultComputeTime());
		}
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.PerAsset.ComputeTimeSeconds"), PerAssetValidResultComputeTimes));
	}
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.ToolActiveDuration"), (FDateTime::UtcNow() - ToolStartTimeAnalytics).ToString()));

	// Tool settings chosen by the user
	//Attributes.Add(AnalyticsEventAttributeEnum(TEXT("Settings.LayoutType"), Settings->LayoutType));
	//Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.TextureResolution"), Settings->TextureResolution));
	//Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Scale"), Settings->Scale));
	//const TArray<FVector2D::FReal> TranslationArray({ Settings->Translation.X, Settings->Translation.Y });
	//Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Translation"), TranslationArray));
	//Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.AllowFlips"), Settings->bAllowFlips));

	FEngineAnalytics::GetProvider().RecordEvent(UVEditorAnalyticsEventName(TEXT("TransformTool")), Attributes);

	if constexpr (false)
	{
		for (const FAnalyticsEventAttribute& Attr : Attributes)
		{
			UE_LOGF(LogGeometry, Log, "Debug %ls.%ls = %ls", *UVEditorAnalyticsEventName(TEXT("TransformTool")), *Attr.GetName(), *Attr.GetValue());
		}
	}
}

#undef LOCTEXT_NAMESPACE

