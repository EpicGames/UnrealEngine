// Copyright Epic Games, Inc. All Rights Reserved.
#include "UVEditorUVSnapshotTool.h"

#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ModelingToolTargetUtil.h"
#include "AssetUtils/Texture2DBuilder.h"
#include "Sampling/MeshUVShellMapEvaluator.h"
#include "ModelingObjectsCreationAPI.h"
#include "ToolSetupUtil.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UVEditorUXSettings.h"
#include "Drawing/MeshElementsVisualizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorUVSnapshotTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UUVEditorUVSnapshotTool"

/*
 * ToolBuilder
 */
bool UUVEditorUVSnapshotToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// ensure only one target is handled and that source target is valid
	return Targets && Targets->Num() == 1;
}
UInteractiveTool* UUVEditorUVSnapshotToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UUVEditorUVSnapshotTool* NewTool = NewObject<UUVEditorUVSnapshotTool>(SceneState.ToolManager);
	NewTool->SetTarget((*Targets)[0]);
	return NewTool;
}

/*
 * Operator
 */
class FMeshUVMapBakerOp : public TGenericDataOperator<FMeshMapBaker>
{
public:
	using ImagePtr = TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>;
	// General bake settings
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> BaseMesh;
	TUniquePtr<UE::Geometry::FMeshMapBaker> Baker;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> DetailSpatial;

	TSharedPtr<FMeshUVShellMapEvaluator> UVShellEval = MakeShared<FMeshUVShellMapEvaluator>();

	UE::Geometry::FImageDimensions BakerDimensions;
	int BakerSamplesPerPixel = 4;

	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		Baker = MakeUnique<FMeshMapBaker>();
		Baker->CancelF = [Progress]() {
			return Progress && Progress->Cancelled();
		};
		Baker->SetTargetMesh(BaseMesh.Get());
		Baker->SetTargetMeshUVLayer(UVShellEval->UVLayer);
		Baker->SetDimensions(BakerDimensions);
		Baker->SetSamplesPerPixel(BakerSamplesPerPixel);
		
		FMeshBakerDynamicMeshSampler DetailSampler(BaseMesh.Get(), DetailSpatial.Get(), nullptr);
		Baker->SetDetailSampler(&DetailSampler);

		UVShellEval->TexelSize = BakerDimensions.GetTexelSize();

		Baker->AddEvaluator(UVShellEval);
		Baker->Bake();
		SetResult(MoveTemp(Baker));
	}
};

/*
 * Tool
 */
void UUVEditorUVSnapshotTool::Setup()
{
	UInteractiveTool::Setup();

	// initialize properties
	UVShellSettings = NewObject<UUVEditorBakeUVShellProperties>(this);
	UVShellSettings->RestoreProperties(this);
	AddToolPropertySource(UVShellSettings);

	const int16 NumUVLayers = Target->AppliedCanonical->Attributes()->NumUVLayers();
	InitializeUVLayerNames(UVShellSettings->TargetUVLayerNamesList, NumUVLayers);

	// retrieve whatever UV Layer is currently being displayed in UV Editor
	UVShellSettings->UVLayer = UVShellSettings->TargetUVLayerNamesList[Target->UVLayerIndex];
	UVShellSettings->WatchProperty(UVShellSettings->UVLayer, [this](FString) { Compute->InvalidateResult(); });

	UVShellSettings->WatchProperty(UVShellSettings->SamplesPerPixel, [this](EBakeTextureSamplesPerPixel) { Compute->InvalidateResult(); });
	UVShellSettings->WatchProperty(UVShellSettings->Resolution, [this](EBakeTextureResolution) { Compute->InvalidateResult(); });
	UVShellSettings->WatchProperty(UVShellSettings->WireframeThickness, [this](float) { Compute->InvalidateResult(); });
	UVShellSettings->WatchProperty(UVShellSettings->WireframeColor, [this](FLinearColor) { Compute->InvalidateResult(); });
	UVShellSettings->WatchProperty(UVShellSettings->ShellColor, [this](FLinearColor) { Compute->InvalidateResult(); });
	UVShellSettings->WatchProperty(UVShellSettings->BackgroundColor, [this](FLinearColor) {Compute->InvalidateResult(); });
	UVShellSettings->Result = nullptr;
	SetToolPropertySourceEnabled(UVShellSettings, true);

	// set up the detail mesh & spatial
	FDynamicMesh3 DetailMeshGet;
	Target->AppliedPreview->GetCurrentResultCopy(DetailMeshGet, false);
	DetailMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(DetailMeshGet);
	
	DetailSpatial = MakeShared<FDynamicMeshAABBTree3, ESPMode::ThreadSafe>();
	DetailSpatial->SetMesh(DetailMesh.Get(), true);

	// set up UPreviewGeometry for visualization in Unwrap viewport
	PreviewGeoBackgroundQuad = NewObject<UPreviewGeometry>(this);
	PreviewGeoBackgroundQuad->CreateInWorld(Target->UnwrapPreview->GetWorld(), FTransform::Identity);
	PreviewGeoBackgroundQuad->AddTriangleSet("UVShellMap");
	PreviewGeoBackgroundQuad->SetAllVisible(false);

	SetUpPreviewQuad();

	// Initialize background compute
	Compute = MakeUnique<TGenericDataBackgroundCompute<FMeshMapBaker>>();
	Compute->Setup(this);
	Compute->OnResultUpdated.AddLambda([this](const TUniquePtr<FMeshMapBaker>& NewResult) { OnMapUpdated(NewResult); });

	Compute->InvalidateResult();

	SetToolDisplayName(LOCTEXT("ToolNameLocal", "UV Snapshot"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartUVSnapshotTool", "Export a texture asset of a UV Layout."),
		EToolMessageLevel::UserNotification);
	
}
void UUVEditorUVSnapshotTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept)
	{
		CreateTextureAsset(UVShellSettings->Result);
	}
	
	UVShellSettings->SaveProperties(this);
	if (UVShellSettings)
	{
		UVShellSettings->RestoreProperties(this, TEXT("UVEditorUVSnapshotTool"));
		UVShellSettings = nullptr;
	}
	if (Compute)
	{
		Compute->Shutdown();
	}

	if (PreviewGeoBackgroundQuad)
	{
		PreviewGeoBackgroundQuad->Disconnect();
		PreviewGeoBackgroundQuad = nullptr;
	}
	// re-enable wireframe display and unwrap preview
	Target->WireframeDisplay->Settings->bVisible = true;
	Target->UnwrapPreview->SetVisibility(true);

	// remove 'in progress' material from 3D/live preview viewport
	Target->AppliedPreview->OverrideMaterial = nullptr;
}
void UUVEditorUVSnapshotTool::OnTick(float DeltaTime)
{
	if (Compute)
	{
		Compute->Tick(DeltaTime);
	}

	// inform the user if computation is taking a longer time
	FText Message = FText();

	if (Compute->HaveValidResult())
	{
		PreviewGeoBackgroundQuad->SetAllVisible(true);
	}
	else
	{
		Message = FText(LOCTEXT("Computing", "Computing..."));

		// hides unwrap/2d preview when computation is occurring
		PreviewGeoBackgroundQuad->SetAllVisible(false);
		// applies scrolling in progress material to 3d view while computation is occurring
		Target->AppliedPreview->OverrideMaterial = ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager());
	}

	// while computing, message will be displayed to user that it's in progress, otherwise nothing
	GetToolManager()->DisplayMessage(Message, EToolMessageLevel::UserWarning);
}

bool UUVEditorUVSnapshotTool::CanAccept() const
{
	// while computing is still occurring, do not allow accept of tool
	return Compute->HaveValidResult();
}

TUniquePtr<UE::Geometry::TGenericDataOperator<UE::Geometry::FMeshMapBaker>> UUVEditorUVSnapshotTool::MakeNewOperator()
{
	TUniquePtr<FMeshUVMapBakerOp> Op = MakeUnique<FMeshUVMapBakerOp>();
	Op->DetailSpatial = DetailSpatial;
	Op->BaseMesh = DetailMesh;

	const int32 ImageSize = (int32)UVShellSettings->Resolution;
	const FImageDimensions ImgDimensions(ImageSize, ImageSize);
	Op->BakerDimensions = ImgDimensions;
	Op->BakerSamplesPerPixel = (int32)UVShellSettings->SamplesPerPixel;

	Op->UVShellEval->UVLayer = UVShellSettings->TargetUVLayerNamesList.IndexOfByKey(UVShellSettings->UVLayer);
	Op->UVShellEval->WireframeThickness = UVShellSettings->WireframeThickness;
	Op->UVShellEval->WireframeColor = UVShellSettings->WireframeColor;
	Op->UVShellEval->ShellColor = UVShellSettings->ShellColor;
	Op->UVShellEval->BackgroundColor = UVShellSettings->BackgroundColor;
	
	return Op;
}

void UUVEditorUVSnapshotTool::OnMapUpdated(const TUniquePtr<UE::Geometry::FMeshMapBaker>& NewResult)
{
	const FImageDimensions BakeDimensions = NewResult->GetDimensions();
	FTexture2DBuilder TextureBuilder;
	TextureBuilder.Initialize(FTexture2DBuilder::ETextureType::Color, BakeDimensions);
	TextureBuilder.Copy(*NewResult->GetBakeResults(0)[0], true);
	TextureBuilder.Commit(false);

	// Copy image to source data after commit. This will avoid incurring
	// the cost of hitting the DDC for texture compile while iterating on
	// bake settings. Since this dirties the texture, the next time the texture
	// is used after accepting the final texture, the DDC will trigger and
	// properly recompile the platform data.
	constexpr ETextureSourceFormat SourceDataFormat = TSF_BGRA8; // default ChannelBits8
	TextureBuilder.CopyImageToSourceData(*NewResult->GetBakeResults(0)[0], SourceDataFormat, true);
	
	CachedUVMap = TextureBuilder.GetTexture2D();
	UpdateVisualization();
	GetToolManager()->PostInvalidation();
}

void UUVEditorUVSnapshotTool::UpdateVisualization() 
{
	UVShellSettings->Result = CachedUVMap;

	UTriangleSetComponent* TriangleSet = PreviewGeoBackgroundQuad->FindTriangleSet("UVShellMap");
	TriangleSet->SetAllTrianglesMaterial(GetMaterialForQuad());

	// apply UV map preview to 3D view
	UMaterialInstanceDynamic* Result3DMaterial = UMaterialInstanceDynamic::Create(LoadObject<UMaterial>(nullptr, TEXT("/UVEditor/Materials/UVEditorBackground")), this);
	Result3DMaterial->SetTextureParameterValue(TEXT("BackgroundBaseMap_Color"), UVShellSettings->Result);
	Target->AppliedPreview->OverrideMaterial = Result3DMaterial;

	// setting visibility here and not in setup to avoid brief moment between tool activation and initial result computation & display
	// which showed default 'BackgroundBaseMap_Color' texture
	PreviewGeoBackgroundQuad->SetAllVisible(true);
}

void UUVEditorUVSnapshotTool::SetUpPreviewQuad()
{
	// temporarily disable wireframe overlay and unwrap preview; re-enabled on shutdown
	Target->WireframeDisplay->Settings->bVisible = false;
	Target->UnwrapPreview->SetVisibility(false);

	const FVector Normal(0, 0, 1);
	const FColor BackgroundColor = FColor::Black;

	// set up rendering of 2 triangles to make one 2d quad
	FIndex2i UDimBlockToRender = FIndex2i(0,0);

	auto MakeQuadVert = [&UDimBlockToRender, &Normal, &BackgroundColor](int32 CornerX, int32 CornerY)
	{
		FVector2f ExternalUV(static_cast<float>(UDimBlockToRender.A + CornerX), static_cast<float>(UDimBlockToRender.B + CornerY));

		return FRenderableTriangleVertex(FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(ExternalUV),
			(FVector2D)FUVEditorUXSettings::ExternalUVToInternalUV(ExternalUV),
			Normal, BackgroundColor);
	};
	
	FRenderableTriangleVertex V00 = MakeQuadVert(0,0);
	FRenderableTriangleVertex V10 = MakeQuadVert(1, 0);
	FRenderableTriangleVertex V11 = MakeQuadVert(1, 1);
	FRenderableTriangleVertex V01 = MakeQuadVert(0,1);

	// connect to existing Preview Geometry
	UTriangleSetComponent* TriangleSet = PreviewGeoBackgroundQuad->FindTriangleSet("UVShellMap");
	TriangleSet->Clear();

	UMaterialInstanceDynamic* QuadMaterial = GetMaterialForQuad();

	// add to TriangleSet in PreviewGeometry
	TriangleSet->AddTriangle(FRenderableTriangle(QuadMaterial, V00, V10, V11));
	TriangleSet->AddTriangle(FRenderableTriangle(QuadMaterial, V00, V11, V01));
}

UMaterialInstanceDynamic* UUVEditorUVSnapshotTool::GetMaterialForQuad()
{
	// using this material so that we can set a texture
	UMaterialInstanceDynamic* QuadMat = UMaterialInstanceDynamic::Create(LoadObject<UMaterial>(nullptr, TEXT("/UVEditor/Materials/UVEditorBackground")), this);
	QuadMat->SetTextureParameterValue(TEXT("BackgroundBaseMap_Color"), UVShellSettings->Result);
	QuadMat->SetScalarParameterValue(TEXT("BackgroundPixelDepthOffset"), FUVEditorUXSettings::BackgroundQuadDepthOffset - 1.0f);

	return QuadMat;
}

void UUVEditorUVSnapshotTool::CreateTextureAsset(const TObjectPtr<UTexture2D>& Texture) const
{
	bool bCreatedAssetOK = true;
	FString ObjName = UE::ToolTarget::GetHumanReadableName(Target->SourceTarget);
	FString UVLayerAsString = UVShellSettings->UVLayer;
	UVLayerAsString.RemoveSpacesInline();
	FString NewAssetName = FString::Printf(TEXT("%s_UVShell_%s"), *ObjName, *UVLayerAsString); // will be something like "Cylinder_UVShell_UV0"

	// open dialog so user can choose where to save out the new asset
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	FSaveAssetDialogConfig Config;
	Config.DefaultAssetName = NewAssetName;
	Config.DialogTitleOverride = LOCTEXT("GenerateStaticMeshActorPathDialogWarning", "Choose Folder Path and Name for New Asset. Cancel to Discard New Asset.");
	// if we have previously saved a UVSnapshot, use that path as default; otherwise default path
	Config.DefaultPath = UVShellSettings->SavedPath.IsEmpty() ?  FString() : UVShellSettings->SavedPath;
	
	const FString SelectedPath = ContentBrowser.CreateModalSaveAssetDialog(Config);

	// ensures that if save dialog is closed without saving, nothing happens
	if (SelectedPath.IsEmpty() == false)
	{
		// save path so that if UV Snapshot is performed again, when save dialog opens we are already in previous location
		UVShellSettings->SavedPath = FPaths::GetPath(SelectedPath);
		NewAssetName = FPaths::GetBaseFilename(SelectedPath, true);

		FString PackageNameOut, AssetNameOut;
		const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(
			FPaths::Combine(UVShellSettings->SavedPath, NewAssetName), TEXT(""),
			PackageNameOut, AssetNameOut);

		// create the asset
		FCreateTextureObjectParams TexParams;
		TexParams.FullAssetPath = PackageNameOut;
		TexParams.GeneratedTransientTexture = Texture;
		bCreatedAssetOK = bCreatedAssetOK && UE::Modeling::CreateTextureObject( GetToolManager(),FCreateTextureObjectParams(TexParams)).IsOK();
	}
		
	ensure(bCreatedAssetOK);
}

void UUVEditorUVSnapshotTool::InitializeUVLayerNames(TArray<FString>& UVLayerNamesList, const int16 NumUVLayers)
{
	UVLayerNamesList.Reset();
	for (int16 k = 0; k < NumUVLayers; ++k)
	{
		UVLayerNamesList.Add(FString::Printf(TEXT("UV %d"), k));
	}
}

#undef LOCTEXT_NAMESPACE
