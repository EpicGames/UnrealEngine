// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorUXSettings.h"
#include "Math/Color.h"

#define LOCTEXT_NAMESPACE "UVEditorUXSettings"

const float FUVEditorUXSettings::UVMeshScalingFactor(1000.0);
const float FUVEditorUXSettings::CameraFarPlaneWorldZ(-10.0);
const float FUVEditorUXSettings::CameraNearPlaneProportionZ(0.8f); // Top layer, equivalent to depth bias 80

// 2D Viewport Depth Offsets (Organized by "layers" from the camera's perspective, descending order
// Note: While these are floating point values, they represent percentages and should be separated
// by at least integer amounts, as they serve double duty in certain cases for translucent primitive
// sorting order.
const float FUVEditorUXSettings::ToolLockedPathDepthBias(9.0);
const float FUVEditorUXSettings::ToolExtendPathDepthBias(9.0);
const float FUVEditorUXSettings::SewLineDepthOffset(8.0f);
const float FUVEditorUXSettings::SelectionHoverWireframeDepthBias(7);
const float FUVEditorUXSettings::SelectionHoverTriangleDepthBias(6);
const float FUVEditorUXSettings::SelectionWireframeDepthBias(5.0);
const float FUVEditorUXSettings::SelectionTriangleDepthBias(4.0);
const float FUVEditorUXSettings::WireframeDepthOffset(3.0);
const float FUVEditorUXSettings::UnwrapTriangleDepthOffset(2.0);

const float FUVEditorUXSettings::LivePreviewExistingSeamDepthBias(2.0);

// Note: that this offset can only be applied when we use our own background material
// for a user-supplied texture, and we can't use it for a user-provided material.
// So for consistency this should stay at zero.

const float FUVEditorUXSettings::BackgroundQuadDepthOffset(1.0); // Bottom layer

// 3D Viewport Depth Offsets
const float FUVEditorUXSettings::LivePreviewHighlightDepthOffset(1.5);

// Opacities
const float FUVEditorUXSettings::UnwrapTriangleOpacity(1.0);
const float FUVEditorUXSettings::UnwrapTriangleOpacityWithBackground(0.25);
const float FUVEditorUXSettings::SelectionTriangleOpacity(1.0f);
const float FUVEditorUXSettings::SelectionHoverTriangleOpacity(1.0f);

// Per Asset Shifts
const float FUVEditorUXSettings::UnwrapBoundaryHueShift(30);
const float FUVEditorUXSettings::UnwrapBoundarySaturation(0.50);
const float FUVEditorUXSettings::UnwrapBoundaryValue(0.50);

// Colors
const FColor FUVEditorUXSettings::UnwrapTriangleFillColor(FColor::FromHex("#696871"));
const FColor FUVEditorUXSettings::UnwrapTriangleWireframeColor(FColor::FromHex("#989898"));
const FColor FUVEditorUXSettings::SelectionTriangleFillColor(FColor::FromHex("#8C7A52"));
const FColor FUVEditorUXSettings::SelectionTriangleWireframeColor(FColor::FromHex("#DDA209"));
const FColor FUVEditorUXSettings::SelectionHoverTriangleFillColor(FColor::FromHex("#4E719B"));
const FColor FUVEditorUXSettings::SelectionHoverTriangleWireframeColor(FColor::FromHex("#0E86FF"));
const FColor FUVEditorUXSettings::SewSideLeftColor(FColor::Red);
const FColor FUVEditorUXSettings::SewSideRightColor(FColor::Green);

const FColor FUVEditorUXSettings::ToolLockedCutPathColor(FColor::Green);
const FColor FUVEditorUXSettings::ToolExtendCutPathColor(FColor::Green);
const FColor FUVEditorUXSettings::ToolLockedJoinPathColor(FColor::Turquoise);
const FColor FUVEditorUXSettings::ToolExtendJoinPathColor(FColor::Turquoise);
const FColor FUVEditorUXSettings::ToolCompletionPathColor(FColor::Orange);

const FColor FUVEditorUXSettings::LivePreviewExistingSeamColor(FColor::Green);

const FColor FUVEditorUXSettings::XAxisColor(FColor::Red);
const FColor FUVEditorUXSettings::YAxisColor(FColor::Green);
const FColor FUVEditorUXSettings::GridMajorColor(FColor::FromHex("#888888"));
const FColor FUVEditorUXSettings::GridMinorColor(FColor::FromHex("#777777"));
const FColor FUVEditorUXSettings::RulerXColor(FColor::FromHex("#888888"));
const FColor FUVEditorUXSettings::RulerYColor(FColor::FromHex("#888888"));
const FColor FUVEditorUXSettings::PivotLineColor(FColor::Cyan);

// Thicknesses
const float FUVEditorUXSettings::LivePreviewHighlightThickness(2.0);
const float FUVEditorUXSettings::LivePreviewHighlightPointSize(4);
const float FUVEditorUXSettings::LivePreviewExistingSeamThickness(2.0);
const float FUVEditorUXSettings::SelectionLineThickness(1.5);
const float FUVEditorUXSettings::ToolLockedPathThickness(3.0f);
const float FUVEditorUXSettings::ToolExtendPathThickness(3.0f);
const float FUVEditorUXSettings::SelectionPointThickness(6);
const float FUVEditorUXSettings::SewLineHighlightThickness(3.0f);
const float FUVEditorUXSettings::AxisThickness(2.0);
const float FUVEditorUXSettings::GridMajorThickness(1.0);
const float FUVEditorUXSettings::WireframeThickness(2.0);
const float FUVEditorUXSettings::BoundaryEdgeThickness(2.0);
const float FUVEditorUXSettings::ToolPointSize(6);
const float FUVEditorUXSettings::PivotLineThickness(1.5);

// Grid
const int32 FUVEditorUXSettings::GridSubdivisionsPerLevel(4);
const int32 FUVEditorUXSettings::GridLevels(3);
const int32 FUVEditorUXSettings::RulerSubdivisionLevel(1);

// Pivot Visuals
const int32 FUVEditorUXSettings::PivotCircleNumSides(32);
const float FUVEditorUXSettings::PivotCircleRadius(10.0);

// CVARs

TAutoConsoleVariable<int32> FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport(
	TEXT("modeling.UVEditor.UDIMSupport"),
	1,
	TEXT("Enable experimental UDIM support in the UVEditor"));


FLinearColor FUVEditorUXSettings::GetTriangleColorByTargetIndex(int32 TargetIndex)
{
	double GoldenAngle = 137.50776405;

	FLinearColor BaseColorHSV = FLinearColor::FromSRGBColor(UnwrapTriangleFillColor).LinearRGBToHSV();
	BaseColorHSV.R = static_cast<float>(FMath::Fmod(BaseColorHSV.R + (GoldenAngle / 2.0 * TargetIndex), 360));

	return BaseColorHSV.HSVToLinearRGB();
}

FLinearColor FUVEditorUXSettings::GetWireframeColorByTargetIndex(int32 TargetIndex)
{
	return FLinearColor::FromSRGBColor(UnwrapTriangleWireframeColor);;
}

FLinearColor FUVEditorUXSettings::GetBoundaryColorByTargetIndex(int32 TargetIndex)
{
	FLinearColor BaseColorHSV = GetTriangleColorByTargetIndex(TargetIndex).LinearRGBToHSV();
	FLinearColor BoundaryColorHSV = BaseColorHSV;
	BoundaryColorHSV.R = FMath::Fmod((BoundaryColorHSV.R + UnwrapBoundaryHueShift), 360);
	BoundaryColorHSV.G = UnwrapBoundarySaturation;
	BoundaryColorHSV.B = UnwrapBoundaryValue;
	return BoundaryColorHSV.HSVToLinearRGB();
}

FColor FUVEditorUXSettings::MakeCividisColorFromScalar(float Scalar)
{
	// Color map sourced from:
	// "Optimizing colormaps with consideration for color vision deficiency to enable accurate interpretation of scientific data"
	// https://doi.org/10.1371/journal.pone.0199239

	static const float RComponents[256] = { 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f,
		0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f, 0.0000f,
		0.0000f, 0.0000f, 0.0055f, 0.0236f, 0.0416f, 0.0576f, 0.0710f, 0.0827f, 0.0932f, 0.1030f, 0.1120f, 0.1204f, 0.1283f, 0.1359f, 0.1431f, 0.1500f,
		0.1566f, 0.1630f, 0.1692f, 0.1752f, 0.1811f, 0.1868f, 0.1923f, 0.1977f, 0.2030f, 0.2082f, 0.2133f, 0.2183f, 0.2232f, 0.2281f, 0.2328f, 0.2375f,
		0.2421f, 0.2466f, 0.2511f, 0.2556f, 0.2599f, 0.2643f, 0.2686f, 0.2728f, 0.2770f, 0.2811f, 0.2853f, 0.2894f, 0.2934f, 0.2974f, 0.3014f, 0.3054f,
		0.3093f, 0.3132f, 0.3170f, 0.3209f, 0.3247f, 0.3285f, 0.3323f, 0.3361f, 0.3398f, 0.3435f, 0.3472f, 0.3509f, 0.3546f, 0.3582f, 0.3619f, 0.3655f,
		0.3691f, 0.3727f, 0.3763f, 0.3798f, 0.3834f, 0.3869f, 0.3905f, 0.3940f, 0.3975f, 0.4010f, 0.4045f, 0.4080f, 0.4114f, 0.4149f, 0.4183f, 0.4218f,
		0.4252f, 0.4286f, 0.4320f, 0.4354f, 0.4388f, 0.4422f, 0.4456f, 0.4489f, 0.4523f, 0.4556f, 0.4589f, 0.4622f, 0.4656f, 0.4689f, 0.4722f, 0.4756f,
		0.4790f, 0.4825f, 0.4861f, 0.4897f, 0.4934f, 0.4971f, 0.5008f, 0.5045f, 0.5083f, 0.5121f, 0.5158f, 0.5196f, 0.5234f, 0.5272f, 0.5310f, 0.5349f,
		0.5387f, 0.5425f, 0.5464f, 0.5502f, 0.5541f, 0.5579f, 0.5618f, 0.5657f, 0.5696f, 0.5735f, 0.5774f, 0.5813f, 0.5852f, 0.5892f, 0.5931f, 0.5970f,
		0.6010f, 0.6050f, 0.6089f, 0.6129f, 0.6168f, 0.6208f, 0.6248f, 0.6288f, 0.6328f, 0.6368f, 0.6408f, 0.6449f, 0.6489f, 0.6529f, 0.6570f, 0.6610f,
		0.6651f, 0.6691f, 0.6732f, 0.6773f, 0.6813f, 0.6854f, 0.6895f, 0.6936f, 0.6977f, 0.7018f, 0.7060f, 0.7101f, 0.7142f, 0.7184f, 0.7225f, 0.7267f,
		0.7308f, 0.7350f, 0.7392f, 0.7434f, 0.7476f, 0.7518f, 0.7560f, 0.7602f, 0.7644f, 0.7686f, 0.7729f, 0.7771f, 0.7814f, 0.7856f, 0.7899f, 0.7942f,
		0.7985f, 0.8027f, 0.8070f, 0.8114f, 0.8157f, 0.8200f, 0.8243f, 0.8287f, 0.8330f, 0.8374f, 0.8417f, 0.8461f, 0.8505f, 0.8548f, 0.8592f, 0.8636f,
		0.8681f, 0.8725f, 0.8769f, 0.8813f, 0.8858f, 0.8902f, 0.8947f, 0.8992f, 0.9037f, 0.9082f, 0.9127f, 0.9172f, 0.9217f, 0.9262f, 0.9308f, 0.9353f,
		0.9399f, 0.9444f, 0.9490f, 0.9536f, 0.9582f, 0.9628f, 0.9674f, 0.9721f, 0.9767f, 0.9814f, 0.9860f, 0.9907f, 0.9954f, 1.0000f, 1.0000f, 1.0000f,
		1.0000f, 1.0000f, 1.0000f};
	static const float GComponents[256] = { 0.1262f, 0.1292f, 0.1321f, 0.1350f, 0.1379f, 0.1408f, 0.1437f, 0.1465f, 0.1492f, 0.1519f, 0.1546f, 0.1574f, 0.1601f,
		0.1629f, 0.1657f, 0.1685f, 0.1714f, 0.1743f, 0.1773f, 0.1798f, 0.1817f, 0.1834f, 0.1852f, 0.1872f, 0.1901f, 0.1930f, 0.1958f, 0.1987f, 0.2015f,
		0.2044f, 0.2073f, 0.2101f, 0.2130f, 0.2158f, 0.2187f, 0.2215f, 0.2244f, 0.2272f, 0.2300f, 0.2329f, 0.2357f, 0.2385f, 0.2414f, 0.2442f, 0.2470f,
		0.2498f, 0.2526f, 0.2555f, 0.2583f, 0.2611f, 0.2639f, 0.2667f, 0.2695f, 0.2723f, 0.2751f, 0.2780f, 0.2808f, 0.2836f, 0.2864f, 0.2892f, 0.2920f,
		0.2948f, 0.2976f, 0.3004f, 0.3032f, 0.3060f, 0.3088f, 0.3116f, 0.3144f, 0.3172f, 0.3200f, 0.3228f, 0.3256f, 0.3284f, 0.3312f, 0.3340f, 0.3368f,
		0.3396f, 0.3424f, 0.3453f, 0.3481f, 0.3509f, 0.3537f, 0.3565f, 0.3593f, 0.3622f, 0.3650f, 0.3678f, 0.3706f, 0.3734f, 0.3763f, 0.3791f, 0.3819f,
		0.3848f, 0.3876f, 0.3904f, 0.3933f, 0.3961f, 0.3990f, 0.4018f, 0.4047f, 0.4075f, 0.4104f, 0.4132f, 0.4161f, 0.4189f, 0.4218f, 0.4247f, 0.4275f,
		0.4304f, 0.4333f, 0.4362f, 0.4390f, 0.4419f, 0.4448f, 0.4477f, 0.4506f, 0.4535f, 0.4564f, 0.4593f, 0.4622f, 0.4651f, 0.4680f, 0.4709f, 0.4738f,
		0.4767f, 0.4797f, 0.4826f, 0.4856f, 0.4886f, 0.4915f, 0.4945f, 0.4975f, 0.5005f, 0.5035f, 0.5065f, 0.5095f, 0.5125f, 0.5155f, 0.5186f, 0.5216f,
		0.5246f, 0.5277f, 0.5307f, 0.5338f, 0.5368f, 0.5399f, 0.5430f, 0.5461f, 0.5491f, 0.5522f, 0.5553f, 0.5584f, 0.5615f, 0.5646f, 0.5678f, 0.5709f,
		0.5740f, 0.5772f, 0.5803f, 0.5835f, 0.5866f, 0.5898f, 0.5929f, 0.5961f, 0.5993f, 0.6025f, 0.6057f, 0.6089f, 0.6121f, 0.6153f, 0.6185f, 0.6217f,
		0.6250f, 0.6282f, 0.6315f, 0.6347f, 0.6380f, 0.6412f, 0.6445f, 0.6478f, 0.6511f, 0.6544f, 0.6577f, 0.6610f, 0.6643f, 0.6676f, 0.6710f, 0.6743f,
		0.6776f, 0.6810f, 0.6844f, 0.6877f, 0.6911f, 0.6945f, 0.6979f, 0.7013f, 0.7047f, 0.7081f, 0.7115f, 0.7150f, 0.7184f, 0.7218f, 0.7253f, 0.7288f,
		0.7322f, 0.7357f, 0.7392f, 0.7427f, 0.7462f, 0.7497f, 0.7532f, 0.7568f, 0.7603f, 0.7639f, 0.7674f, 0.7710f, 0.7745f, 0.7781f, 0.7817f, 0.7853f,
		0.7889f, 0.7926f, 0.7962f, 0.7998f, 0.8035f, 0.8071f, 0.8108f, 0.8145f, 0.8182f, 0.8219f, 0.8256f, 0.8293f, 0.8330f, 0.8367f, 0.8405f, 0.8442f,
		0.8480f, 0.8518f, 0.8556f, 0.8593f, 0.8632f, 0.8670f, 0.8708f, 0.8746f, 0.8785f, 0.8823f, 0.8862f, 0.8901f, 0.8940f, 0.8979f, 0.9018f, 0.9057f,
		0.9094f, 0.9131f, 0.9169f};
	static const float BComponents[256] = { 0.3015f, 0.3077f, 0.3142f, 0.3205f, 0.3269f, 0.3334f, 0.3400f, 0.3467f, 0.3537f, 0.3606f, 0.3676f, 0.3746f, 0.3817f,
		0.3888f, 0.3960f, 0.4031f, 0.4102f, 0.4172f, 0.4241f, 0.4307f, 0.4347f, 0.4363f, 0.4368f, 0.4368f, 0.4365f, 0.4361f, 0.4356f, 0.4349f, 0.4343f,
		0.4336f, 0.4329f, 0.4322f, 0.4314f, 0.4308f, 0.4301f, 0.4293f, 0.4287f, 0.4280f, 0.4274f, 0.4268f, 0.4262f, 0.4256f, 0.4251f, 0.4245f, 0.4241f,
		0.4236f, 0.4232f, 0.4228f, 0.4224f, 0.4220f, 0.4217f, 0.4214f, 0.4212f, 0.4209f, 0.4207f, 0.4205f, 0.4204f, 0.4203f, 0.4202f, 0.4201f, 0.4200f,
		0.4200f, 0.4200f, 0.4201f, 0.4201f, 0.4202f, 0.4203f, 0.4205f, 0.4206f, 0.4208f, 0.4210f, 0.4212f, 0.4215f, 0.4218f, 0.4221f, 0.4224f, 0.4227f,
		0.4231f, 0.4236f, 0.4240f, 0.4244f, 0.4249f, 0.4254f, 0.4259f, 0.4264f, 0.4270f, 0.4276f, 0.4282f, 0.4288f, 0.4294f, 0.4302f, 0.4308f, 0.4316f,
		0.4322f, 0.4331f, 0.4338f, 0.4346f, 0.4355f, 0.4364f, 0.4372f, 0.4381f, 0.4390f, 0.4400f, 0.4409f, 0.4419f, 0.4430f, 0.4440f, 0.4450f, 0.4462f,
		0.4473f, 0.4485f, 0.4496f, 0.4508f, 0.4521f, 0.4534f, 0.4547f, 0.4561f, 0.4575f, 0.4589f, 0.4604f, 0.4620f, 0.4635f, 0.4650f, 0.4665f, 0.4679f,
		0.4691f, 0.4701f, 0.4707f, 0.4714f, 0.4719f, 0.4723f, 0.4727f, 0.4730f, 0.4732f, 0.4734f, 0.4736f, 0.4737f, 0.4738f, 0.4739f, 0.4739f, 0.4738f,
		0.4739f, 0.4738f, 0.4736f, 0.4735f, 0.4733f, 0.4732f, 0.4729f, 0.4727f, 0.4723f, 0.4720f, 0.4717f, 0.4714f, 0.4709f, 0.4705f, 0.4701f, 0.4696f,
		0.4691f, 0.4685f, 0.4680f, 0.4673f, 0.4668f, 0.4662f, 0.4655f, 0.4649f, 0.4641f, 0.4632f, 0.4625f, 0.4617f, 0.4609f, 0.4600f, 0.4591f, 0.4583f,
		0.4573f, 0.4562f, 0.4553f, 0.4543f, 0.4532f, 0.4521f, 0.4511f, 0.4499f, 0.4487f, 0.4475f, 0.4463f, 0.4450f, 0.4437f, 0.4424f, 0.4409f, 0.4396f,
		0.4382f, 0.4368f, 0.4352f, 0.4338f, 0.4322f, 0.4307f, 0.4290f, 0.4273f, 0.4258f, 0.4241f, 0.4223f, 0.4205f, 0.4188f, 0.4168f, 0.4150f, 0.4129f, 
		0.4111f, 0.4090f, 0.4070f, 0.4049f, 0.4028f, 0.4007f, 0.3984f, 0.3961f, 0.3938f, 0.3915f, 0.3892f, 0.3869f, 0.3843f, 0.3818f, 0.3793f, 0.3766f,
		0.3739f, 0.3712f, 0.3684f, 0.3657f, 0.3627f, 0.3599f, 0.3569f, 0.3538f, 0.3507f, 0.3474f, 0.3442f, 0.3409f, 0.3374f, 0.3340f, 0.3306f, 0.3268f,
		0.3232f, 0.3195f, 0.3155f, 0.3116f, 0.3076f, 0.3034f, 0.2990f, 0.2947f, 0.2901f, 0.2856f, 0.2807f, 0.2759f, 0.2708f, 0.2655f, 0.2600f, 0.2593f,
		0.2634f, 0.2680f, 0.2731f};

	const float RedSclr = FMath::Clamp<float>(Scalar, 0.f, 1.f);
	const float GreenSclr = FMath::Clamp<float>(Scalar, 0.f, 1.f);
	const float BlueSclr = FMath::Clamp<float>(Scalar, 0.f, 1.f);
	const uint8 R = (uint8)FMath::TruncToInt(254 * RComponents[(uint8)FMath::TruncToInt(254 * RedSclr)]);
	const uint8 G = (uint8)FMath::TruncToInt(254 * GComponents[(uint8)FMath::TruncToInt(254 * GreenSclr)]);
	const uint8 B = (uint8)FMath::TruncToInt(254 * BComponents[(uint8)FMath::TruncToInt(254 * BlueSclr)]);
	return FColor(R, G, B);
}

FColor FUVEditorUXSettings::MakeTurboColorFromScalar(float Scalar)
{
	// Color map sourced from:
	// https://ai.googleblog.com/2019/08/turbo-improved-rainbow-colormap-for.html

	static const float TurboSRGB[256][3] = { {0.18995f,0.07176f,0.23217f},{0.19483f,0.08339f,0.26149f},{0.19956f,0.09498f,0.29024f},{0.20415f,0.10652f,0.31844f},
		{0.20860f,0.11802f,0.34607f},{0.21291f,0.12947f,0.37314f},{0.21708f,0.14087f,0.39964f},{0.22111f,0.15223f,0.42558f},{0.22500f,0.16354f,0.45096f},
		{0.22875f,0.17481f,0.47578f},{0.23236f,0.18603f,0.50004f},{0.23582f,0.19720f,0.52373f},{0.23915f,0.20833f,0.54686f},{0.24234f,0.21941f,0.56942f},
		{0.24539f,0.23044f,0.59142f},{0.24830f,0.24143f,0.61286f},{0.25107f,0.25237f,0.63374f},{0.25369f,0.26327f,0.65406f},{0.25618f,0.27412f,0.67381f},
		{0.25853f,0.28492f,0.69300f},{0.26074f,0.29568f,0.71162f},{0.26280f,0.30639f,0.72968f},{0.26473f,0.31706f,0.74718f},{0.26652f,0.32768f,0.76412f},
		{0.26816f,0.33825f,0.78050f},{0.26967f,0.34878f,0.79631f},{0.27103f,0.35926f,0.81156f},{0.27226f,0.36970f,0.82624f},{0.27334f,0.38008f,0.84037f},
		{0.27429f,0.39043f,0.85393f},{0.27509f,0.40072f,0.86692f},{0.27576f,0.41097f,0.87936f},{0.27628f,0.42118f,0.89123f},{0.27667f,0.43134f,0.90254f},
		{0.27691f,0.44145f,0.91328f},{0.27701f,0.45152f,0.92347f},{0.27698f,0.46153f,0.93309f},{0.27680f,0.47151f,0.94214f},{0.27648f,0.48144f,0.95064f},
		{0.27603f,0.49132f,0.95857f},{0.27543f,0.50115f,0.96594f},{0.27469f,0.51094f,0.97275f},{0.27381f,0.52069f,0.97899f},{0.27273f,0.53040f,0.98461f},
		{0.27106f,0.54015f,0.98930f},{0.26878f,0.54995f,0.99303f},{0.26592f,0.55979f,0.99583f},{0.26252f,0.56967f,0.99773f},{0.25862f,0.57958f,0.99876f},
		{0.25425f,0.58950f,0.99896f},{0.24946f,0.59943f,0.99835f},{0.24427f,0.60937f,0.99697f},{0.23874f,0.61931f,0.99485f},{0.23288f,0.62923f,0.99202f},
		{0.22676f,0.63913f,0.98851f},{0.22039f,0.64901f,0.98436f},{0.21382f,0.65886f,0.97959f},{0.20708f,0.66866f,0.97423f},{0.20021f,0.67842f,0.96833f},
		{0.19326f,0.68812f,0.96190f},{0.18625f,0.69775f,0.95498f},{0.17923f,0.70732f,0.94761f},{0.17223f,0.71680f,0.93981f},{0.16529f,0.72620f,0.93161f},
		{0.15844f,0.73551f,0.92305f},{0.15173f,0.74472f,0.91416f},{0.14519f,0.75381f,0.90496f},{0.13886f,0.76279f,0.89550f},{0.13278f,0.77165f,0.88580f},
		{0.12698f,0.78037f,0.87590f},{0.12151f,0.78896f,0.86581f},{0.11639f,0.79740f,0.85559f},{0.11167f,0.80569f,0.84525f},{0.10738f,0.81381f,0.83484f},
		{0.10357f,0.82177f,0.82437f},{0.10026f,0.82955f,0.81389f},{0.09750f,0.83714f,0.80342f},{0.09532f,0.84455f,0.79299f},{0.09377f,0.85175f,0.78264f},
		{0.09287f,0.85875f,0.77240f},{0.09267f,0.86554f,0.76230f},{0.09320f,0.87211f,0.75237f},{0.09451f,0.87844f,0.74265f},{0.09662f,0.88454f,0.73316f},
		{0.09958f,0.89040f,0.72393f},{0.10342f,0.89600f,0.71500f},{0.10815f,0.90142f,0.70599f},{0.11374f,0.90673f,0.69651f},{0.12014f,0.91193f,0.68660f},
		{0.12733f,0.91701f,0.67627f},{0.13526f,0.92197f,0.66556f},{0.14391f,0.92680f,0.65448f},{0.15323f,0.93151f,0.64308f},{0.16319f,0.93609f,0.63137f},
		{0.17377f,0.94053f,0.61938f},{0.18491f,0.94484f,0.60713f},{0.19659f,0.94901f,0.59466f},{0.20877f,0.95304f,0.58199f},{0.22142f,0.95692f,0.56914f},
		{0.23449f,0.96065f,0.55614f},{0.24797f,0.96423f,0.54303f},{0.26180f,0.96765f,0.52981f},{0.27597f,0.97092f,0.51653f},{0.29042f,0.97403f,0.50321f},
		{0.30513f,0.97697f,0.48987f},{0.32006f,0.97974f,0.47654f},{0.33517f,0.98234f,0.46325f},{0.35043f,0.98477f,0.45002f},{0.36581f,0.98702f,0.43688f},
		{0.38127f,0.98909f,0.42386f},{0.39678f,0.99098f,0.41098f},{0.41229f,0.99268f,0.39826f},{0.42778f,0.99419f,0.38575f},{0.44321f,0.99551f,0.37345f},
		{0.45854f,0.99663f,0.36140f},{0.47375f,0.99755f,0.34963f},{0.48879f,0.99828f,0.33816f},{0.50362f,0.99879f,0.32701f},{0.51822f,0.99910f,0.31622f},
		{0.53255f,0.99919f,0.30581f},{0.54658f,0.99907f,0.29581f},{0.56026f,0.99873f,0.28623f},{0.57357f,0.99817f,0.27712f},{0.58646f,0.99739f,0.26849f},
		{0.59891f,0.99638f,0.26038f},{0.61088f,0.99514f,0.25280f},{0.62233f,0.99366f,0.24579f},{0.63323f,0.99195f,0.23937f},{0.64362f,0.98999f,0.23356f},
		{0.65394f,0.98775f,0.22835f},{0.66428f,0.98524f,0.22370f},{0.67462f,0.98246f,0.21960f},{0.68494f,0.97941f,0.21602f},{0.69525f,0.97610f,0.21294f},
		{0.70553f,0.97255f,0.21032f},{0.71577f,0.96875f,0.20815f},{0.72596f,0.96470f,0.20640f},{0.73610f,0.96043f,0.20504f},{0.74617f,0.95593f,0.20406f},
		{0.75617f,0.95121f,0.20343f},{0.76608f,0.94627f,0.20311f},{0.77591f,0.94113f,0.20310f},{0.78563f,0.93579f,0.20336f},{0.79524f,0.93025f,0.20386f},
		{0.80473f,0.92452f,0.20459f},{0.81410f,0.91861f,0.20552f},{0.82333f,0.91253f,0.20663f},{0.83241f,0.90627f,0.20788f},{0.84133f,0.89986f,0.20926f},
		{0.85010f,0.89328f,0.21074f},{0.85868f,0.88655f,0.21230f},{0.86709f,0.87968f,0.21391f},{0.87530f,0.87267f,0.21555f},{0.88331f,0.86553f,0.21719f},
		{0.89112f,0.85826f,0.21880f},{0.89870f,0.85087f,0.22038f},{0.90605f,0.84337f,0.22188f},{0.91317f,0.83576f,0.22328f},{0.92004f,0.82806f,0.22456f},
		{0.92666f,0.82025f,0.22570f},{0.93301f,0.81236f,0.22667f},{0.93909f,0.80439f,0.22744f},{0.94489f,0.79634f,0.22800f},{0.95039f,0.78823f,0.22831f},
		{0.95560f,0.78005f,0.22836f},{0.96049f,0.77181f,0.22811f},{0.96507f,0.76352f,0.22754f},{0.96931f,0.75519f,0.22663f},{0.97323f,0.74682f,0.22536f},
		{0.97679f,0.73842f,0.22369f},{0.98000f,0.73000f,0.22161f},{0.98289f,0.72140f,0.21918f},{0.98549f,0.71250f,0.21650f},{0.98781f,0.70330f,0.21358f},
		{0.98986f,0.69382f,0.21043f},{0.99163f,0.68408f,0.20706f},{0.99314f,0.67408f,0.20348f},{0.99438f,0.66386f,0.19971f},{0.99535f,0.65341f,0.19577f},
		{0.99607f,0.64277f,0.19165f},{0.99654f,0.63193f,0.18738f},{0.99675f,0.62093f,0.18297f},{0.99672f,0.60977f,0.17842f},{0.99644f,0.59846f,0.17376f},
		{0.99593f,0.58703f,0.16899f},{0.99517f,0.57549f,0.16412f},{0.99419f,0.56386f,0.15918f},{0.99297f,0.55214f,0.15417f},{0.99153f,0.54036f,0.14910f},
		{0.98987f,0.52854f,0.14398f},{0.98799f,0.51667f,0.13883f},{0.98590f,0.50479f,0.13367f},{0.98360f,0.49291f,0.12849f},{0.98108f,0.48104f,0.12332f},
		{0.97837f,0.46920f,0.11817f},{0.97545f,0.45740f,0.11305f},{0.97234f,0.44565f,0.10797f},{0.96904f,0.43399f,0.10294f},{0.96555f,0.42241f,0.09798f},
		{0.96187f,0.41093f,0.09310f},{0.95801f,0.39958f,0.08831f},{0.95398f,0.38836f,0.08362f},{0.94977f,0.37729f,0.07905f},{0.94538f,0.36638f,0.07461f},
		{0.94084f,0.35566f,0.07031f},{0.93612f,0.34513f,0.06616f},{0.93125f,0.33482f,0.06218f},{0.92623f,0.32473f,0.05837f},{0.92105f,0.31489f,0.05475f},
		{0.91572f,0.30530f,0.05134f},{0.91024f,0.29599f,0.04814f},{0.90463f,0.28696f,0.04516f},{0.89888f,0.27824f,0.04243f},{0.89298f,0.26981f,0.03993f},
		{0.88691f,0.26152f,0.03753f},{0.88066f,0.25334f,0.03521f},{0.87422f,0.24526f,0.03297f},{0.86760f,0.23730f,0.03082f},{0.86079f,0.22945f,0.02875f},
		{0.85380f,0.22170f,0.02677f},{0.84662f,0.21407f,0.02487f},{0.83926f,0.20654f,0.02305f},{0.83172f,0.19912f,0.02131f},{0.82399f,0.19182f,0.01966f},
		{0.81608f,0.18462f,0.01809f},{0.80799f,0.17753f,0.01660f},{0.79971f,0.17055f,0.01520f},{0.79125f,0.16368f,0.01387f},{0.78260f,0.15693f,0.01264f},
		{0.77377f,0.15028f,0.01148f},{0.76476f,0.14374f,0.01041f},{0.75556f,0.13731f,0.00942f},{0.74617f,0.13098f,0.00851f},{0.73661f,0.12477f,0.00769f},
		{0.72686f,0.11867f,0.00695f},{0.71692f,0.11268f,0.00629f},{0.70680f,0.10680f,0.00571f},{0.69650f,0.10102f,0.00522f},{0.68602f,0.09536f,0.00481f},
		{0.67535f,0.08980f,0.00449f},{0.66449f,0.08436f,0.00424f},{0.65345f,0.07902f,0.00408f},{0.64223f,0.07380f,0.00401f},{0.63082f,0.06868f,0.00401f},
		{0.61923f,0.06367f,0.00410f},{0.60746f,0.05878f,0.00427f},{0.59550f,0.05399f,0.00453f},{0.58336f,0.04931f,0.00486f},{0.57103f,0.04474f,0.00529f},
		{0.55852f,0.04028f,0.00579f},{0.54583f,0.03593f,0.00638f},{0.53295f,0.03169f,0.00705f},{0.51989f,0.02756f,0.00780f},{0.50664f,0.02354f,0.00863f},
		{0.49321f,0.01963f,0.00955f},{0.47960f,0.01583f,0.01055f} };

	const float ClampedScalar = FMath::Clamp<float>(Scalar, -0.5f, 0.5f);
	const uint8 RampIndex = (uint8)FMath::TruncToInt(255 * (ClampedScalar + 0.5f));
	const uint8 R = (uint8)FMath::TruncToInt(255 * TurboSRGB[RampIndex][0]);
	const uint8 G = (uint8)FMath::TruncToInt(255 * TurboSRGB[RampIndex][1]);
	const uint8 B = (uint8)FMath::TruncToInt(255 * TurboSRGB[RampIndex][2]);
	FLinearColor LinearColor = FLinearColor::FromSRGBColor(FColor(R, G, B));
	return LinearColor.ToFColor(false);
}

// The mappings here depend on the way we view the unwrap world, set up in UVEditorToolkit.cpp.
// Currently, we look at the world with Z pointing towards the camera, the positive X axis pointing right,
// and the positive Y axis pointing down. This means that we have to flip the V coordinate to map it
// to the up direction.
FVector3d FUVEditorUXSettings::ExternalUVToUnwrapWorldPosition(const FVector2f& UV)
{
	return FVector3d(UV.X * UVMeshScalingFactor, -UV.Y * UVMeshScalingFactor, 0);
}
FVector2f FUVEditorUXSettings::UnwrapWorldPositionToExternalUV(const FVector3d& VertPosition)
{
	return FVector2f(static_cast<float>(VertPosition.X) / UVMeshScalingFactor, -(static_cast<float>(VertPosition.Y) / UVMeshScalingFactor));
}

// Unreal stores its UVs with V subtracted from 1.
FVector2f FUVEditorUXSettings::ExternalUVToInternalUV(const FVector2f& UV)
{
	return FVector2f(UV.X, 1-UV.Y);
}
FVector2f FUVEditorUXSettings::InternalUVToExternalUV(const FVector2f& UV)
{
	return FVector2f(UV.X, 1-UV.Y);
}

// Should be equivalent to converting from unreal's UV to regular (external) and then to unwrap world,
// i.e. ExternalUVToUnwrapWorldPosition(InternalUVToExternalUV(UV))
FVector3d FUVEditorUXSettings::UVToVertPosition(const FVector2f& UV)
{
	return FVector3d(UV.X * UVMeshScalingFactor, (UV.Y - 1) * UVMeshScalingFactor, 0);
}

// Should be equivalent to converting from world to regular (external) UV, and then to unreal's representation,
// i.e. ExternalUVToInternalUV(UnwrapWorldPositionToExternalUV(VertPosition))
FVector2f FUVEditorUXSettings::VertPositionToUV(const FVector3d& VertPosition)
{
	return FVector2f(static_cast<float>(VertPosition.X) / UVMeshScalingFactor, 1.0 + (static_cast<float>(VertPosition.Y) / UVMeshScalingFactor));
};


float FUVEditorUXSettings::LocationSnapValue(int32 LocationSnapMenuIndex)
{
	switch (LocationSnapMenuIndex)
	{
	case 0:
		return 1.0;
	case 1:
		return 0.5;
	case 2:
		return 0.25;
	case 3:
		return 0.125;
	case 4:
		return 0.0625;
	default:
		ensure(false);
		return 0;
	}
}

int32 FUVEditorUXSettings::MaxLocationSnapValue()
{
	return 5;
}

#undef LOCTEXT_NAMESPACE 
