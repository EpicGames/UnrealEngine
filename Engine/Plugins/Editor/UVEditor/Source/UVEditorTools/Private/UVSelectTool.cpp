// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVSelectTool.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/GizmoBaseComponent.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "ContextObjects/UVToolViewportButtonsAPI.h"
#include "ContextObjectStore.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "PreviewMesh.h"
#include "Selection/UVToolSelection.h"
#include "ToolSetupUtil.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "UVEditorUXSettings.h"

#include "ToolTargetManager.h"
#include "Algo/Unique.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVSelectTool)

#define LOCTEXT_NAMESPACE "UUVSelectTool"

using namespace UE::Geometry;

namespace UVSelectToolLocals
{
	FText GizmoChangeTransactionName = LOCTEXT("GizmoChange", "Gizmo Update");

	/**
	 * Helper change that allows us to preserve the rotational component of the gizmo, since
	 * just updating the gizmo from the current selection will always reset rotation.
	 */
	class  FGizmoChange : public FToolCommandChange
	{
	public:
		FGizmoChange(const FTransform& GizmoBeforeIn)
			: GizmoBefore(GizmoBeforeIn)
		{
		};

		virtual void Apply(UObject* Object) override
		{
			// Route the action to the currently running instance of the select tool
			UInteractiveToolManager* ToolManager = Cast<UInteractiveToolManager>(Object);
			UUVSelectTool* SelectTool = Cast<UUVSelectTool>(ToolManager->GetActiveTool(EToolSide::Mouse));
			if (ensure(SelectTool))
			{
				SelectTool->SetGizmoTransform(GizmoAfter);
			}
		}

		virtual void Revert(UObject* Object) override
		{
			// Route the action to the currently running instance of the select tool
			UInteractiveToolManager* ToolManager = Cast<UInteractiveToolManager>(Object);
			UUVSelectTool* SelectTool = Cast<UUVSelectTool>(ToolManager->GetActiveTool(EToolSide::Mouse));
			if (ensure(SelectTool))
			{
				// This is the easiest way to make sure the transform reverts back to whatever it was changed to
				// on redo (which we may not fully know at time of emitting if we're emitting on tool shutdown,
				// and the next transform is whatever exists at next tool invocation).
				GizmoAfter = SelectTool->GetGizmoTransform();
				SelectTool->SetGizmoTransform(GizmoBefore);
			}
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			// Expired if currently active tool cannot be cast to select tool
			UInteractiveToolManager* ToolManager = Cast<UInteractiveToolManager>(Object);
			return Cast<UUVSelectTool>(ToolManager->GetActiveTool(EToolSide::Mouse)) == nullptr;
		}


		virtual FString ToString() const override
		{
			return TEXT("UVSelectToolLocals::FGizmoChange");
		}

	protected:
		FTransform GizmoBefore;
		FTransform GizmoAfter;
	};

	/**
	 * A change similar to the one emitted by EmitChangeApi->EmitToolIndependentUnwrapCanonicalChange,
	 * but which updates the Select tool's gizmo in a way that preserves the rotational component
	 * (which would be lost if we just updated the gizmo from the current selection on undo/redo).
	 * 
	 * There is some built-in change tracking for the gizmo component in our transform gizmo, but 
	 * due to the order in which changes get emitted, there is not a good way to make sure that we
	 * update the selection mechanic (which needs to know the gizmo transform) at the correct time
	 * relative to those built-in changes. So, those built-in changes are actually wasted on us,
	 * but it was not easy to deactivate them because the change emitter is linked to the transform
	 * proxy...
	 *
	 * Expects UUVSelectToolChangeRouter to be the passed-in object
	 */
	class  FGizmoMeshChange : public FToolCommandChange
	{
	public:
		FGizmoMeshChange(UUVEditorToolMeshInput* UVToolInputObjectIn,
			TUniquePtr<UE::Geometry::FDynamicMeshChange> UnwrapCanonicalMeshChangeIn,
			const FTransform& GizmoBeforeIn, const FTransform& GizmoAfterIn)

			: UVToolInputObject(UVToolInputObjectIn)
			, UnwrapCanonicalMeshChange(MoveTemp(UnwrapCanonicalMeshChangeIn))
			, GizmoBefore(GizmoBeforeIn)
			, GizmoAfter(GizmoAfterIn)
		{
			ensure(UVToolInputObjectIn);
			ensure(UnwrapCanonicalMeshChange);
		};

		virtual void Apply(UObject* Object) override
		{
			UnwrapCanonicalMeshChange->Apply(UVToolInputObject->UnwrapCanonical.Get(), false);
			UVToolInputObject->UpdateFromCanonicalUnwrapUsingMeshChange(*UnwrapCanonicalMeshChange);

			UInteractiveToolManager* ToolManager = Cast<UInteractiveToolManager>(Object);
			UUVSelectTool* SelectTool = Cast<UUVSelectTool>(ToolManager->GetActiveTool(EToolSide::Mouse));
			if (SelectTool)
			{
				SelectTool->SetGizmoTransform(GizmoAfter);
			}

		}

		virtual void Revert(UObject* Object) override
		{
			UnwrapCanonicalMeshChange->Apply(UVToolInputObject->UnwrapCanonical.Get(), true);
			UVToolInputObject->UpdateFromCanonicalUnwrapUsingMeshChange(*UnwrapCanonicalMeshChange);

			UInteractiveToolManager* ToolManager = Cast<UInteractiveToolManager>(Object);
			UUVSelectTool* SelectTool = Cast<UUVSelectTool>(ToolManager->GetActiveTool(EToolSide::Mouse));
			if (SelectTool)
			{
				SelectTool->SetGizmoTransform(GizmoBefore);
			}
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			return !(UVToolInputObject.IsValid() && UVToolInputObject->IsValid() && UnwrapCanonicalMeshChange);
		}


		virtual FString ToString() const override
		{
			return TEXT("UVSelectToolLocals::FGizmoMeshChange");
		}

	protected:
		TWeakObjectPtr<UUVEditorToolMeshInput> UVToolInputObject;
		TUniquePtr<UE::Geometry::FDynamicMeshChange> UnwrapCanonicalMeshChange;
		FTransform GizmoBefore;
		FTransform GizmoAfter;
	};

}


/*
 * ToolBuilder
 */

bool UUVSelectToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return Targets && Targets->Num() > 0;
}

UInteractiveTool* UUVSelectToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UUVSelectTool* NewTool = NewObject<UUVSelectTool>(SceneState.ToolManager);
	NewTool->SetTargets(*Targets);

	return NewTool;
}

void UUVSelectTool::Setup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVSelectTool_Setup);

	using namespace UVSelectToolLocals;

	check(Targets.Num() > 0);

	UInteractiveTool::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "UV Select Tool"));

	// Get all the API's we'll need
	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();
	EmitChangeAPI = ContextStore->FindContext<UUVToolEmitChangeAPI>();
	ViewportButtonsAPI = ContextStore->FindContext<UUVToolViewportButtonsAPI>();
	SelectionAPI = ContextStore->FindContext<UUVToolSelectionAPI>();

	ViewportButtonsAPI->SetGizmoButtonsEnabled(true);
	ViewportButtonsAPI->OnGizmoModeChange.AddWeakLambda(this, 
		[this](UUVToolViewportButtonsAPI::EGizmoMode NewGizmoMode) {
			UpdateGizmo();
		});

	SelectionAPI->OnSelectionChanged.AddUObject(this, &UUVSelectTool::OnSelectionChanged);
	SelectionAPI->OnPreSelectionChange.AddWeakLambda(this, 
		[this](bool bEmitChange, uint32 SelectionChangeType) {
			if (bEmitChange)
			{
				EmitChangeAPI->EmitToolIndependentChange(GetToolManager(),
					MakeUnique<FGizmoChange>(TransformGizmo->GetGizmoTransform()),
					GizmoChangeTransactionName);
			}
		});

	// Make sure that if undo/redo events act on the meshes, we update our state.
	// The trees will be updated by the tree store, which listens to the same broadcasts.
	// This also handles the case of UV editor one-off actions, which don't actually shut down this tool when they run.
	for (int32 i = 0; i < Targets.Num(); ++i)
	{
		Targets[i]->OnCanonicalModified.AddWeakLambda(this, [this]
			(UUVEditorToolMeshInput* InputObject, const UUVEditorToolMeshInput::FCanonicalModifiedInfo& Info) 
		{
			if (bUpdateGizmoOnCanonicalChange) // Used to avoid reacting to broadcasts that we ourselves caused via gizmo movement
			{
				// We update the gizmo with 'true' here to have the selection api recompute the
				// centroid because it may not yet be notified that the mesh has changed.
				ReinitializeFromSelection(true);
				SelectionAPI->RebuildUnwrapHighlight(TransformGizmo->GetGizmoTransform());
			}
		});
	}

	// Gizmo setup
	UInteractiveGizmoManager* GizmoManager = GetToolManager()->GetPairedGizmoManager();
	UTransformProxy* TransformProxy = NewObject<UTransformProxy>(this);
	TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager, 
		ETransformGizmoSubElements::TranslateAxisX | ETransformGizmoSubElements::TranslateAxisY | ETransformGizmoSubElements::TranslatePlaneXY
		| ETransformGizmoSubElements::ScaleAxisX | ETransformGizmoSubElements::ScaleAxisY | ETransformGizmoSubElements::ScalePlaneXY
		| ETransformGizmoSubElements::RotateAxisZ,
		this);
	TransformProxy->OnBeginTransformEdit.AddUObject(this, &UUVSelectTool::GizmoTransformStarted);
	TransformProxy->OnTransformChanged.AddUObject(this, &UUVSelectTool::GizmoTransformChanged);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &UUVSelectTool::GizmoTransformEnded);

	// TODO: Perhaps ideally we would be able to use these built-in options for gizmo, but they would require us to
	// use our own IToolsContextQueriesAPI implementation that does not query GEditor for the snapping settings, since
	// the UV editor needs to have its own snap settings. That might be the proper thing to do in the long term. 
	// Currently, however, the UV editor manages and queries the snap settings "by hand" through the viewport buttons
	// api, and letting the gizmo query the editor will give us the wrong value.
	TransformGizmo->bSnapToWorldGrid = false;
	TransformGizmo->bSnapToWorldRotGrid = false;
	TransformGizmo->bUseContextCoordinateSystem = false;
	TransformGizmo->bSnapToScaleGrid = false;

	TransformGizmo->SetActiveTarget(TransformProxy, GetToolManager());
	
	// Tell the gizmo to be drawn on top even over translucent-mode materials.
	// Note: this may someday not be necessary, if we get this to work properly by default. Normally we can't
	// use this approach in modeling mode because it adds dithering to the occluded sections, but we are able
	// to disable that in the uv editor viewports.
	for (UActorComponent* Component : TransformGizmo->GetGizmoActor()->GetComponents())
	{
		UGizmoBaseComponent* GizmoComponent = Cast<UGizmoBaseComponent>(Component);
		if (GizmoComponent)
		{
			GizmoComponent->bUseEditorCompositing = true;
		}
	}

	SelectionAPI->SetSelectionMechanicEnabled(true);

	// Turn on auto highlighting for live preview.
	SelectionAPI->SetHighlightVisible(true, true);
	UUVToolSelectionAPI::FHighlightOptions HighlightOptions;
	HighlightOptions.bBaseHighlightOnPreviews = true; // Our translation happens in preview
	HighlightOptions.bAutoUpdateApplied = true;
	HighlightOptions.bAutoUpdateUnwrap = true;
	HighlightOptions.bUseCentroidForUnwrapAutoUpdate = true;
	HighlightOptions.bShowPairedEdgeHighlights = true;
	SelectionAPI->SetHighlightOptions(HighlightOptions);

	if (SelectionAPI->HaveSelections())
	{
		// This will also update the gizmo
		OnSelectionChanged(false, (uint32)ESelectionChangeTypeFlag::SelectionChanged);
		SelectionAPI->RebuildUnwrapHighlight(TransformGizmo->GetGizmoTransform());
		SelectionAPI->RebuildAppliedPreviewHighlight();
	}
	else
	{
		// Make sure gizmo is hidden
		UpdateGizmo();
	}

	GetToolManager()->DisplayMessage(LOCTEXT("SelectToolStatusBarMessage", 
		"Select elements in the viewport. Activate transform gizmo in viewport to transform them."), 
		EToolMessageLevel::UserNotification);
}

void UUVSelectTool::Shutdown(EToolShutdownType ShutdownType)
{
	using namespace UVSelectToolLocals;

	TRACE_CPUPROFILER_EVENT_SCOPE(UVSelectTool_Shutdown);
	
	// If the gizmo has a rotated transform, emit an undo transaction so we can get it back if we
	// undo back into this tool.
	if (SelectionAPI->HaveSelections() && !TransformGizmo->GetGizmoTransform().RotationEquals(FTransform::Identity))
	{
		EmitChangeAPI->EmitToolIndependentChange(GetToolManager(),
			MakeUnique<FGizmoChange>(TransformGizmo->GetGizmoTransform()),
			GizmoChangeTransactionName);
	}

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->OnCanonicalModified.RemoveAll(this);
	}

	// Don't actually need to do this since selection api should reset after us, but still.
	SelectionAPI->SetSelectionMechanicEnabled(false);
	SelectionAPI->SetHighlightVisible(false, false);

	// Calls shutdown on gizmo and destroys it.
	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);

	ViewportButtonsAPI->OnGizmoModeChange.RemoveAll(this);
	ViewportButtonsAPI->SetGizmoButtonsEnabled(false);

	ViewportButtonsAPI = nullptr;
	EmitChangeAPI = nullptr;
	SelectionAPI = nullptr;
}

FTransform UUVSelectTool::GetGizmoTransform() const
{
	return TransformGizmo->GetGizmoTransform();
}

void UUVSelectTool::SetGizmoTransform(const FTransform& NewTransform)
{
	TransformGizmo->ReinitializeGizmoTransform(NewTransform);
	SelectionAPI->RebuildUnwrapHighlight(NewTransform);
}

void UUVSelectTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
}

void UUVSelectTool::UpdateGizmo(bool bForceRecomputeSelectionCenters)
{
	if (SelectionAPI->HaveSelections())
	{
		TransformGizmo->ReinitializeGizmoTransform(FTransform(SelectionAPI->GetUnwrapSelectionBoundingBoxCenter(bForceRecomputeSelectionCenters)));
	}

	TransformGizmo->SetVisibility(
		ViewportButtonsAPI->GetGizmoMode() != UUVToolViewportButtonsAPI::EGizmoMode::Select
		&& SelectionAPI->HaveSelections());
}

void UUVSelectTool::OnSelectionChanged(bool, uint32 SelectionChangeType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVSelectTool_OnSelectionChanged);

	using namespace UVSelectToolLocals;

	if (!(SelectionChangeType & (uint32)ESelectionChangeTypeFlag::SelectionChanged))
	{
		return;
	}

	ReinitializeFromSelection(false);
}

void UUVSelectTool::ReinitializeFromSelection(bool bForceRecomputeCachedSelectionCentroid)
{
	MovingVidsPerSelection.Reset();
	MovingVertOriginalPositionsPerSelection.Reset();
	RenderUpdateTidsPerSelection.Reset();

	CurrentSelections = SelectionAPI->GetSelections();

	// Do a check for validity of selection so that we are more robust to tools/actions
	// improperly setting selection.
	for (const FUVToolSelection& Selection : CurrentSelections)
	{
		if (!ensure(Selection.Target.IsValid()
			&& Selection.AreElementsPresentInMesh(*Selection.Target->UnwrapCanonical)))
		{
			// One way we could end up here is if an action that changes the topology doesn't
			// properly update the selection. While the mode may clear it for us, undoing that
			// change will still end up with a selection that is incompatible with the current
			// topology.
			SelectionAPI->ClearSelections(false, false);
			break;
		}
	}

	for (int32 SelectionIndex = 0; SelectionIndex < CurrentSelections.Num(); ++SelectionIndex)
	{
		const FUVToolSelection& Selection = CurrentSelections[SelectionIndex];
		const UUVEditorToolMeshInput* Target = Selection.Target.Get();
		const FDynamicMesh3* UnwrapMesh = Target->UnwrapCanonical.Get();

		// Note the selected vertex IDs and triangle IDs
		TArray<int32> VidSet;
		TArray<int32> TidSet;
		if (Selection.Type == FUVToolSelection::EType::Triangle)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Triangle);

			VidSet.SetNumUninitialized(Selection.SelectedIDs.Num() * 3);
			TidSet.SetNumUninitialized(Selection.SelectedIDs.Num());

			int32 TIndex = 0;
			for (const int32 Tid : Selection.SelectedIDs)
			{
				const FIndex3i TriVids = UnwrapMesh->GetTriangle(Tid);
				VidSet[TIndex * 3 + 0] = TriVids[0];
				VidSet[TIndex * 3 + 1] = TriVids[1];
				VidSet[TIndex * 3 + 2] = TriVids[2];
				TidSet[TIndex++] = Tid;
			}
		}
		else if (Selection.Type == FUVToolSelection::EType::Edge)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Edge);

			VidSet.SetNumUninitialized(Selection.SelectedIDs.Num() * 2);
			TidSet.Reserve(Selection.SelectedIDs.Num());

			int32 VIndex = 0;
			TArray<int> Tids;
			for (const int32 Eid : Selection.SelectedIDs)
			{
				const FIndex2i EdgeVids = UnwrapMesh->GetEdgeV(Eid);
				VidSet[VIndex++] = EdgeVids[0];
				VidSet[VIndex++] = EdgeVids[1];

				Tids.Reset();
				UnwrapMesh->GetVtxTriangles(EdgeVids[0], Tids);
				UnwrapMesh->GetVtxTriangles(EdgeVids[1], Tids);
				TidSet.Append(Tids);
			}
		}
		else if (Selection.Type == FUVToolSelection::EType::Vertex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Vertex);

			VidSet.SetNumUninitialized(Selection.SelectedIDs.Num());
			
			int32 VIndex = 0;
			FDynamicMesh3::FLocalIntArray Tids;
			for (const int32 Vid : Selection.SelectedIDs)
			{
				VidSet[VIndex++] = Vid;

				Tids.Reset();
				UnwrapMesh->GetVtxTriangles(Vid, Tids);
				TidSet.Append(Tids);
			}
		}
		else
		{
			ensure(false);
		}

		// Make entries unique
		VidSet.Sort();
		VidSet.SetNum(Algo::Unique(VidSet));
		TidSet.Sort();
		TidSet.SetNum(Algo::Unique(TidSet));
		
		MovingVidsPerSelection.Emplace(VidSet);
		RenderUpdateTidsPerSelection.Emplace(TidSet);
	}

	UpdateGizmo(bForceRecomputeCachedSelectionCentroid);
}

void UUVSelectTool::GizmoTransformStarted(UTransformProxy* Proxy)
{
	bInDrag = true;

	InitialGizmoFrame = FFrame3d(TransformGizmo->ActiveTarget->GetTransform());

	MovingVertOriginalPositionsPerSelection.Reset();

	for (int32 SelectionIndex = 0; SelectionIndex < CurrentSelections.Num(); ++SelectionIndex)
	{
		FDynamicMesh3* Mesh = CurrentSelections[SelectionIndex].Target->UnwrapCanonical.Get();
		const TArray<int32>& MovingVids = MovingVidsPerSelection[SelectionIndex];

		MovingVertOriginalPositionsPerSelection.Emplace();
		TArray<FVector3d>& MovingVertOriginalPositions = MovingVertOriginalPositionsPerSelection.Last();
		MovingVertOriginalPositions.SetNum(MovingVids.Num());

		// Note: Our meshes currently don't have a transform. Otherwise we'd need to convert vid location to world
		// space first, then to the frame.
		for (int32 i = 0; i < MovingVids.Num(); ++i)
		{
			MovingVertOriginalPositions[i] = InitialGizmoFrame.ToFramePoint(Mesh->GetVertex(MovingVids[i]));
		}
	}
}

void UUVSelectTool::GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	// This function gets called both during drag and on undo/redo. This might have been ok if
	// undo/redo also called GizmoTransformStarted/GizmoTransformEnded, but they don't, which
	// means the two types of events operate quite differently. We just ignore any non-drag calls.
	if (!bInDrag)
	{
		return;
	}

	FTransform DeltaTransform = Transform.GetRelativeTransform(InitialGizmoFrame.ToFTransform());

	if (!DeltaTransform.GetTranslation().IsNearlyZero() || !DeltaTransform.GetRotation().IsIdentity() || Transform.GetScale3D() != FVector::One())
	{
	
		FTransform SnappedDeltaTransform = DeltaTransform;		
		bool bSnapExecuted = false;

		if (!DeltaTransform.GetTranslation().IsNearlyZero() && 
			ViewportButtonsAPI->GetSnapEnabled(UUVToolViewportButtonsAPI::ESnapTypeFlag::Location))
		{
			float SnapValue = ViewportButtonsAPI->GetSnapValue(UUVToolViewportButtonsAPI::ESnapTypeFlag::Location);
			FVector Translation = DeltaTransform.GetTranslation();
			Translation = Translation.GridSnap(SnapValue * FUVEditorUXSettings::UVMeshScalingFactor);
			Translation = Transform.GetRotation() * Translation; // Apply the current rotated frame, so we translate in local coordinates.
			SnappedDeltaTransform.SetTranslation(Translation);
			bSnapExecuted = true;
		}

		if (!DeltaTransform.GetRotation().IsIdentity() &&
			ViewportButtonsAPI->GetSnapEnabled(UUVToolViewportButtonsAPI::ESnapTypeFlag::Rotation))
		{
			float SnapValue = ViewportButtonsAPI->GetSnapValue(UUVToolViewportButtonsAPI::ESnapTypeFlag::Rotation);
			FQuat DeltaRotation = DeltaTransform.GetRotation();
			FVector DeltaPosition = InitialGizmoFrame.Origin;

			FRotator Rotator(DeltaTransform.GetRotation());
			FRotator RotGrid(SnapValue, SnapValue, SnapValue);
			Rotator = Rotator.GridSnap(RotGrid);
			DeltaPosition = ( Rotator.Quaternion() * -DeltaPosition ) + DeltaPosition;

			SnappedDeltaTransform.SetRotation(Rotator.Quaternion());
			SnappedDeltaTransform.SetLocation(DeltaPosition);
			bSnapExecuted = true;
		}

		if (Transform.GetScale3D() != FVector::One() &&
			ViewportButtonsAPI->GetSnapEnabled(UUVToolViewportButtonsAPI::ESnapTypeFlag::Scale))
		{
			float SnapValue = ViewportButtonsAPI->GetSnapValue(UUVToolViewportButtonsAPI::ESnapTypeFlag::Scale);
			FVector DeltaScale = DeltaTransform.GetScale3D();
			FVector DeltaPosition = InitialGizmoFrame.Origin;			
			DeltaScale = DeltaScale.GridSnap(SnapValue);
			DeltaPosition = (DeltaScale * -DeltaPosition) + DeltaPosition;

			SnappedDeltaTransform.SetScale3D(DeltaScale);
			SnappedDeltaTransform.SetLocation(DeltaPosition);
			bSnapExecuted = true;
		}


		FTransform SnappedTransform = InitialGizmoFrame.ToFTransform() * SnappedDeltaTransform;
		
		if (bSnapExecuted)
		{
			UnappliedGizmoTransform = SnappedTransform;
			TransformGizmo->ReinitializeGizmoTransform(SnappedTransform);
		}
		else
		{
			UnappliedGizmoTransform = Transform;
		}
		bGizmoTransformNeedsApplication = true;		
	}	
}

void UUVSelectTool::GizmoTransformEnded(UTransformProxy* Proxy)
{
	bInDrag = false;

	const FText TransactionName(LOCTEXT("DragCompleteTransactionName", "Move Items"));
	EmitChangeAPI->BeginUndoTransaction(TransactionName);

	for (int32 SelectionIndex = 0; SelectionIndex < CurrentSelections.Num(); ++SelectionIndex)
	{
		UUVEditorToolMeshInput* Target = CurrentSelections[SelectionIndex].Target.Get();
		const TArray<int32>& MovingVids = MovingVidsPerSelection[SelectionIndex];

		// Set things up for undo.
		// TODO: We should really use FMeshVertexChange instead of FDynamicMeshChange because we don't
		// need to alter the mesh topology. However we currently don't have a way to apply a FMeshVertexChange
		// directly to a dynamic mesh pointer, only via UDynamicMesh. We should change things here once
		// that ability exists.
		FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
		ChangeTracker.BeginChange();
		ChangeTracker.SaveTriangles(RenderUpdateTidsPerSelection[SelectionIndex], true);

		// One final attempt to apply transforms if OnTick hasn't happened yet
		if (bGizmoTransformNeedsApplication)
		{
			ApplyGizmoTransform();
		}

		// Both previews must already be updated, so only need to update canonical. 
		{
			// We don't want to fully react to the ensuing broadcast so that we don't lose the gizmo rotation,
			// but do want to broadcast so that other things like the trees update themselves.
			TGuardValue<bool> GizmoUpdateGuard(bUpdateGizmoOnCanonicalChange, false); // sets to false, restores at end

			Target->UpdateCanonicalFromPreviews(&MovingVids, UUVEditorToolMeshInput::NONE_CHANGED_ARG);
		}

		EmitChangeAPI->EmitToolIndependentChange(GetToolManager(), MakeUnique<UVSelectToolLocals::FGizmoMeshChange>(
			Target, ChangeTracker.EndChange(),
			InitialGizmoFrame.ToFTransform(), TransformGizmo->GetGizmoTransform()),
			TransactionName);

		TransformGizmo->SetNewChildScale(FVector::One());

		SelectionAPI->RebuildUnwrapHighlight(TransformGizmo->GetGizmoTransform());
	}
	EmitChangeAPI->EndUndoTransaction();
}

/**
 * Updates previews based off of the current gizmo transform, CurrentSelections, MovingVidsPerSelection, 
 * and MovingVertOriginalPositions. Gets emitted while dragging the gizmo on tick.
 */
void UUVSelectTool::ApplyGizmoTransform()
{
	FTransformSRT3d TransformToApply(UnappliedGizmoTransform);

	for (int32 SelectionIndex = 0; SelectionIndex < CurrentSelections.Num(); ++SelectionIndex)
	{
		UUVEditorToolMeshInput* Target = CurrentSelections[SelectionIndex].Target.Get();
		const TArray<int32>& MovingVids = MovingVidsPerSelection[SelectionIndex];
		const TArray<FVector3d>& MovingVertOriginalPositions = MovingVertOriginalPositionsPerSelection[SelectionIndex];

		Target->UnwrapPreview->PreviewMesh->DeferredEditMesh([&TransformToApply, &MovingVids, &MovingVertOriginalPositions](FDynamicMesh3& MeshIn)
		{
			for (int32 i = 0; i < MovingVids.Num(); ++i)
			{
				MeshIn.SetVertex(MovingVids[i], TransformToApply.TransformPosition(MovingVertOriginalPositions[i]));
			}
		}, false);

		Target->UpdateUnwrapPreviewOverlayFromPositions(&MovingVids, UUVEditorToolMeshInput::NONE_CHANGED_ARG, 
			&RenderUpdateTidsPerSelection[SelectionIndex]);
		Target->UpdateAppliedPreviewFromUnwrapPreview(&MovingVids, UUVEditorToolMeshInput::NONE_CHANGED_ARG, 
			&RenderUpdateTidsPerSelection[SelectionIndex]);
	}

	SelectionAPI->SetUnwrapHighlightTransform((FTransform)TransformToApply);
	bGizmoTransformNeedsApplication = false;
}

void UUVSelectTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

void UUVSelectTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
}

void UUVSelectTool::OnTick(float DeltaTime)
{
	if (bGizmoTransformNeedsApplication)
	{
		ApplyGizmoTransform();
	}
}

bool UUVSelectTool::CanCurrentlyNestedCancel()
{
	return CurrentSelections.Num() > 0;
}

bool UUVSelectTool::ExecuteNestedCancelCommand()
{
	if (CurrentSelections.Num() > 0)
	{
		SelectionAPI->BeginChange();
		SelectionAPI->ClearSelections(false, false);
		SelectionAPI->ClearUnsetElementAppliedMeshSelections(false, false);
		SelectionAPI->EndChangeAndEmitIfModified(true);

		return true;
	}

	return false;
}

void UUVSelectTool::SelectAll()
{
	UUVToolSelectionAPI::EUVEditorSelectionMode CurrentSelectionMode = ViewportButtonsAPI->GetSelectionMode();
	FUVToolSelection::EType SelectionType = FUVToolSelection::EType::Triangle;
	if (CurrentSelectionMode == UUVToolSelectionAPI::EUVEditorSelectionMode::None)
	{
		return; // If we're in none selection mode, don't do anything for select all behavior
	}

	switch (CurrentSelectionMode)
	{
	case UUVToolSelectionAPI::EUVEditorSelectionMode::Vertex:
		SelectionType = FUVToolSelection::EType::Vertex;
		break;
	case UUVToolSelectionAPI::EUVEditorSelectionMode::Edge:
		SelectionType = FUVToolSelection::EType::Edge;
		break;
	case UUVToolSelectionAPI::EUVEditorSelectionMode::Triangle:
	case UUVToolSelectionAPI::EUVEditorSelectionMode::Island:
	case UUVToolSelectionAPI::EUVEditorSelectionMode::Mesh:
		SelectionType = FUVToolSelection::EType::Triangle;
		break;
	default:
		ensure(false);
	}

	SelectionAPI->BeginChange();
	SelectionAPI->ClearSelections(false, false);
	SelectionAPI->ClearUnsetElementAppliedMeshSelections(false, false);

	TArray<FUVToolSelection> AllSelections;
	AllSelections.SetNum(Targets.Num());
	for (int32 AssetID = 0; AssetID < Targets.Num(); ++AssetID)
	{
		AllSelections[AssetID].Target = Targets[AssetID];
		AllSelections[AssetID].SelectAll(*Targets[AssetID]->UnwrapCanonical, SelectionType);
	}
	SelectionAPI->SetSelections(AllSelections, false, false);

	SelectionAPI->EndChangeAndEmitIfModified(true);
}

#undef LOCTEXT_NAMESPACE

