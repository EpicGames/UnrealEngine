// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Actions/UVToolAction.h"

#include "UVMakeIslandAction.generated.h"

#define UE_API UVEDITORTOOLS_API

/**
 * Action that takes the currently selected triangles and makes a separate UV island out of them,
 *  i.e. any interior seams are removed, and seams are added around the boundary of the selection.
 *  If the selection is not connected in the mesh, islands will be created for each connected
 *  component of selected triangles. If some of the selected triangles have unset UVs, they will
 *  be initialized to 0 for the purposes to creating an island. The output is the triangles of
 *  the created islands.
 * An unwrap operation on the island is typically a good thing to follow this action.
 */
UCLASS(MinimalAPI)
class UUVMakeIslandAction : public UUVToolAction
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanExecuteAction() const override;
	UE_API virtual bool ExecuteAction() override;
};

#undef UE_API
