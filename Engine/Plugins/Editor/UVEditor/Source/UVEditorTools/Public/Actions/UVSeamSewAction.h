// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Actions/UVToolAction.h"
#include "GeometryBase.h"
#include "IndexTypes.h"

#include "Actions/UVToolAction.h"

#include "UVSeamSewAction.generated.h"

#define UE_API UVEDITORTOOLS_API

PREDECLARE_GEOMETRY(class FUVEditorDynamicMeshSelection);
PREDECLARE_GEOMETRY(class FDynamicMesh3);
class APreviewGeometryActor;
class ULineSetComponent;

UCLASS(MinimalAPI)
class UUVSeamSewAction : public UUVToolAction
{	
	GENERATED_BODY()

	using FDynamicMesh3 = UE::Geometry::FDynamicMesh3;

public:
	UE_API virtual bool CanExecuteAction() const override;
	UE_API virtual bool ExecuteAction() override;

	static UE_API int32 FindSewEdgeOppositePairing(const FDynamicMesh3& UnwrapMesh, 
		const FDynamicMesh3& AppliedMesh, int32 UVLayerIndex, int32 UnwrapEid, 
		bool& bWouldPreferOppositeOrderOut);
};

#undef UE_API
