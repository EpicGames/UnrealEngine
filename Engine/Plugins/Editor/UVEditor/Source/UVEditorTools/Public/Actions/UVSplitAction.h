// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Actions/UVToolAction.h"
#include "CoreMinimal.h"

#include "UVSplitAction.generated.h"

#define UE_API UVEDITORTOOLS_API

UCLASS(MinimalAPI)
class UUVSplitAction : public UUVToolAction
{	
	GENERATED_BODY()

public:
	UE_API virtual bool CanExecuteAction() const override;
	UE_API virtual bool ExecuteAction() override;
};

#undef UE_API
