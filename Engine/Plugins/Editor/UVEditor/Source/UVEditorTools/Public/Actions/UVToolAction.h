// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UVToolAction.generated.h"

#define UE_API UVEDITORTOOLS_API

class UUVToolEmitChangeAPI;
class UUVToolSelectionAPI;
class UInteractiveToolManager;

UCLASS(MinimalAPI)
class UUVToolAction : public UObject
{
	GENERATED_BODY()
public:

	UE_API virtual void Setup(UInteractiveToolManager* ToolManager);
	UE_API virtual void Shutdown();
	virtual bool CanExecuteAction() const { return false;}
	virtual bool ExecuteAction() { return false; }

protected:
	UPROPERTY()
	TObjectPtr<UUVToolSelectionAPI> SelectionAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolEmitChangeAPI> EmitChangeAPI = nullptr;

	TWeakObjectPtr<UInteractiveToolManager> ToolManager;

	virtual UInteractiveToolManager* GetToolManager() const { return ToolManager.Get(); }
};

#undef UE_API
