// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ContextObjects/UVToolContextObjects.h"

#include "UVToolAssetInputsContext.generated.h"

/**
 * Simple context object used by UV asset editor to pass inputs down to the UV editor mode.
 */
UCLASS(MinimalAPI)
class UUVToolAssetInputsContext : public UUVToolContextObject
{
	GENERATED_BODY()
public:

	void Initialize(TArray<TObjectPtr<UObject>> AssetsIn, TArray<FTransform> TransformsIn)
	{
		Assets = MoveTemp(AssetsIn);
		Transforms = MoveTemp(TransformsIn);
		ensure(Transforms.Num() == Assets.Num());
	}

	void GetAssetInputs(TArray<TObjectPtr<UObject>>& AssetsOut, TArray<FTransform>& TransformsOut) const
	{
		AssetsOut = Assets;
		TransformsOut = Transforms;
	}

protected:
	TArray<TObjectPtr<UObject>> Assets;
	TArray<FTransform> Transforms;
};
