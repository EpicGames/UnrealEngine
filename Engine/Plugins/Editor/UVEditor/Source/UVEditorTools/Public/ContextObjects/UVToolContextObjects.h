// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h" // FDynamicMeshChange for TUniquePtr
#include "InteractiveToolManager.h"
#include "GeometryBase.h"

#include "UVToolContextObjects.generated.h"

#define UE_API UVEDITORTOOLS_API

// TODO: This should be spread out across multiple files

PREDECLARE_GEOMETRY(class FDynamicMesh3);
class FAssetEditorModeManager;
class FEditorViewportClient;
class FToolCommandChange;
struct FViewCameraState;
class UInputRouter;
class UWorld;
class UUVEditorToolMeshInput;

/**
 * Base class for context objects used in the UV editor.
 */
UCLASS(MinimalAPI)
class UUVToolContextObject : public UObject
{
	GENERATED_BODY()
public:
	/**
	 * Called by the mode when shutting context objects down, allowing them to do any cleanup.
	 * Initialization, on the other hand is usually done by some class-specific Initialize() method.
	 */
	virtual void Shutdown() {}

	/**
	 * Called whenever a tool is ended, for instance to let a context object remove listeners associated
	 * with that tool (it shouldn't have to do so, but may choose to for robustness).
	 */
	virtual void OnToolEnded(UInteractiveTool* DeadTool) {}
};

/**
 * An API object meant to be stored in a context object store that allows UV editor tools
 * to emit appropriate undo/redo transactions.
 */
UCLASS(MinimalAPI)
class UUVToolEmitChangeAPI : public UUVToolContextObject
{
	GENERATED_BODY()
public:

	void Initialize(TObjectPtr<UInteractiveToolManager> ToolManagerIn)
	{
		ToolManager = ToolManagerIn;
	}

	virtual void BeginUndoTransaction(const FText& Description)
	{
		ToolManager->BeginUndoTransaction(Description);
	}
	virtual void EndUndoTransaction()
	{
		ToolManager->EndUndoTransaction();
	}

	/**
	 * Emit a change that can be undone even if we leave the tool from which it is emitted (as
	 * long as that UV editor instance is still open). 
	 * Minor note: because we undo "out of" tools into a default tool and never out of a default tool,
	 * in practice, tool-independent changes will only ever be applied/reverted in the same tool 
	 * invocation that they were emitted or in the default tool, not in other arbitrary tools.
	 * 
	 * Since tool-independent changes usually operate on UV editor mesh input object, it is probably
	 * preferable to use EmitToolIndependentUnwrapCanonicalChange, which will set up a proper transaction
	 * for you.
	 */
	UE_API virtual void EmitToolIndependentChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description);

	/**
	 * A convenience function that is like EmitToolIndependentChange, but uses a FDynamicMeshChange
	 * that operates on the UnwrapCanonical of an input to create a change object that updates the other
	 * views and issues an OnUndoRedo broadcast on the input object.
	 */
	UE_API virtual void EmitToolIndependentUnwrapCanonicalChange(UUVEditorToolMeshInput* InputObject,
		TUniquePtr<UE::Geometry::FDynamicMeshChange> UnwrapCanonicalMeshChange, const FText& Description);

	/**
	 * Emits a change that is considered expired when the active tool does not match the tool that was active
	 * when it was emitted.
	 */
	UE_API virtual void EmitToolDependentChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description);

protected:
	TWeakObjectPtr<UInteractiveToolManager> ToolManager = nullptr;
};

/**
 * Allows tools to interact with the 3d preview viewport, which has a separate
 * world and input router.
 */
UCLASS(MinimalAPI)
class UUVToolLivePreviewAPI : public UUVToolContextObject
{
	GENERATED_BODY()
public:

	UE_API void Initialize(UWorld* WorldIn, UInputRouter* RouterIn,
		TUniqueFunction<void(FViewCameraState& CameraStateOut)> GetLivePreviewCameraStateFuncIn,
		TUniqueFunction<void(const UE::Geometry::FAxisAlignedBox3d& BoundingBox)> SetLivePreviewCameraToLookAtVolumeFuncIn,
		TUniqueFunction<void(const EMouseCursor::Type Cursor, bool bEnableOverride)> SetCursorOverrideFuncIn, 
		UInteractiveGizmoManager* GizmoManager = nullptr);

	UWorld* GetLivePreviewWorld() { return World.Get(); }
	UInputRouter* GetLivePreviewInputRouter() { return InputRouter.Get(); }
	UInteractiveGizmoManager* GetGizmoManager() { return GizmoManager.Get(); }
	void GetLivePreviewCameraState(FViewCameraState& CameraStateOut) 
	{ 
		if (GetLivePreviewCameraStateFunc)
		{
			GetLivePreviewCameraStateFunc(CameraStateOut);
		}
	}

	void SetLivePreviewCameraToLookAtVolume(const UE::Geometry::FAxisAlignedBox3d& BoundingBox)
	{
		if (SetLivePreviewCameraToLookAtVolumeFunc)
		{
			SetLivePreviewCameraToLookAtVolumeFunc(BoundingBox);
		}
	}

	void SetCursorOverride(const EMouseCursor::Type Cursor)
	{
		if (SetCursorOverrideFunc)
		{
			SetCursorOverrideFunc(Cursor, true);
		}
	}

	void ClearCursorOverride()
	{
		if (SetCursorOverrideFunc)
		{
			SetCursorOverrideFunc(EMouseCursor::Default, false);
		}
	}

	UE_API virtual void OnToolEnded(UInteractiveTool* DeadTool) override;

	/**
	 * Broadcast by the 3D live preview viewport on Render() so that mechanics/tools can
	 * render there.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRender, IToolsContextRenderAPI* RenderAPI);
	FOnRender OnRender;

	/**
	 * Broadcast by the 3D live preview viewport on DrawHUD() so that mechanics/tools can
	 * draw there.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDrawHUD, FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);
	FOnDrawHUD OnDrawHUD;

protected:
	UPROPERTY()
	TWeakObjectPtr<UWorld> World;

	UPROPERTY()
	TWeakObjectPtr<UInputRouter> InputRouter;

	TUniqueFunction<void(FViewCameraState& CameraStateOut)> GetLivePreviewCameraStateFunc;
	TUniqueFunction<void(const UE::Geometry::FAxisAlignedBox3d& BoundingBox)> SetLivePreviewCameraToLookAtVolumeFunc;
	TUniqueFunction<void(const EMouseCursor::Type Cursor, bool bEnableOverride)> SetCursorOverrideFunc;

private:
	TWeakObjectPtr<UInteractiveGizmoManager> GizmoManager;
};

USTRUCT()
struct FUDIMBlock
{
	GENERATED_BODY();

	UPROPERTY()
	int32 UDIM = 1001;

	UPROPERTY()
	int32 TextureResolution = 512;

	UE_API int32 BlockU() const;
	UE_API int32 BlockV() const;
	UE_API void SetFromBlocks(int32 BlockU, int32 BlockV);
};

/**
 * Allows tools to interact with the 2d preview viewport 
 */
UCLASS(MinimalAPI)
class UUVTool2DViewportAPI : public UUVToolContextObject
{
	GENERATED_BODY()
public:

	void SetUDIMBlocks(TArray<FUDIMBlock>& BlocksIn, bool bBroadcast = true)
	{
		UDIMBlocks = BlocksIn;
		if (bBroadcast)
		{
			OnUDIMBlockChange.Broadcast(UDIMBlocks);
		}
	}

	const TArray<FUDIMBlock>& GetUDIMBlocks() const
	{
		return UDIMBlocks;
	}

	void SetDrawGrid(bool bDrawGridIn, bool bBroadcast = true)
	{
		bDrawGrid = bDrawGridIn;
		if (bBroadcast)
		{
			OnDrawGridChange.Broadcast(bDrawGrid);
		}
	}

	const bool GetDrawGrid() const
    {
		return bDrawGrid;
	}

	void SetDrawRulers(bool bDrawRulersIn, bool bBroadcast = true)
	{
		bDrawRulers = bDrawRulersIn;
		if (bBroadcast)
		{
			OnDrawRulersChange.Broadcast(bDrawRulers);
		}
	}

	const bool GetDrawRulers() const
    {
		return bDrawRulers;
	}


	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUDIMBlockChange, const TArray<FUDIMBlock>& UDIMBlocks);
	FOnUDIMBlockChange OnUDIMBlockChange;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDrawGridChange, bool bDrawGrid);
	FOnDrawGridChange OnDrawGridChange;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDrawRulersChange, bool bDrawRulers);
	FOnDrawRulersChange OnDrawRulersChange;

protected:

	UPROPERTY(Transient)
	TArray<FUDIMBlock> UDIMBlocks;

	UPROPERTY(Transient)
	bool bDrawGrid;

	UPROPERTY(Transient)
	bool bDrawRulers;
};


/**
 * Allows tools to interact with the assets and their UV layers
*/
UCLASS(MinimalAPI)
class UUVToolAssetAndChannelAPI : public UUVToolContextObject
{
	GENERATED_BODY()
public:

	TArray<int32> GetCurrentChannelVisibility()
	{
		if (GetCurrentChannelVisibilityFunc)
		{
			return GetCurrentChannelVisibilityFunc();
		}
		return TArray<int32>();
	}

	void RequestChannelVisibilityChange(const TArray<int32>& ChannelPerAsset, bool bEmitUndoTransaction=true)
	{
		if (RequestChannelVisibilityChangeFunc)
		{
			RequestChannelVisibilityChangeFunc(ChannelPerAsset, bEmitUndoTransaction);
		}
	}

	void NotifyOfAssetChannelCountChange(int32 AssetID)
	{
		if (NotifyOfAssetChannelCountChangeFunc)
		{
			NotifyOfAssetChannelCountChangeFunc(AssetID);
		}
	}


	TUniqueFunction<TArray<int32>()> GetCurrentChannelVisibilityFunc;
	TUniqueFunction<void(const TArray<int32>&, bool)> RequestChannelVisibilityChangeFunc;
	TUniqueFunction<void(int32 AssetID)> NotifyOfAssetChannelCountChangeFunc;

};

/** 
 * Stores AABB trees for UV input object unwrap canonical or applied canonical meshes.
 * Binds to the input objects' OnCanonicalModified delegate to automatically update 
 * the tree when necessary.
 */
UCLASS(MinimalAPI)
class UUVToolAABBTreeStorage : public UUVToolContextObject
{
	GENERATED_BODY()
public:
	using FDynamicMesh3 = UE::Geometry::FDynamicMesh3;
	using FDynamicMeshAABBTree3 = UE::Geometry::FDynamicMeshAABBTree3;


	UE_API void Set(FDynamicMesh3* MeshKey, TSharedPtr<FDynamicMeshAABBTree3> Tree,
		UUVEditorToolMeshInput* InputObject);

	UE_API TSharedPtr<FDynamicMeshAABBTree3> Get(FDynamicMesh3* MeshKey);

	UE_API void Remove(FDynamicMesh3* MeshKey);

	UE_API void RemoveByPredicate(TUniqueFunction<
		bool(const FDynamicMesh3* Mesh, TWeakObjectPtr<UUVEditorToolMeshInput> InputObject,
			TSharedPtr<FDynamicMeshAABBTree3> Tree)> Predicate);

	UE_API void Empty();

	UE_API virtual void Shutdown() override;

protected:

	using TreeInputObjectPair = TPair<TSharedPtr<FDynamicMeshAABBTree3>, TWeakObjectPtr<UUVEditorToolMeshInput>>;

	TMap<FDynamicMesh3*, TreeInputObjectPair> AABBTreeStorage;
};


/**
* Stores pointers to additional property sets for the active tool, used by the UVEditor Mode and Toolkit to 
* populate other settings menus, such as the Display menu.
*/

UCLASS(MinimalAPI)
class UUVEditorToolPropertiesAPI : public UUVToolContextObject
{
	GENERATED_BODY()
public:

	/* Property sets passed here will be displayed in the Display menu at the top of the UV Editor. */
	void SetToolDisplayProperties(UObject* ToolDisplayPropertesIn)
	{
		ToolDisplayProperties = ToolDisplayPropertesIn;
	}

	UObject* GetToolDisplayProperties() {
		if (ToolDisplayProperties)
		{
			return ToolDisplayProperties;
		}
		return nullptr;
	}

	virtual void OnToolEnded(UInteractiveTool* DeadTool) {
		ToolDisplayProperties = nullptr;
	}

protected:

	UPROPERTY()
	TObjectPtr<UObject> ToolDisplayProperties = nullptr;

};

#undef UE_API
