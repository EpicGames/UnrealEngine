// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Selection/UVToolSelectionAPI.h"

#include "UVToolViewportButtonsAPI.generated.h"

/**
 * Allows tools to interact with buttons in the viewport
 */
UCLASS(MinimalAPI)
class UUVToolViewportButtonsAPI : public UUVToolContextObject
{
	GENERATED_BODY()
public:

	enum class EGizmoMode
	{
		Select,
		Transform
	};

	enum class ESnapTypeFlag : uint8
	{
		NoSnap = 0,
		Location = 1 << 0,
		Rotation = 1 << 1,
		Scale = 1 << 2
	};

	using ESelectionMode = UUVToolSelectionAPI::EUVEditorSelectionMode;

	void SetGizmoButtonsEnabled(bool bOn)
	{
		bGizmoButtonsEnabled = bOn;
	}

	bool AreGizmoButtonsEnabled() const
	{
		return bGizmoButtonsEnabled;
	}

	void SetGizmoMode(EGizmoMode ModeIn, bool bBroadcast = true)
	{
		GizmoMode = ModeIn;
		if (bBroadcast)
		{
			OnGizmoModeChange.Broadcast(GizmoMode);
		}
	}

	EGizmoMode GetGizmoMode() const
	{
		return GizmoMode;
	}


	void SetSelectionButtonsEnabled(bool bOn)
	{
		bSelectionButtonsEnabled = bOn;
	}

	bool AreSelectionButtonsEnabled() const
	{
		return bSelectionButtonsEnabled;
	}

	void SetSelectionMode(ESelectionMode ModeIn, bool bBroadcast = true)
	{
		SelectionMode = ModeIn;
		if (bBroadcast)
		{
			OnSelectionModeChange.Broadcast(SelectionMode);
		}
	}

	ESelectionMode GetSelectionMode() const
	{
		return SelectionMode;
	}

	void InitiateFocusCameraOnSelection() const
	{
		OnInitiateFocusCameraOnSelection.Broadcast();
	}

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGizmoModeChange, EGizmoMode NewGizmoMode);
	FOnGizmoModeChange OnGizmoModeChange;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionModeChange, ESelectionMode NewSelectionMode);
	FOnSelectionModeChange OnSelectionModeChange;

	DECLARE_MULTICAST_DELEGATE(FOnInitiateFocusCameraOnSelection);
	FOnInitiateFocusCameraOnSelection OnInitiateFocusCameraOnSelection;

	virtual void OnToolEnded(UInteractiveTool* DeadTool) override 
	{
		OnGizmoModeChange.RemoveAll(DeadTool);
		OnSelectionModeChange.RemoveAll(DeadTool);
	}

	// Sets snapping on/off, rather than the explicit type to enable/disable
	void SetSnappingActive(bool bInEnabled)
	{
		bSnappingActive = bInEnabled;
	}

	bool IsSnappingActive() const
	{
		return bSnappingActive;
	}

	void ToggleSnapEnabled(ESnapTypeFlag SnapMode)
	{
		SnapEnabled ^= (uint8)SnapMode;
	}

	bool GetSnapEnabled(ESnapTypeFlag SnapMode) const
	{
		return SnapEnabled & (uint8)SnapMode;
	}

	// Whether to perform the snapping, not just get it's checkbox state
	bool IsSnapActive(ESnapTypeFlag SnapMode) const
	{
		return bSnappingActive && GetSnapEnabled(SnapMode);
	}

	void SetSnapValue(ESnapTypeFlag SnapMode, float SnapValue)
	{
		switch (SnapMode)
		{
		case ESnapTypeFlag::Location:
			LocationSnap = SnapValue;
			break;
		case ESnapTypeFlag::Rotation:
			RotationSnap = SnapValue;
			break;
		case ESnapTypeFlag::Scale:
			ScaleSnap = SnapValue;
			break;
		default:
			ensure(false);
		}
	}

	float GetSnapValue(ESnapTypeFlag SnapMode) const 
	{
		switch (SnapMode)
		{
		case ESnapTypeFlag::Location:
			return LocationSnap;			
		case ESnapTypeFlag::Rotation:
			return RotationSnap;
		case ESnapTypeFlag::Scale:
			return ScaleSnap;
		default:
			ensure(false);
			return 0.0;
		}
	}


protected:

	// Toggle for all snapping, can be toggled so that whatever options were stored aren't cleared on enable (vs. ESnapTypeFlag::NoSnap)
	bool bSnappingActive = true;
	int32 SnapEnabled = (uint8)ESnapTypeFlag::NoSnap;
	float LocationSnap = 1.0;
	float RotationSnap = 5.0;
	float ScaleSnap = 1.0;

	bool bGizmoButtonsEnabled = false;
	EGizmoMode GizmoMode = EGizmoMode::Select;
	bool bSelectionButtonsEnabled = false;
	ESelectionMode SelectionMode = ESelectionMode::Island;
};
