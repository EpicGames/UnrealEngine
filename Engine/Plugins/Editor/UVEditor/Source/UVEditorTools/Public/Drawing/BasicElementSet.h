// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/MeshComponent.h"

class FBasicPointSetSceneProxy;


/**
 * A generic data storage class template for 3D or 2D vector data representing geometric elements composed of 1, 2, or 3 points, nominally
 * points, lines and triangles respectively. 
 * 
 * 
 * To support both 2D and 3D with minimal code duplication, the heigher level point/line/triangle set components are composed from the UBasic[*]SetComponentBase class for the component
 * specific functionality, and an instantiation of the TBasicElementSet template class for either FVector2f or FVector3f that encapsulates all data storage specific
 * functionality. The final derived classes UBasic[2/3]D[Point/Line/Triangle]SetComponent contain identical boilerplate code to facilitate calls that
 * need to go both to the component base class and the point base class, e.g. Clear() will both mark the component as dirty and delete all points.  
 */

/** Generic storage for sets of geometry. */
template <typename VectorType, int32 ElementDataSize>
class TBasicElementSet
{
public:

	using GeometryType = VectorType;
	static const int32 ElementSize = ElementDataSize;

	VectorType& operator[](int32 Index)
	{
		return Elements[Index];
	}

	const VectorType& operator[](int32 Index) const
	{
		return Elements[Index];
	}

	/** Current number of points. */
	int32 NumElements() const
	{
		return Num;
	}

	/** Reserve enough memory for up to the given maximum number of points. This function needs to be called before any calls to AddElement(). */
	void ReserveElements(const int32 MaxNum)
	{
		Elements.SetNumUninitialized(MaxNum * ElementDataSize);
	}
	
	/** Reserve enough memory for an additional number of points over the existing number already reserved. This function needs to be called before any calls to AddElement(). */
	void ReserveAdditionalElements(const int32 AdditionalNum)
	{
		Elements.SetNumUninitialized(Elements.Num() + AdditionalNum * ElementDataSize);
	}

	/** Add a point to be rendered using the component. */
	template<typename... Targs>
	void AddElement(Targs... ElementPoints)
	{
		static_assert(sizeof...(Targs) == ElementDataSize);
		checkSlow(Num < Elements.Num());
		AddElementInternal(ElementPoints...);
		Num++;
		bElementsDirty = true;
	}

	void ClearElements()
	{
		Elements.Reset();
		Num = 0;
		bElementsDirty = true;
	}

protected:

	template<typename T, typename... Targs>
	void AddElementInternal(const T& ElementPoint, Targs... ElementPoints)
	{
		int32 Offset = (ElementDataSize - sizeof...(Targs));
		Elements[Num * ElementDataSize + Offset - 1] = ElementPoint;
		AddElementInternal(ElementPoints...);
	}

	template<typename T>
	void AddElementInternal(const T& ElementPoint)
	{
		Elements[Num* ElementDataSize + ElementDataSize-1] = ElementPoint;
	}

	FBox CalcElementsBox() const
	{
		FBox Box(ForceInit);
		for (int32 i = 0; i < Num; ++i)
		{
			for (int32 j = 0; j < ElementDataSize; ++j)
			{
				// Adding a point to an FBox is specific to the point type, and thus we call out to a helper function with dedicated template instantiations.
				AddPointToBox(Elements[i*ElementDataSize + j], Box);
			}
		}
		return Box;
	}

	TArray<VectorType> Elements;
	int32 Num = 0;

	bool bElementsDirty = true;

private:

	// Helper function to generically add a point to an FBox.
	static void AddPointToBox(const VectorType& Point, FBox& Box);
};

// Helper function template instantiations for FVector2f and FVector3f
template <> inline void TBasicElementSet<FVector2f, 1>::AddPointToBox(const FVector2f& Point, FBox& Box) { Box += FVector3d(Point.X, Point.Y, 0.0); }
template <> inline void TBasicElementSet<FVector3f, 1>::AddPointToBox(const FVector3f& Point, FBox& Box) { Box += static_cast<FVector3d>(Point); }
template <> inline void TBasicElementSet<FVector2f, 2>::AddPointToBox(const FVector2f& Point, FBox& Box) { Box += FVector3d(Point.X, Point.Y, 0.0); }
template <> inline void TBasicElementSet<FVector3f, 2>::AddPointToBox(const FVector3f& Point, FBox& Box) { Box += static_cast<FVector3d>(Point); }
template <> inline void TBasicElementSet<FVector2f, 3>::AddPointToBox(const FVector2f& Point, FBox& Box) { Box += FVector3d(Point.X, Point.Y, 0.0); }
template <> inline void TBasicElementSet<FVector3f, 3>::AddPointToBox(const FVector3f& Point, FBox& Box) { Box += static_cast<FVector3d>(Point); }
