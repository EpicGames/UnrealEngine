// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/MeshComponent.h"
#include "BasicElementSet.h"
#include "BasicLineSetComponent.generated.h"

#define UE_API UVEDITORTOOLS_API

class FBasicLineSetSceneProxy;


/**
 * UBasic2DLineSetComponent and UBasic3DLineSetComponent are components that draw a set of lines as small squares. The 2D version stores XY coordinates with
 * Z set to 0 whereas the 3D version stores XYZ coordinates.
 * All lines have the same color, size, and depth bias based on what is set via SetLineSetParameters(). The lines are drawn as two triangles, i.e. a
 * square, orthogonal to the view direction. The actual line size is calculated in the shader using a custom material.
 *
 * Before inserting any lines via AddElement(), the maximum number of lines that will be added needs to be declared via ReserveElements().
 * The actual number of lines that were added is available via NumElements().
 *
 * To support both 2D and 3D with minimal code duplication, the line set components are composed from the UBasicLineSetComponentBase class for the component
 * specific functionality, and an instantiation of the TBasicElementSet template class for either FVector2f or FVector3f that encapsulates all line specific
 * functionality. The final derived classes UBasic2DLineSetComponent and UBasic3DLineSetComponent contain identical boilerplate code to facilitate calls that
 * need to go both to the component base class and the line base class, e.g. Clear() will both mark the component as dirty and delete all lines.
 */

/** Base class for component specific functionality independent of the type of line stored in the component. */
UCLASS(MinimalAPI)
class UBasicLineSetComponentBase : public UMeshComponent
{
	GENERATED_BODY()

public:

	UE_API UBasicLineSetComponentBase();

	/** Clears the component state and marks component as dirty. */
	UE_API void ClearComponent();

	/** Specify material that handles lines. */
	UE_API void SetLineMaterial(UMaterialInterface* InLineMaterial);

	/** Set per line material parameters that are uniform for all lines. */
	UE_API void SetLineSetParameters(FColor InColor, float InSize, float InDepthBias);

	/** Set per line material Line Color parameter that is uniform for all lines */
	UE_API void SetColor(FColor InColor);

	/** Set per line material Line Size/Thickness parameter that is uniform for all lines */
	UE_API void SetLineThickness(float InSize);

protected:

	//~ UMeshComponent Interface.
	virtual int32 GetNumMaterials() const override { return 1; }

	/** Update component bounds using a provided function that determines the box around all line positions.
	 *  This unifies how the bounds are calculated while keeping the calculation of the box around all line positions generic. */
	template <typename CalcLineBoxFn>
	FBoxSphereBounds UpdateComponentBounds(const FTransform& LocalToWorld, bool bLinesDirty, CalcLineBoxFn&& CalcLineBox) const
	{
		bBoundsDirty |= bLinesDirty;
		if (bBoundsDirty)
		{
			// Calculating an FBox around a set of lines is specific to the line type, and thus we call out to a provided type specific lambda.
			const FBox Box = CalcLineBox();

			Bounds = FBoxSphereBounds(Box);
			bBoundsDirty = false;

			// TODO: This next bit is not ideal because the line size is specified in onscreen pixels,
			// so the true amount by which we would need to expand bounds depends on camera location, FOV, etc.
			// We mainly do this as a hack against a problem in orthographic viewports, which cull small items
			// based on their bounds, and a set consisting of a single line will always be culled due
			// to having 0-sized bounds. It's worth noting that when zooming out sufficiently far, the
			// line will still be culled even with this hack, however.
			// The proper solution is to be able to opt out of the orthographic culling behavior, which is something
			// we need to add.
			if (Box.IsValid)
			{
				Bounds = Bounds.ExpandBy(Size);
			}
		}
		return Bounds.TransformBy(LocalToWorld);
	}

	UPROPERTY()
		TObjectPtr<const UMaterialInterface> LineMaterial;

	UPROPERTY()
		mutable FBoxSphereBounds Bounds;

	UPROPERTY()
		mutable bool bBoundsDirty = true;

	UPROPERTY()
		FColor Color;

	UPROPERTY()
		float Size = 1.0f;

	UPROPERTY()
		float DepthBias = 0.0f;
};

/** Instantiation of a basic line set component in 2D using FVector2f for line positions. */
UCLASS(MinimalAPI)
class UBasic2DLineSetComponent final : public UBasicLineSetComponentBase, public TBasicElementSet<FVector2f, 2>
{
	GENERATED_BODY()

public:
	/** Clear all lines and component state. */
	void Clear()
	{
		ClearElements();
		ClearComponent();
	}

private:

	//~ UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy * CreateSceneProxy() override;

	//~ USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override
	{
		// The call to CalcElementsBox() is specific to the element type but already available in the generic implementation in TBasicElementSet<>.
		return UpdateComponentBounds(LocalToWorld, bElementsDirty, [this]() { return CalcElementsBox(); });
	}

	friend class FBasicLineSetSceneProxy;
};

/** Instantiation of a basic line set component in 3D using FVector3f for line positions. */
UCLASS(MinimalAPI)
class UBasic3DLineSetComponent final : public UBasicLineSetComponentBase, public TBasicElementSet<FVector3f, 2>
{
	GENERATED_BODY()

public:
	/** Clear all lines and component state. */
	void Clear()
	{
		ClearElements();
		ClearComponent();
	}

private:

	//~ UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy * CreateSceneProxy() override;

	//~ USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override
	{
		// The call to CalcElementsBox() is specific to the element type but already available in the generic implementation in TBasicElementSet<>.
		return UpdateComponentBounds(LocalToWorld, bElementsDirty, [this]() { return CalcElementsBox(); });
	}

	friend class FBasicLineSetSceneProxy;
};

#undef UE_API
