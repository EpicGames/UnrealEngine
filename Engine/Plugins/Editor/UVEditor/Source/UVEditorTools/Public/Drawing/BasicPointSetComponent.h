// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/MeshComponent.h"
#include "BasicElementSet.h"
#include "BasicPointSetComponent.generated.h"

#define UE_API UVEDITORTOOLS_API

class FBasicPointSetSceneProxy;


/**
 * UBasic2DPointSetComponent and UBasic3DPointSetComponent are components that draw a set of points as small squares. The 2D version stores XY coordinates with
 * Z set to 0 whereas the 3D version stores XYZ coordinates.
 * All points have the same color, size, and depth bias based on what is set via SetPointSetParameters(). The points are drawn as two triangles, i.e. a
 * square, orthogonal to the view direction. The actual point size is calculated in the shader using a custom material.
 *
 * Before inserting any points via AddElement(), the maximum number of points that will be added needs to be declared via ReserveElements().
 * The actual number of points that were added is available via NumElements().
 *
 * To support both 2D and 3D with minimal code duplication, the point set components are composed from the UBasicPointSetComponentBase class for the component
 * specific functionality, and an instantiation of the TBasicElementSet template class for either FVector2f or FVector3f that encapsulates all point specific
 * functionality. The final derived classes UBasic2DPointSetComponent and UBasic3DPointSetComponent contain identical boilerplate code to facilitate calls that
 * need to go both to the component base class and the point base class, e.g. Clear() will both mark the component as dirty and delete all points.  
 */

/** Base class for component specific functionality independent of the type of point stored in the component. */
UCLASS(MinimalAPI)
class UBasicPointSetComponentBase : public UMeshComponent
{
	GENERATED_BODY()

public:

	UE_API UBasicPointSetComponentBase();

	/** Clears the component state and marks component as dirty. */
	UE_API void ClearComponent();

	/** Specify material that handles points. */
	UE_API void SetPointMaterial(UMaterialInterface* InPointMaterial);

	/** Set per point material parameters that are uniform for all points. */
	UE_API void SetPointSetParameters(FColor InColor, float InSize, float InDepthBias);

	/** Set per point material Color parameter that is uniform for all points. */
	UE_API void SetColor(const FColor InColor);

	/** Set per point material Size parameter tat is uniform for all points. */
	UE_API void SetPointSize(const float InSize);

protected:

	//~ UMeshComponent Interface.
	virtual int32 GetNumMaterials() const override { return 1; }

	/** Update component bounds using a provided function that determines the box around all point positions.
	 *  This unifies how the bounds are calculated while keeping the calculation of the box around all point positions generic. */
	template <typename CalcPointBoxFn>
	FBoxSphereBounds UpdateComponentBounds(const FTransform& LocalToWorld, bool bPointsDirty, CalcPointBoxFn&& CalcPointBox) const
	{
		bBoundsDirty |= bPointsDirty;
		if (bBoundsDirty)
		{
			// Calculating an FBox around a set of points is specific to the point type, and thus we call out to a provided type specific lambda.
			const FBox Box = CalcPointBox();

			Bounds = FBoxSphereBounds(Box);
			bBoundsDirty = false;

			// TODO: This next bit is not ideal because the point size is specified in onscreen pixels,
			// so the true amount by which we would need to expand bounds depends on camera location, FOV, etc.
			// We mainly do this as a hack against a problem in orthographic viewports, which cull small items
			// based on their bounds, and a set consisting of a single point will always be culled due
			// to having 0-sized bounds. It's worth noting that when zooming out sufficiently far, the
			// point will still be culled even with this hack, however.
			// The proper solution is to be able to opt out of the orthographic culling behavior, which is something
			// we need to add.
			if (Box.IsValid)
			{
				Bounds = Bounds.ExpandBy(Size);
			}
		}
		return Bounds.TransformBy(LocalToWorld);
	}

	UPROPERTY()
	TObjectPtr<const UMaterialInterface> PointMaterial;

	UPROPERTY()
	mutable FBoxSphereBounds Bounds;

	UPROPERTY()
	mutable bool bBoundsDirty = true;

	UPROPERTY()
	FColor Color;

	UPROPERTY()
	float Size = 1.0f;

	UPROPERTY()
	float DepthBias = 0.0f;
};

/** Instantiation of a basic point set component in 2D using FVector2f for point positions. */
UCLASS(MinimalAPI)
class UBasic2DPointSetComponent final : public UBasicPointSetComponentBase, public TBasicElementSet<FVector2f, 1>
{
	GENERATED_BODY()

public:
	/** Clear all points and component state. */
	void Clear()
	{
		ClearElements();
		ClearComponent();
	}

private:

	//~ UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;

	//~ USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override
	{
		// The call to CalcElementsBox() is specific to the element type but already available in the generic implementation in TBasicElementSet<>.
		return UpdateComponentBounds(LocalToWorld, bElementsDirty, [this]() { return CalcElementsBox(); });
	}

	friend class FBasicPointSetSceneProxy;
};

/** Instantiation of a basic point set component in 3D using FVector3f for point positions. */
UCLASS(MinimalAPI)
class UBasic3DPointSetComponent final : public UBasicPointSetComponentBase, public TBasicElementSet<FVector3f, 1>
{
	GENERATED_BODY()

public:
	/** Clear all points and component state. */
	void Clear()
	{
		ClearElements();
		ClearComponent();
	}

private:

	//~ UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;

	//~ USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override
	{
		// The call to CalcElementsBox() is specific to the element type but already available in the generic implementation in TBasicElementSet<>.
		return UpdateComponentBounds(LocalToWorld, bElementsDirty, [this]() { return CalcElementsBox(); });
	}

	friend class FBasicPointSetSceneProxy;
};

#undef UE_API
