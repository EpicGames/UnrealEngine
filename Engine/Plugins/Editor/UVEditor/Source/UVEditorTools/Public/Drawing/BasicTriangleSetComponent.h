// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/MeshComponent.h"
#include "BasicElementSet.h"
#include "BasicTriangleSetComponent.generated.h"

#define UE_API UVEDITORTOOLS_API

class FBasicTriangleSetSceneProxy;


/**
 * UBasic2DTriangleSetComponent and UBasic3DTriangleSetComponent are components that draw a set of lines as small squares. The 2D version stores XY coordinates with
 * Z set to 0 whereas the 3D version stores XYZ coordinates.
 * All lines have the same color, size, and depth bias based on what is set via SetTriangleSetParameters(). The lines are drawn as two triangles, i.e. a
 * square, orthogonal to the view direction. The actual line size is calculated in the shader using a custom material.
 *
 * Before inserting any lines via AddElement(), the maximum number of lines that will be added needs to be declared via ReserveElements().
 * The actual number of lines that were added is available via NumElements().
 *
 * To support both 2D and 3D with minimal code duplication, the line set components are composed from the UBasicTriangleSetComponentBase class for the component
 * specific functionality, and an instantiation of the TBasicElementSet template class for either FVector2f or FVector3f that encapsulates all line specific
 * functionality. The final derived classes UBasic2DTriangleSetComponent and UBasic3DTriangleSetComponent contain identical boilerplate code to facilitate calls that
 * need to go both to the component base class and the line base class, e.g. Clear() will both mark the component as dirty and delete all lines.
 */

 /** Base class for component specific functionality independent of the type of line stored in the component. */
UCLASS(MinimalAPI)
class UBasicTriangleSetComponentBase : public UMeshComponent
{
	GENERATED_BODY()

public:

	UE_API UBasicTriangleSetComponentBase();

	/** Clears the component state and marks component as dirty. */
	UE_API void ClearComponent();

	/** Specify material that handles lines. */
	UE_API void SetTriangleMaterial(UMaterialInterface* InTriangleMaterial);

	/** Set per line material parameters that are uniform for all lines. */
	UE_API void SetTriangleSetParameters(FColor InColor, const FVector3f& NormalIn);

protected:

	//~ UMeshComponent Interface.
	virtual int32 GetNumMaterials() const override { return 1; }

	/** Update component bounds using a provided function that determines the box around all line positions.
	 *  This unifies how the bounds are calculated while keeping the calculation of the box around all line positions generic. */
	template <typename CalcTriangleBoxFn>
	FBoxSphereBounds UpdateComponentBounds(const FTransform& LocalToWorld, bool bTrianglesDirty, CalcTriangleBoxFn&& CalcTriangleBox) const
	{
		bBoundsDirty |= bTrianglesDirty;
		if (bBoundsDirty)
		{
			// Calculating an FBox around a set of lines is specific to the line type, and thus we call out to a provided type specific lambda.
			const FBox Box = CalcTriangleBox();

			Bounds = FBoxSphereBounds(Box);
			bBoundsDirty = false;
		}
		return Bounds.TransformBy(LocalToWorld);
	}

	UPROPERTY()
	TObjectPtr<const UMaterialInterface> TriangleMaterial;

	UPROPERTY()
	mutable FBoxSphereBounds Bounds;

	UPROPERTY()
	mutable bool bBoundsDirty = true;

	UPROPERTY()
	FColor Color;

	UPROPERTY()
	FVector3f Normal;
};

/** Instantiation of a basic line set component in 2D using FVector2f for line positions. */
UCLASS(MinimalAPI)
class UBasic2DTriangleSetComponent final : public UBasicTriangleSetComponentBase, public TBasicElementSet<FVector2f, 3>
{
	GENERATED_BODY()

public:
	/** Clear all lines and component state. */
	void Clear()
	{
		ClearElements();
		ClearComponent();
	}

private:

	//~ UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy * CreateSceneProxy() override;

	//~ USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override
	{
		// The call to CalcElementsBox() is specific to the element type but already available in the generic implementation in TBasicElementSet<>.
		return UpdateComponentBounds(LocalToWorld, bElementsDirty, [this]() { return CalcElementsBox(); });
	}

	friend class FBasicTriangleSetSceneProxy;
};

/** Instantiation of a basic line set component in 3D using FVector3f for line positions. */
UCLASS(MinimalAPI)
class UBasic3DTriangleSetComponent final : public UBasicTriangleSetComponentBase, public TBasicElementSet<FVector3f, 3>
{
	GENERATED_BODY()

public:
	/** Clear all lines and component state. */
	void Clear()
	{
		ClearElements();
		ClearComponent();
	}

private:

	//~ UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy * CreateSceneProxy() override;

	//~ USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override
	{
		// The call to CalcElementsBox() is specific to the element type but already available in the generic implementation in TBasicElementSet<>.
		return UpdateComponentBounds(LocalToWorld, bElementsDirty, [this]() { return CalcElementsBox(); });
	}

	friend class FBasicTriangleSetSceneProxy;
};

#undef UE_API
