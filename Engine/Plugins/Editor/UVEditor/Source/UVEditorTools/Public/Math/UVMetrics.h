// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.5, "Use Parameterization/UVMetrics.h from GeometryAlgorithms instead of UVMetrics.h")

#include "Parameterization/UVMetrics.h"