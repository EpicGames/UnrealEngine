// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "InteractiveTool.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Selection/UVToolSelection.h"

#include "UVEditorUVTransformOp.generated.h"

#define UE_API UVEDITORTOOLS_API

class UUVTransformProperties;
class UUVEditorTransformTool;

/**
 * UV Transform Strategies for the UV Transform Tool
 */
UENUM()
enum class EUVEditorUVTransformType
{
	/** Apply Scale, Totation and Translation properties to all UV values */
	Transform,
	/** Position all selected elements given alignment rules */
	Align,
	/** Position all selected elements given distribution rules */
	Distribute
};

/**
 * Transform Pivot Mode
 */
UENUM()
enum class EUVEditorPivotType
{
	/** Pivot around the collective center of all island bounding boxes */
	BoundingBoxCenter,
	/** Pivot around the global origin point */
	Origin,
	/** Pivot around each island's bounding box center */
	IndividualBoundingBoxCenter,
	/** Pivot around a user specified point */
	Manual
};

/**
 * Translation Mode
 */
UENUM()
enum class EUVEditorTranslationMode
{
	/** Move elements relative to their current position by the amount specified */
	Relative,
	/** Move elements such that the transform origin is placed at the value specified */
	Absolute
};

UENUM()
enum class EUVEditorAlignDirection
{
	None,
	Top,
	Bottom,
	Left,
	Right,
	CenterVertically,
	CenterHorizontally
};

UENUM()
enum class EUVEditorAlignAnchor
{
	//FirstItem, // TODO: Support this later once we have support in the Selection API

	/** Align relative to the collective bounding box of all islands */
	BoundingBox,
	/** Align relative to the local UDIM tile containing the island */
	UDIMTile,
	/** Align relative to a user specified point */
	Manual
};

UENUM()
enum class EUVEditorDistributeMode : uint16
{
	None,
	VerticalSpace,
	HorizontalSpace,
	TopEdges,
	BottomEdges,
	LeftEdges,
	RightEdges,
	CentersVertically,
	CentersHorizontally,
	MinimallyRemoveOverlap
};

UENUM()
enum class EUVEditorAlignDistributeGroupingMode : uint8
{
	/** Treat selection as individual connected components, moving each according to their bounding boxes.*/
	IndividualBoundingBoxes,
	/** Treat selection as a single unit, moving everything together and preserving relative positions.*/
	EnclosingBoundingBox,
	/** Treat selection as isolated UV vertices, moving each UV independently of each other.*/
	IndividualVertices
};

/**
 * UV Transform Settings
 */
UCLASS(MinimalAPI)
class UUVEditorUVTransformPropertiesBase : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UUVEditorUVTransformPropertiesBase()
	{
		// TODO: This makes the changes to the Transform, Align, and Distribute property sets show up
		// in the Undo history, so the button presses in Quick Translate, Align and Distribute modes
		// can be undone. However, it doens't properly track tool lifetime and therefore doesn't handle
		// correct expiry when the tools using the properties shutdown. A custom change should probably
		// be implemented to handle this behavior more correctly.
		SetFlags(RF_Transactional);
	}

};


/**
 * UV Transform Settings
 */
UCLASS(MinimalAPI)
class UUVEditorUVTransformProperties : public UUVEditorUVTransformPropertiesBase
{
	GENERATED_BODY()

public:
	/** Scale applied to UVs, potentially non-uniform */
	UPROPERTY(EditAnywhere, Category = "AdvancedTransform|Scaling", DuplicateTransient, meta = (TransientToolProperty, DisplayName = "Scale"))
	FVector2D Scale = FVector2D(1.0, 1.0); 

	/** Rotation applied to UVs after scaling, specified in degrees */
	UPROPERTY(EditAnywhere, Category = "AdvancedTransform|Rotation", meta = (TransientToolProperty, DisplayName = "Rotation", 
		                                                                        UIMin = "-360", UIMax = "360", ClampMin = "-100000", ClampMax = "100000"))
	float Rotation = 0.0;

	/** Translation applied to UVs, and after scaling and rotation */
	UPROPERTY(EditAnywhere, Category = "AdvancedTransform|Translation", meta = (TransientToolProperty, DisplayName = "Translation"))
	FVector2D Translation = FVector2D(0, 0);

	/** Translation applied to UVs, and after scaling and rotation */
	UPROPERTY(EditAnywhere, Category = "AdvancedTransform|Translation", meta = (TransientToolProperty, DisplayName = "Translation Mode"))
	EUVEditorTranslationMode TranslationMode = EUVEditorTranslationMode::Relative;

	/** Transformation origin mode used for scaling and rotation */
	UPROPERTY(EditAnywhere, Category = "AdvancedTransform|TransformOrigin", meta = (TransientToolProperty, DisplayName = "Mode"))
	EUVEditorPivotType PivotMode = EUVEditorPivotType::BoundingBoxCenter;

	/** Manual Transformation origin point */
	UPROPERTY(EditAnywhere, Category = "AdvancedTransform|TransformOrigin", meta = (TransientToolProperty, DisplayName = "Coords",
		EditCondition = "PivotMode == EUVEditorPivotType::Manual", EditConditionHides, HideEditConditionToggle = true))
	FVector2D ManualPivot = FVector2D(0, 0);

	UPROPERTY(EditAnywhere, Category = "Quick Transform", meta = (TransientToolProperty, DisplayName = "Translation"))
	float QuickTranslateOffset = 0.0;

	UPROPERTY(EditAnywhere, Category = "Quick Transform", meta = (TransientToolProperty, DisplayName = "Rotation", 
		                                                          UIMin = "-360", UIMax = "360", ClampMin = "-100000", ClampMax = "100000"))
	float QuickRotationOffset = 0.0;

	UPROPERTY(EditAnywhere, Category = "Quick Transform", meta = (TransientToolProperty))
	FVector2D QuickTranslation;

	UPROPERTY(EditAnywhere, Category = "Quick Transform", meta = (TransientToolProperty))
	float QuickRotation;
};

/**
 * UV Quick Transform Only Settings
 * 
 * We are using a subclass here to "trick" the details customization system, allowing us to reuse a lot of the logic
 * without having to build new operators or new customizations.
 * 
 * See FUVEditorUVTransformToolDetails and FUVEditorUVQuickTransformToolDetails, where we provide two customizations
 * which present different views of the "same" properties - one with all settings and one with simply the quick transform settings.
 * This is designed to be used in the future where we want to provide a quick translate "tool" when no other tools are running.
 * 
 */
UCLASS(MinimalAPI)
class UUVEditorUVQuickTransformProperties : public UUVEditorUVTransformProperties
{
	GENERATED_BODY()
};

/**
 * UV Align Settings
 */
UCLASS(MinimalAPI)
class UUVEditorUVAlignProperties : public UUVEditorUVTransformPropertiesBase
{
	GENERATED_BODY()

public:	

	/** Controls what geometry the alignment is to be relative to when performed. */
	UPROPERTY(EditAnywhere, Category = "Align", meta = (DisplayName = "Alignment Anchor"))
	EUVEditorAlignAnchor AlignAnchor = EUVEditorAlignAnchor::BoundingBox;

	/** Manual anchor location for relative alignment */
	UPROPERTY(EditAnywhere, Category = "Align", meta = (DisplayName = "Anchor Coords",
		EditCondition = "AlignAnchor == EUVEditorAlignAnchor::Manual", EditConditionHides, HideEditConditionToggle = true))
	FVector2D ManualAnchor = FVector2D(0, 0);

	/** Controls what side of the island bounding boxes are being aligned */
	UPROPERTY(EditAnywhere, Category = "Align", meta = (TransientToolProperty, DisplayName = "Alignment Direction"))
	EUVEditorAlignDirection AlignDirection = EUVEditorAlignDirection::None;

	/** Controls how alignment considers grouping selected objects with respect to the alignment behavior.*/
	UPROPERTY(EditAnywhere, Category = "Align", meta = (DisplayName = "Grouping Mode"))
	EUVEditorAlignDistributeGroupingMode Grouping = EUVEditorAlignDistributeGroupingMode::IndividualBoundingBoxes;

};

/**
 * UV Distribute Settings
 */
UCLASS(MinimalAPI)
class UUVEditorUVDistributeProperties : public UUVEditorUVTransformPropertiesBase
{
	GENERATED_BODY()
public:
	/** Controls the distribution behavior */
	UPROPERTY(EditAnywhere, Category = "Distribute", meta = (TransientToolProperty, DisplayName = "Distribution Mode"))
	EUVEditorDistributeMode DistributeMode = EUVEditorDistributeMode::None;

	/** Controls how distribution considers grouping selected objects with respect to the distribution behavior.*/
	UPROPERTY(EditAnywhere, Category = "Distribute", meta = (DisplayName = "Grouping Mode"))
	EUVEditorAlignDistributeGroupingMode Grouping = EUVEditorAlignDistributeGroupingMode::IndividualBoundingBoxes;

	/** If true, enable overriding distances used in the distribution behavior with manually entered values.*/
	UPROPERTY(EditAnywhere, Category = "Distribute", meta = (DisplayName = "Manual Distances"))
	bool bEnableManualDistances;

	/** For Edge and Center distribution modes, specify the desired overall distance within which to evenly place the edges or centers. */
	UPROPERTY(EditAnywhere, Category = "Distribute", meta = (DisplayName = "Manual Extent",
		EditCondition = "bEnableManualDistances", EditConditionHides, HideEditConditionToggle = true))
	float ManualExtent;

	/** For Spacing and Remove Overlap distribution modes, specify the desired distance between objects. */
	UPROPERTY(EditAnywhere, Category = "Distribute", meta = (DisplayName = "Manual Spacing",
		EditCondition = "bEnableManualDistances", EditConditionHides, HideEditConditionToggle = true))
	float ManualSpacing;

};


namespace UE
{
namespace Geometry
{
	class FDynamicMesh3;
	class FDynamicMeshUVPacker;
	class FMeshConnectedComponents;

	enum class EUVEditorUVTransformTypeBackend
	{
		Transform,
		Align,
		Distribute
	};

	enum class EUVEditorPivotTypeBackend
	{
		Origin,
		IndividualBoundingBoxCenter,
		BoundingBoxCenter,
		Manual
	};

	enum class EUVEditorTranslationModeBackend
	{
		Relative,
		Absolute
	};

	enum class EUVEditorAlignDirectionBackend
	{
		None,
		Top,
		Bottom,
		Left,
		Right,
		CenterVertically,
		CenterHorizontally
	};

	enum class EUVEditorAlignAnchorBackend
	{
		//FirstItem, // TODO:  Support this later once we have support in the Selection API
		UDIMTile,
		BoundingBox,
		Manual
	};

	enum class EUVEditorDistributeModeBackend
	{
		None,
		LeftEdges,
		RightEdges,
		TopEdges,
		BottomEdges,
		CentersVertically,
		CentersHorizontally,
		VerticalSpace,
		HorizontalSpace,
		MinimallyRemoveOverlap
	};

	enum class EUVEditorAlignDistributeGroupingModeBackend
	{
		IndividualBoundingBoxes,
		EnclosingBoundingBox,
		IndividualVertices
	};

class FUVEditorUVTransformBaseOp : public FDynamicMeshOperator
{
public:
	virtual ~FUVEditorUVTransformBaseOp() = default;

	// inputs
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	int UVLayerIndex = 0;
	EUVEditorAlignDistributeGroupingModeBackend GroupingMode;

	UE_API void SetTransform(const FTransformSRT3d& Transform);
	UE_API void SetSelection(const TSet<int32>& EdgeSelection, const TSet<int32>& VertexSelection);

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;
	const TArray<FVector2D>& GetPivotLocations() const { return VisualizationPivots; }

protected:
	virtual void HandleTransformationOp(FProgressCancel* Progress) = 0;

	UE_API FVector2f GetAlignmentPointFromBoundingBoxAndDirection(EUVEditorAlignDirectionBackend Direction, const FAxisAlignedBox2d& BoundingBox);
	UE_API FVector2f GetAlignmentPointFromUDIMAndDirection(EUVEditorAlignDirectionBackend Direction, FVector2i UDIMTile);
	UE_API void RebuildBoundingBoxes();
	UE_API void CollectTransformElements();
	UE_API void SortComponentsByBoundingBox(bool bSortXThenY = true);

	TMap<int32, int32> ElementToComponent;
	FDynamicMeshUVOverlay* ActiveUVLayer;
	FAxisAlignedBox2d OverallBoundingBox;
	TArray<FAxisAlignedBox2d> PerComponentBoundingBoxes;
	TOptional< TSet<int32> > TransformingElements;
	TSharedPtr<FMeshConnectedComponents> UVComponents;
	TArray<int32> SpatiallyOrderedComponentIndex;
	TArray<FVector2D> VisualizationPivots;

	TOptional<TSet<int32>> VertexSelection;
	TOptional<TSet<int32>> EdgeSelection;
};


/**
 */
class FUVEditorUVTransformOp : public FUVEditorUVTransformBaseOp
{
public:
	virtual ~FUVEditorUVTransformOp() = default;

	// inputs
	FVector2D QuickTranslation = FVector2D(0.0, 0.0);
	float QuickRotation = 0.0;

	FVector2D Scale = FVector2D(1.0, 1.0);
	float Rotation = 0.0;
	FVector2D Translation = FVector2D(0, 0);
	EUVEditorTranslationModeBackend TranslationMode = EUVEditorTranslationModeBackend::Relative;

	EUVEditorPivotTypeBackend PivotMode = EUVEditorPivotTypeBackend::Origin;
	FVector2D ManualPivot = FVector2D(0, 0);

protected:
	UE_API virtual void HandleTransformationOp(FProgressCancel* Progress) override;
	UE_API FVector2f GetPivotFromMode(int32 ElementID, EUVEditorPivotTypeBackend Mode);
};

class FUVEditorUVAlignOp : public FUVEditorUVTransformBaseOp
{
public:
	virtual ~FUVEditorUVAlignOp() = default;

	EUVEditorAlignAnchorBackend AlignAnchor = EUVEditorAlignAnchorBackend::BoundingBox;
	EUVEditorAlignDirectionBackend AlignDirection = EUVEditorAlignDirectionBackend::None;
	FVector2D ManualAnchor = FVector2D(0, 0);

protected:
	UE_API virtual void HandleTransformationOp(FProgressCancel* Progress) override;



};

class FUVEditorUVDistributeOp : public FUVEditorUVTransformBaseOp
{
public:
	virtual ~FUVEditorUVDistributeOp() = default;

	EUVEditorDistributeModeBackend DistributeMode = EUVEditorDistributeModeBackend::None;
	bool bEnableManualDistances;
	float ManualExtent;
	float ManualSpacing;

protected:
	UE_API virtual void HandleTransformationOp(FProgressCancel* Progress) override;

};


} // end namespace UE::Geometry
} // end namespace UE

/**
 * Can be hooked up to a UMeshOpPreviewWithBackgroundCompute to perform UV Transform operations.
 *
 * Inherits from UObject so that it can hold a strong pointer to the settings UObject, which
 * needs to be a UObject to be displayed in the details panel.
 */
UCLASS(MinimalAPI)
class UUVEditorUVTransformOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	EUVEditorUVTransformType TransformType;

	UPROPERTY()
	TObjectPtr<UUVEditorUVTransformPropertiesBase> Settings;

	TOptional<TSet<int32>> EdgeSelection;
	TOptional<TSet<int32>> VertexSelection;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TUniqueFunction<int32()> GetSelectedUVChannel = []() { return 0; };
	FTransform TargetTransform;
};

#undef UE_API
