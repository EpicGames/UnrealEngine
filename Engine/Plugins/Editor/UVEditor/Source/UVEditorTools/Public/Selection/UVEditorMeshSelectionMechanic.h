// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryBase.h" // Predeclare macros

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "InteractionMechanic.h"
#include "InteractiveTool.h"
#include "Selection/UVToolSelection.h"
#include "Selection/UVToolSelectionAPI.h" // EUVEditorSelectionMode
#include "Mechanics/RectangleMarqueeMechanic.h"
#include "ToolContextInterfaces.h" //FViewCameraState

#include "UVEditorMeshSelectionMechanic.generated.h"

#define UE_API UVEDITORTOOLS_API

class APreviewGeometryActor;
struct FCameraRectangle;
class ULineSetComponent;
class UMaterialInstanceDynamic;
class UPointSetComponent;
class UTriangleSetComponent;
class UUVToolViewportButtonsAPI;
class ULocalSingleClickInputBehavior;
class ULocalMouseHoverBehavior;

/**
 * Mechanic for selecting elements of a dynamic mesh in the UV editor. Interacts
 * heavily with UUVToolSelectionAPI, which actually stores selections.
 */
UCLASS(MinimalAPI)
class UUVEditorMeshSelectionMechanic : public UInteractionMechanic
{
	GENERATED_BODY()

public:
	using FDynamicMeshAABBTree3 = UE::Geometry::FDynamicMeshAABBTree3;
	using FUVToolSelection = UE::Geometry::FUVToolSelection;

	virtual ~UUVEditorMeshSelectionMechanic() = default;

	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown() override;

	// Initialization functions.
	// The selection API is provided as a parameter rather than being grabbed out of the context
	// store mainly because UVToolSelectionAPI itself sets up a selection mechanic, and is not 
	// yet in the context store when it does this. 
	UE_API void Initialize(UWorld* World, UWorld* LivePreviewWorld, UUVToolSelectionAPI* SelectionAPI);
	UE_API void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn);

	UE_API void SetIsEnabled(bool bIsEnabled);
	bool IsEnabled() { return bIsEnabled; };

	struct FRaycastResult
	{
		int32 AssetID = IndexConstants::InvalidID;
		int32 Tid = IndexConstants::InvalidID;
		FVector3d HitPosition;
	};
	/**
	 * This is a helper method that doesn't get used in normal selection mechanic operation, but
	 *  can be used by clients if they need to raycast canonical meshes, since the mechanic already
	 *  keeps aabb trees for them. The mechanic does not need to be enabled to use this (and typically
	 *  will not be, if a tool is doing its own raycasting).
	 */
	UE_API bool RaycastCanonicals(const FRay& WorldRay, bool bRaycastIsForUnwrap,
		bool bPreferSelected, FRaycastResult& HitOut) const;

	UE_API TArray<FUVToolSelection> GetAllCanonicalTrianglesInUnwrapRadius(const FVector2d& UnwrapWorldHitPoint, double Radius) const;

	UE_API void SetShowHoveredElements(bool bShow);

	using ESelectionMode = UUVToolSelectionAPI::EUVEditorSelectionMode;
	using FModeChangeOptions = UUVToolSelectionAPI::FSelectionMechanicModeChangeOptions;
	/**
	 * Sets selection mode for the mechanic.
	 */
	UE_API void SetSelectionMode(ESelectionMode TargetMode,
		const FModeChangeOptions& Options = FModeChangeOptions());
	
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);
	UE_API virtual void LivePreviewRender(IToolsContextRenderAPI* RenderAPI);
	UE_API virtual void LivePreviewDrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);
	// IClickBehaviorTarget implementation
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos, bool bSourceIsLivePreview);
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos, bool bSourceIsLivePreview);

	// IModifierToggleBehaviorTarget implementation
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn);

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos, bool bSourceIsLivePreview);
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos);
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos, bool bSourceIsLivePreview);
	UE_API virtual void OnEndHover();

	/**
	 * Broadcasted whenever the marquee mechanic rectangle is changed, since these changes
	 * don't trigger normal selection broadcasts.
	 */ 
	FSimpleMulticastDelegate OnDragSelectionChanged;

protected:

	UPROPERTY()
	TObjectPtr<UUVToolSelectionAPI> SelectionAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolViewportButtonsAPI> ViewportButtonsAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolEmitChangeAPI> EmitChangeAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolLivePreviewAPI> LivePreviewAPI = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalSingleClickInputBehavior> UnwrapClickTargetRouter = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalSingleClickInputBehavior> LivePreviewClickTargetRouter = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalMouseHoverBehavior> UnwrapHoverBehaviorTargetRouter = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalMouseHoverBehavior> LivePreviewHoverBehaviorTargetRouter = nullptr;

	UPROPERTY()
	TObjectPtr<URectangleMarqueeMechanic> MarqueeMechanic = nullptr;
	
	UPROPERTY()
	TObjectPtr<URectangleMarqueeMechanic> LivePreviewMarqueeMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> HoverTriangleSetMaterial = nullptr;

	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> HoverGeometryActor = nullptr;
	// Weak pointers so that they go away when geometry actor is destroyed
	TWeakObjectPtr<UTriangleSetComponent> HoverTriangleSet = nullptr;
	TWeakObjectPtr<ULineSetComponent> HoverLineSet = nullptr;
	TWeakObjectPtr<UPointSetComponent> HoverPointSet = nullptr;

	UPROPERTY()
	TObjectPtr<UInputBehaviorSet> LivePreviewBehaviorSet = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalInputBehaviorSource> LivePreviewBehaviorSource = nullptr;

	TWeakObjectPtr<UInputRouter> LivePreviewInputRouter = nullptr;

	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> LivePreviewHoverGeometryActor = nullptr;
	// Weak pointers so that they go away when geometry actor is destroyed
	TWeakObjectPtr<UTriangleSetComponent> LivePreviewHoverTriangleSet = nullptr;
	TWeakObjectPtr<ULineSetComponent> LivePreviewHoverLineSet = nullptr;
	TWeakObjectPtr<UPointSetComponent> LivePreviewHoverPointSet = nullptr;

	// Should be the same as the mode-level targets array, indexed by AssetID
	UE_API TSharedPtr<FDynamicMeshAABBTree3> GetMeshSpatial(int32 TargetId, bool bUseUnwrap) const;
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;
	TArray<TSharedPtr<FDynamicMeshAABBTree3>> UnwrapMeshSpatials; // 1:1 with Targets
	TArray<TSharedPtr<FDynamicMeshAABBTree3>> AppliedMeshSpatials; // 1:1 with Targets

	ESelectionMode SelectionMode;
	bool bIsEnabled = false;
	bool bShowHoveredElements = true;

	UE_API bool GetHitTid(const FInputDeviceRay& ClickPos, int32& TidOut,
		int32& AssetIDOut, bool bUseUnwrap, int32* ExistingSelectionObjectIndexOut = nullptr);
	UE_API void ModifyExistingSelection(TSet<int32>& SelectionSetToModify, const TArray<int32>& SelectedIDs);

	FViewCameraState CameraState;

	bool bShiftToggle = false;
	bool bCtrlToggle = false;
	static const int32 ShiftModifierID = 1;
	static const int32 CtrlModifierID = 2;

	// All four combinations of shift/ctrl down are assigned a behaviour
	bool ShouldAddToSelection() const { return !bCtrlToggle && bShiftToggle; }
	bool ShouldRemoveFromSelection() const { return bCtrlToggle && !bShiftToggle; }
	bool ShouldToggleFromSelection() const { return bCtrlToggle && bShiftToggle; }
	bool ShouldRestartSelection() const { return !bCtrlToggle && !bShiftToggle; }

	// For marquee mechanic
	UE_API void OnDragRectangleStarted();
	UE_API void OnDragRectangleChanged(const FCameraRectangle& CurrentRectangle, bool bSourceIsLivePreview);
	UE_API void OnDragRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled, bool bSourceIsLivePreview);

	TArray<FUVToolSelection> PreDragSelections;
	TArray<FUVToolSelection> PreDragUnsetSelections;
	// Maps asset id to a pre drag selection so that it is easy to tell which assets
	// started with a selection. 1:1 with Targets.
	TArray<const FUVToolSelection*> AssetIDToPreDragSelection;
	TArray<const FUVToolSelection*> AssetIDToPreDragUnsetSelection;
};

#undef UE_API
