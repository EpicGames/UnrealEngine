// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolTargets/UVEditorToolMeshInput.h"
#include "Selection/StoredMeshSelectionUtil.h" //FEdgesFromTriangleSubIndices
#include "DynamicMesh/DynamicMesh3.h"
#include "Util/IndexUtil.h"

#define UE_API UVEDITORTOOLS_API

namespace UE {
namespace Geometry {

/**
 * Class that represents a selection in the canonical unwrap of a UV editor input object.
 */
class FUVToolSelection
{
public:
	enum class EType
	{
		Vertex,
		Edge,
		Triangle
	};

	TWeakObjectPtr<UUVEditorToolMeshInput> Target = nullptr;
	EType Type = EType::Vertex;
	TSet<int32> SelectedIDs;

	void Clear()
	{
		Target = nullptr;
		SelectedIDs.Reset();
		StableEdgeIDs.Reset();
	}

	UE_API void SelectAll(const FDynamicMesh3& Mesh, EType TypeIn);

	UE_API bool IsAllSelected(const FDynamicMesh3& Mesh) const;

	bool IsEmpty() const
	{
		return SelectedIDs.IsEmpty();
	}

	bool HasStableEdgeIdentifiers() const
	{
		return !StableEdgeIDs.IsEmpty();
	}

	bool operator==(const FUVToolSelection& Other) const
	{
		return Target == Other.Target
			&& Type == Other.Type
			&& SelectedIDs.Num() == Other.SelectedIDs.Num()
			// Don't need to check the reverse because we checked Num above.
			&& SelectedIDs.Includes(Other.SelectedIDs); 
	}

	bool operator!=(const FUVToolSelection& Other) const
	{
		return !(*this == Other);
	}

	UE_API void SaveStableEdgeIdentifiers(const FDynamicMesh3& Mesh);

	UE_API void RestoreFromStableEdgeIdentifiers(const FDynamicMesh3& Mesh);

	UE_API bool AreElementsPresentInMesh(const FDynamicMesh3& Mesh) const;

	UE_API FAxisAlignedBox3d ToBoundingBox(const FDynamicMesh3& Mesh, const FTransform Transform = FTransform3d::Identity) const;

	/**
	 * Utility method to construct a new selection from an existing reference selection, only with a new geometry type.
	 */
	UE_API FUVToolSelection GetConvertedSelection(const FDynamicMesh3& Mesh, FUVToolSelection::EType ExpectedSelectionType) const;

	/** 
	* Utility method to construct a new selection from an existing reference Applied mesh selection, only for the equivalent Unwrap mesh geometry.
	* 
	* If the target for this selection is currently invalid or unset, this will return an empty selection.
	*/
	UE_API FUVToolSelection GetConvertedSelectionForUnwrappedMesh() const;

	/**
	* Utility method to construct a new selection from an existing reference Unwrap mesh selection, only for the equivalent Applied mesh geometry.
	*
	* If the target for this selection is currently invalid or unset, this will return an empty selection.
	*/
	UE_API FUVToolSelection GetConvertedSelectionForAppliedMesh() const;

	/**
	 * Utility method to convert element ids from those belonging to the unwrapped mesh to those belonging to equivalent ids on the applied mesh
	 * 
	 * @param SelectionMode The type of element the input ids represent
	 * @param UnwrapMesh A dynamic mesh instance representing the unwrapped mesh which the input ids belong to 
	 * @param AppliedMesh A dynamic mesh instance representing the applied mesh for which the resulting ids will belong to
	 * @param UVOverlay An overlay representing the current UV mapping linking the unwrapped and applied mesh
	 * @param IDsIn The ids of the elements from the unwrapped mesh to convert to applied element ids
	 */
	static UE_API TArray<int32> ConvertUnwrappedElementIdsToAppliedElementIds(EType SelectionMode, const FDynamicMesh3& UnwrapMesh,
		const FDynamicMesh3& AppliedMesh, const FDynamicMeshUVOverlay& UVOverlay, const TArray<int32>& IDsIn);

	/**
	 * Utility method to convert element ids from those belonging to the applied mesh to those belonging to equivalent ids on the unwrapped mesh
	 *
	 * @param SelectionMode The type of element the input ids represent
 	 * @param AppliedMesh A dynamic mesh instance representing the applied mesh which the input ids belong to
	 * @param UnwrapMesh A dynamic mesh instance representing the unwrapped mesh for which the resulting ids will belong to
	 * @param UVOverlay An overlay representing the current UV mapping linking the unwrapped and applied mesh
	 * @param IDsIn The ids of the elements from the unwrapped mesh to convert to applied element ids
	 * @param AppliedOnlyElementIds The ids of any remaining elements that have no direct mappings, due to unset UVs, in the unwrapped mesh
	 */
	static UE_API TArray<int32> ConvertAppliedElementIdsToUnwrappedElementIds(EType SelectionMode, const FDynamicMesh3& AppliedMesh,
		const FDynamicMesh3& UnwrapMesh, const FDynamicMeshUVOverlay& UVOverlay, const TArray<int32>& IDsIn, TArray<int32>& AppliedOnlyElementIds);

protected:
	
	FMeshEdgesFromTriangleSubIndices StableEdgeIDs;
};

}} //end namespace UE::Geometry

#undef UE_API
