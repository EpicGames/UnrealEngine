// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ContextObjects/UVToolContextObjects.h"
#include "Selection/UVToolSelection.h"

#include "UVToolSelectionAPI.generated.h"

#define UE_API UVEDITORTOOLS_API

class UUVEditorMechanicAdapterTool;
class UUVEditorMeshSelectionMechanic;
class UUVToolSelectionHighlightMechanic;

/*
 *  Enum to describe the nature of the selection change during either a OnPreSelectionChange callback or
 *  OnSelectionChange callback.
 */
enum class ESelectionChangeTypeFlag : uint8
{
	None = 0,
	SelectionChanged = 1 << 0, // Set for normal selection change events
	UnsetSelectionChanged = 1 << 1 // Set for unset selection change events
};
ENUM_CLASS_FLAGS(ESelectionChangeTypeFlag);

/**
 * API for dealing with mode-level selection in the UV editor.
 * 
 * Selections are stored in a list of objects, one object per asset that contains a
 * selection, all of the same type (vert/edge/tri), none empty. Selections are
 * considered to be referring to the UnwrapCanonical mesh of the corresponding target.
 * 
 * There are also functions to enable automatic highlighting of the current selection,
 * and to enable a selection mechanic in the viewport (to which tools can respond via
 * OnSelectionChanged broadcasts).
 */
UCLASS(MinimalAPI)
class UUVToolSelectionAPI : public UUVToolContextObject
{
	GENERATED_BODY()
public:
	using FUVToolSelection = UE::Geometry::FUVToolSelection;

	/**
	 * Returns true when current selection is not empty.
	 */
	bool HaveSelections() const { return CurrentSelections.Num() > 0; }

	/**
	 * Returns the current selection. When there are multiple selection objects, it means that there
	 * are selections in multiple assets, with a separate selection object per asset containing
	 * a selection. In such a case, all objects will have the same type (vertex/edge/triangle).
	 * 
	 * Each selection object is considered to be referring to the CanonicalUnwrap mesh of the
	 * associated target.
	 */
	const TArray<FUVToolSelection>& GetSelections() const { return CurrentSelections; }

	/**
	 * Returns type (vertex/edge/triangle) of current selection. Undefined if selection is empty.
	 */
	FUVToolSelection::EType GetSelectionsType()
	{
		if (CurrentSelections.Num() > 0)
		{
			return CurrentSelections[0].Type;
		}
		else if (CurrentUnsetSelections.Num() > 0)
		{
			return CurrentUnsetSelections[0].Type;
		}
		else {
			return FUVToolSelection::EType::Triangle;
		}		
	}

	/**
	 * Sets the current selection. Selections should be same type, with no more than one selection
	 * object per asset, and no empty selection objects. Selections should be referring to the
	 * UnwrapCanonical of the associated target.
	 * 
	 * @param bBroadcast If true, broadcast OnPreSelectionChange and OnSelectionChanged
	 * @param bEmitChange If true, emit an undo/redo transaction.
	 * @param bAllowInconsistentSelectionTypes If true, allow setting selections with different selection types than any currently set UnsetElement selections.
	          Primarily useful during undo/redo
	 */
	UE_API void SetSelections(const TArray<FUVToolSelection>& SelectionsIn, bool bBroadcast, bool bEmitChange);

	/**
	 * Equivalent to calling SetSelections with an empty array.
	 */
	UE_API void ClearSelections(bool bBroadcast, bool bEmitChange);

	/**
	 * Gets the bounding box center of the current selection. The center point is cached and
	 * invalidated on the next SetSelection call, but can be forced to be
	 * recalculated.
	 * 
	 * Note: you should call with bForceRecalculate = true whenever calling this from OnCanonicalModified,
	 * because it is not guaranteed that the selection api will detect the mesh change (and therefore
	 * invalidate the cached value) before your call site, due to the order of OnCanonicalModified broadcasts
	 */
	UE_API FVector3d GetUnwrapSelectionBoundingBoxCenter(bool bForceRecalculate = false);

	// These are functions that are used if a selection made in the applied mesh cannot be
	// converted to an unwrap mesh selection due to unset UV elements (meaning that the selected
	// portion does not exist in the unwrap mesh). Such selections are stored separately 
	// from all those that are able to be converted (which get stored with the usual unwrap selections).
	// These "unset element" selections are ignored by many tools/actions but operated on 
	// by some others.
	UE_API bool HaveUnsetElementAppliedMeshSelections() const;
	const TArray<FUVToolSelection>& GetUnsetElementAppliedMeshSelections() { return CurrentUnsetSelections; }
	UE_API void SetUnsetElementAppliedMeshSelections(const TArray<FUVToolSelection>& SelectionsIn, bool bBroadcast, bool bEmitChange);
	UE_API void ClearUnsetElementAppliedMeshSelections(bool bBroadcast, bool bEmitChange);

	/**
	 * Mode of operation for the selection mechanic.
	 */
	enum class EUVEditorSelectionMode
	{
		// When None, means that mechanic will not select anything
		None,

		Vertex,
		Edge,
		Triangle,
		Island,
		Mesh,
	};

	// Selection mechanic controls:

	/**
	 * Most clients shouldn't need to touch the selection mechanic directly, but it can
	 * be useful for certain utility functions that exist on the mechanic.
	 */
	const UUVEditorMeshSelectionMechanic* GetSelectionMechanic() const { return SelectionMechanic; }

	UE_API void SetSelectionMechanicEnabled(bool bEnabled);

	struct FSelectionMechanicOptions
	{
		bool bShowHoveredElements = true;
	};
	UE_API void SetSelectionMechanicOptions(const FSelectionMechanicOptions& Options);

	// Options for how a SetSelectionMechanicMode() call is performed
	struct FSelectionMechanicModeChangeOptions
	{
		/**
		 * If true, any existing selection will be converted to be compatible with the new mode.
		 * For instance if an existing selection is vertices and we switch to Island selection
		 * mode, the vertex selection will be converted to a triangle selection.
		 */
		bool bConvertExisting = true;

		/**
		 * If true and a conversion is performed (requires bConvertExisting to be true), the
		 * OnPreSelectionChanged and OnSelectionChanged delegates will be broadcast.
		 */
		bool bBroadcastIfConverted = true;

		/** If true, emit appropriate undo / redo transactions. */
		bool bEmitChanges = true;

		// Shouldn't really need this constructor but there seems to be a clang bug that requires it if we
		// want to use a default-constructed instance as a default arg as we do below.
		FSelectionMechanicModeChangeOptions() {}
	};
	/**
	 * Sets the mechanic mode of operation.
	 */
	UE_API void SetSelectionMechanicMode(EUVEditorSelectionMode Mode, 
		const FSelectionMechanicModeChangeOptions& Options = FSelectionMechanicModeChangeOptions());
	
	// Highlighting controls
	
	/**
	 * Changes the visibility of the highlight.
	 */
	UE_API void SetHighlightVisible(bool bUnwrapHighlightVisible, bool bAppliedHighlightVisible, bool bRebuild = true);

	struct FHighlightOptions
	{
		// When true, highlighting is based off of the preview meshes rather than the
		// canonical meshes.
		bool bBaseHighlightOnPreviews = false;

		// When true, SetSelection calls will trigger a rebuild of the applied highlight
		bool bAutoUpdateApplied = false;

		// When true, SetSelection calls will trigger a rebuild of the unwrap highlight
		bool bAutoUpdateUnwrap = false;

		// If unwrap highlights are automatically updated, the start transform of these
		// will be set to the centroid of the selection when the below is true. This
		// requires centroid calculation but allows the highlight to be easily translated
		// in response to gizmo movement.
		bool bUseCentroidForUnwrapAutoUpdate = true;

		// When building the unwrap highlight, show each edge and the paired edge to which 
		// it can be welded (in different colors).
		bool bShowPairedEdgeHighlights = false;
	};
	UE_API void SetHighlightOptions(const FHighlightOptions& Options);

	UE_API void ClearHighlight(bool bClearForUnwrap = true, bool bClearForAppliedPreview = true);

	/**
	 * Build up a highlight of the current selection in the Unwrap, with a given transform
	 * considered as its start transform.
	 */
	UE_API void RebuildUnwrapHighlight(const FTransform& StartTransform);

	/**
	 * Change the transform of the unwrap highlight without rebuilding it (for cheap movement
	 * of the highlight when translating the elements). Note that unmoved paired edges (if 
	 * paired edge highlighting is enabled) will still be rebuilt since their shape may be
	 * changed by the movement of adjacent edges.
	 */
	UE_API void SetUnwrapHighlightTransform(const FTransform& NewTransform);

	UE_API FTransform GetUnwrapHighlightTransform();

	UE_API void RebuildAppliedPreviewHighlight();

	/**
	 * Broadcasted right before a selection change is applied (and therefore before a selection change
	 * transaction is emitted). Useful if a user wants to emit their own bookend transaction first.
	 * 
	 * @param bEmitChangeAllowed If false, the callback must not emit any undo/redo transactions,
	 *  likely because this is being called from an apply/revert of an existing transaction.
	 * @param SelectionChangeType Set as some combination of ESelectionChangeCondition to describe
	 *  the nature of the selection change.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPreSelectionChange, bool bEmitChangeAllowed, uint32 SelectionChangeType );
	FOnPreSelectionChange OnPreSelectionChange;

	/**
	 * Broadcasted after a selection change is applied (after the selection change transaction is
	 * emitted, if relevant).
	 * 
	 * @param bEmitChangeAllowed If false, the callback must not emit any undo/redo transactions,
	 *  likely because this is being called from an apply/revert of an existing transaction.
	 * @param SelectionChangeType Set as some combination of ESelectionChangeCondition to describe
	 *  the nature of the selection change.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSelectionChanged, bool bEmitChangeAllowed, uint32 SelectionChangeType);
	FOnSelectionChanged OnSelectionChanged;

	/**
	 * Broadcasted whenever the marquee rectangle is changed, since these changes don't trigger
	 * normal selection broadcasts.
	 */
	FSimpleMulticastDelegate OnDragSelectionChanged;


	// Undo/redo support:

	/**
	 * An object representing a selection that can be as an undo/redo item, usually emitted by the selection API
	 * itself. Expects UUVToolSelectionAPI to be the associated UObject.
	 */
	class FSelectionChange : public FToolCommandChange
	{
	public:
		/**
		 * When true, edge-type selections are not stored as eids, but rather as stable identifiers 
		 * relative to the UnwrapCanonical mesh in the Target of the selection. This keeps them
		 * from being invalidated when other transactions edit the mesh in a way that changes
		 * Eids without actually changing topology.
		 */
		bool bUseStableUnwrapCanonicalIDsForEdges = true;

		virtual void SetBefore(TArray<FUVToolSelection> Selections);
		virtual void SetAfter(TArray<FUVToolSelection> Selections);

		// Useful for storing a pending change and figuring out whether it needs to be
		// emitted. Respects bUseStableUnwrapCanonicalIDsForEdges.
		virtual TArray<FUVToolSelection> GetBefore() const;

		virtual bool HasExpired(UObject* Object) const override;
		virtual void Apply(UObject* Object) override;
		virtual void Revert(UObject* Object) override;
		virtual FString ToString() const override;

	protected:
		TArray<FUVToolSelection> Before;
		TArray<FUVToolSelection> After;
	};

	/**
	 * An object representing an unset UV selection that can be as an undo/redo item, usually emitted by the selection API
	 * itself. Expects UUVToolSelectionAPI to be the associated UObject.
	 */
	class FUnsetSelectionChange : public FToolCommandChange
	{
	public:
		virtual void SetBefore(TArray<FUVToolSelection> Selections);
		virtual void SetAfter(TArray<FUVToolSelection> Selections);

		virtual const TArray<FUVToolSelection>& GetBefore() const;

		virtual bool HasExpired(UObject* Object) const override;
		virtual void Apply(UObject* Object) override;
		virtual void Revert(UObject* Object) override;
		virtual FString ToString() const override;

	protected:
		TArray<FUVToolSelection> Before;
		TArray<FUVToolSelection> After;
	};

	// Returns true in between Initialize() and Shutdown(). Used by undo/redo to expire transactions
	// when editor is closed.
	bool IsActive() const { return bIsActive; }

	/**
	 * Preps a selection change transaction, if the user wants more control on what the previous and
	 * current selection is.
	 */
	UE_API void BeginChange();

	/**
	 * Ends the active change and emits it via the EmitChangeAPI
	 */
	UE_API bool EndChangeAndEmitIfModified(bool bBroadcast);

	// Initialization functions:
	UE_API void Initialize(UInteractiveToolManager* ToolManagerIn, UWorld* UnwrapWorld,
		UInputRouter* UnwrapInputRouterIn, UUVToolLivePreviewAPI* LivePreviewAPI,
		UUVToolEmitChangeAPI* EmitChangeAPIIn);
	// Should be called after Initialize()
	UE_API void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn);

	// Called by the owner of the API
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI);
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);
	// Could have had the api place these into the LivePreviewAPI delegates, but decided against
	// it to align with Render() and DrawHUD() above. The owner will just call these directly.
	UE_API virtual void LivePreviewRender(IToolsContextRenderAPI* RenderAPI);
	UE_API virtual void LivePreviewDrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	// UUVToolContextObject
	UE_API virtual void Shutdown() override;
	UE_API virtual void OnToolEnded(UInteractiveTool* DeadTool) override;

	// Holds the properties of the UV Live Preview Selection (in 3d viewport) for customization
	struct FLivePreviewSelectionUXSettings
	{
		TOptional<FColor> SelectionColor;
		TOptional<float> LineThickness;
		TOptional<float> PointSize;
	};

	// Sets the UV Live Preview Selection settings
	UE_API void SetLivePreviewSelectionUXSettings(const FLivePreviewSelectionUXSettings& Settings) const;

protected:
	TArray<FUVToolSelection> CurrentSelections;
	TArray<FUVToolSelection> CurrentUnsetSelections;

	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	TWeakObjectPtr<UInputRouter> UnwrapInputRouter = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorMechanicAdapterTool> MechanicAdapter = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolSelectionHighlightMechanic> HighlightMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorMeshSelectionMechanic> SelectionMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolEmitChangeAPI> EmitChangeAPI = nullptr;

	TUniquePtr<FSelectionChange> PendingSelectionChange = nullptr;
	TUniquePtr<FUnsetSelectionChange> PendingUnsetSelectionChange = nullptr;


	FHighlightOptions HighlightOptions;

	FVector3d CachedUnwrapSelectionBoundingBoxCenter;
	bool bCachedUnwrapSelectionBoundingBoxCenterValid = false;

	bool bIsActive = false;
};

// UInterface for IUVToolSupportsSelection
UINTERFACE(MinimalAPI)
class UUVToolSupportsSelection : public UInterface
{
	GENERATED_BODY()
};

/**
 * If a tool does not inherit from IUVToolSupportsSelection, then selection
 * will automatically be cleared before the tool invocation via an undoable 
 * transaction, to avoid a state where the selection refers to invalid items
 * after tool completion.
 * If a tool does inherit from IUVToolSupportsSelection, then the UV editor will
 * not clear the selection before invocation, allowing the tool to use it. However,
 * the tool is expected to properly deal with selection itself and avoid an
 * invalid state (including avoiding incorrect undo/redo selection event ordering.
 */
class IUVToolSupportsSelection
{
	GENERATED_BODY()

public:

	/*
	 * A tool that returns false here will have the unset element selections 
	 * (ie selections of triangles that don't have associated UV's) cleared before
	 * its invocation. Returning true will keep these unset element selections, but
	 * the tool will be responsible for dealing with them and avoiding any invalid states.
	 */
	virtual bool SupportsUnsetElementAppliedMeshSelections() const { return false; }

};

#undef UE_API
