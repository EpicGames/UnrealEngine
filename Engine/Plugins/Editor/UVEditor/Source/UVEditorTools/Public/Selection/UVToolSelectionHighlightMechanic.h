// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GeometryBase.h"
#include "InteractionMechanic.h"

#include "UVToolSelectionHighlightMechanic.generated.h"

#define UE_API UVEDITORTOOLS_API

PREDECLARE_GEOMETRY(class FDynamicMesh3);
PREDECLARE_GEOMETRY(class FUVToolSelection);

class APreviewGeometryActor;
class UMaterialInstanceDynamic;
class UBasic2DPointSetComponent;
class UBasic3DPointSetComponent;
class UBasic2DLineSetComponent;
class UBasic3DLineSetComponent;
class UBasic2DTriangleSetComponent;
class UUVEditorToolMeshInput;
class UWorld;

/**
 * Mechanic for highlighting mesh elements (usually selection) in the UV editor.
 */
UCLASS(MinimalAPI)
class UUVToolSelectionHighlightMechanic : public UInteractionMechanic
{
	GENERATED_BODY()

	using FUVToolSelection = UE::Geometry::FUVToolSelection;

public:

	UE_API void Initialize(UWorld* UnwrapWorld, UWorld* LivePreviewWorld);
	UE_API void Shutdown() override;

	UE_API void SetIsVisible(bool bUnwrapHighlightVisible, bool bLivePreviewHighlightVisible);

	/**
	 * Rebuilds the unwrap mesh highlights.
	 * 
	 * @param StartTransform The transform to consider as the start transform, so that the highlights
	 *   can be repositioned easily later via SetUnwrapHighlightTransform().
	 * @param bUsePreviews If true, the selected element positions will be gotten from the UnwrapPreview
	 *   of each selection target rather than UnwrapCanonical.
	 */
	UE_API void RebuildUnwrapHighlight(const TArray<FUVToolSelection>& Selections, 
		const FTransform& StartTransform, bool bUsePreviews = false);

	/**
	 * Set the transform of the unwrap mesh highlights without rebuilding them, likely to 
	 * follow the translation of the underlying elements via gizmo.
	 * 
	 * @param bRebuildStaticPairedEdges If true, and paired edge highlighting is enabled,
	 *   unselected paired edges are rebuilt in the highlight. This is usually necessary
	 *   because such edges may be attached to moving selected edges.
	 * @param bUsePreviews If bRebuildStaticPairedEdges is true, this will prefer to use the
	 *   previews rather than the unwrap canonical.
	 */
	UE_API void SetUnwrapHighlightTransform(const FTransform& Transform, bool bRebuildStaticPairedEdges = true, 
		bool bUsePreviews = false);

	/**
	 * Get the current transform of the unwrap highlight.
	 */
	UE_API FTransform GetUnwrapHighlightTransform();

	/**
	 * Convert the unwrap selections to elements in the applied mesh, then highlight those there.
	 * 
	 * @param bUsePreviews If true, the previews are used rather than the canonicals both for converting
	 *   over to applied mesh elements and to finding the highlight positions.
	 */
	UE_API void RebuildAppliedHighlightFromUnwrapSelection(const TArray<FUVToolSelection>& UnwrapSelections, bool bUsePreviews = false);

	/**
	 * Adds additional highlighting elements based on selection information. This is used to extend selection
	 * line sets for the unset selections if they are needed. 
	 *
	 * @param bUsePreviews If true, the previews are used rather than the canonicals for finding the highlight positions.
	 */
	UE_API void AppendAppliedHighlight(const TArray<FUVToolSelection>& AppliedSelections, bool bUsePreviews = false);

	/**
	 * When true, edge selections also highlight the edge pairings (i.e. edges that could weld with the
	 * selected edges). Note that the call itself controls visibility and the next rebuild call, but 
	 * won't cause an immediate rebuild on its own.
	 */
	UE_API void SetEnablePairedEdgeHighlights(bool bEnable);

	/**
	 * Directly set the Color property of the selection line and point set.
	 */
	UE_API void SetColor(const FColor Color) const;

	/**
	 * Directly set the Size/Thickness property of the selection line set.
	 */
	UE_API void SetLineThickness(const float LineSize) const;

	/**
	 * directly set the Size property of the selection point set.
	 */
	UE_API void SetPointSize(const float PointSize) const;

protected:

	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> UnwrapGeometryActor = nullptr;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> TriangleSetMaterial = nullptr;

	TWeakObjectPtr<UBasic2DTriangleSetComponent> UnwrapTriangleSet = nullptr;
	TWeakObjectPtr<UBasic2DLineSetComponent> UnwrapLineSet = nullptr;
	TWeakObjectPtr<UBasic2DLineSetComponent> SewEdgePairingLeftLineSet = nullptr;
	TWeakObjectPtr<UBasic2DLineSetComponent> SewEdgePairingRightLineSet = nullptr;
	TWeakObjectPtr<UBasic2DPointSetComponent> UnwrapPointSet = nullptr;
	TWeakObjectPtr<UBasic2DLineSetComponent> UnwrapPairedEdgeLineSet = nullptr;

	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> UnwrapStationaryGeometryActor = nullptr;
	TWeakObjectPtr<UBasic2DLineSetComponent> SewEdgeUnselectedPairingLineSet = nullptr;

	// Per mesh, gives a list of endpoint vid pairs for the unselected edges that
	// are part of edge pairs, for rebuilding when necessary.
	TArray<TPair<TWeakObjectPtr<UUVEditorToolMeshInput>,
		TArray<TPair<int32, int32>>>> StaticPairedEdgeVidsPerMesh;

	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> LivePreviewGeometryActor = nullptr;
	TWeakObjectPtr<UBasic3DLineSetComponent> LivePreviewLineSet = nullptr;
	TWeakObjectPtr<UBasic3DPointSetComponent> LivePreviewPointSet = nullptr;

	bool bPairedEdgeHighlightsEnabled = true;
};

#undef UE_API
