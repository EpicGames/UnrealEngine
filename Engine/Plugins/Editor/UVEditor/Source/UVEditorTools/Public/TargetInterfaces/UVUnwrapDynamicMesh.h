// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GeometryBase.h"
#include "UObject/Interface.h"

#include "UVUnwrapDynamicMesh.generated.h"

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

UINTERFACE(MinimalAPI)
class UUVUnwrapDynamicMesh : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that is able to use the UV's of a target to generate a flat FDynamicMesh
 * representing a given layer, and to bake a modified such mesh back to the UV's of
 * the target provided that the triangle id's have all stayed the same.
 */
class IUVUnwrapDynamicMesh
{
	GENERATED_BODY()

public:
	virtual int32 GetNumUVLayers() = 0;
	virtual TSharedPtr<FDynamicMesh3> GetMesh(int32 LayerIndex) = 0;
	virtual void SaveBackToUVs(const FDynamicMesh3* MeshToSave, int32 LayerIndex) = 0;
};
