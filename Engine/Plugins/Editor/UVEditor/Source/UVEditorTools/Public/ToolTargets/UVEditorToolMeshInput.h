// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h" // FDynamicMeshUVOverlay
#include "Components/BaseDynamicMeshComponent.h" // EMeshRenderAttributeFlags
#include "ToolTargets/ToolTarget.h"
#include "VectorTypes.h"

#include "GeometryBase.h"

#include "UVEditorToolMeshInput.generated.h"

#define UE_API UVEDITORTOOLS_API

class UMaterialInterface;
class UMeshOpPreviewWithBackgroundCompute;
class UMeshElementsVisualizer;
PREDECLARE_GEOMETRY(class FDynamicMesh3);
PREDECLARE_GEOMETRY(class FDynamicMeshChange);

/**
 * A package of the needed information for an asset being operated on by a
 * UV editor tool. It includes a UV unwrap mesh, a mesh with the UV layer applied,
 * and background-op-compatible previews for each. It also has convenience methods
 * for updating all of the represenations from just one of them, using a "fast update"
 * code path when possible.
 * 
 * This tool target is a bit different from usual in that it is not created
 * by a tool target manager, and therefore doesn't have an accompanying factory.
 * Instead, it is created by the mode, because the mode has access to the worlds
 * in which the previews need to be created.
 * 
 * It's arguable whether this should even inherit from UToolTarget.
 */
UCLASS(MinimalAPI)
class UUVEditorToolMeshInput : public UToolTarget
{
	GENERATED_BODY()

public:

	/** 
	 * Mesh representing the unwrapped UV layer. If the UnwrapPreview is changed via background
	 * ops, then this mesh can be used to restart an operation as parameters change. Once a change
	 * is completed, this mesh should be updated (i.e., it is the "canonical" unwrap mesh, though
	 * the final UV layer truth is in the UV's of AppliedCanonical).
	 */
	TSharedPtr<UE::Geometry::FDynamicMesh3> UnwrapCanonical;

	/**
	 * Preview of the unwrapped UV layer, suitable for being manipulated by background ops.
	 */
	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> UnwrapPreview = nullptr;

	// Note that both UnwrapCanonical and UnwrapPreview, besides having vert positions that represent
	// a UV layer, also have a primary UV overlay that represents the same layer, to make it possible
	// to someday texture the unwrap. This UV overlay will differ from the UV overlays in AppliedCanonical
	// and AppliedPreview only in the parent pointers of its elements, since there will not be any elements 
	// pointing back to the same vertex. The element id's and the triangles will be the same.

	/** 
	 * A 3d mesh with the UV layer applied. This is the canonical result that will be baked back
	 * on the application of changes. It can also be used to reset background ops that may operate
	 * on AppliedPreview.
	 */
	TSharedPtr<UE::Geometry::FDynamicMesh3> AppliedCanonical;

	/**
	 * 3d preview of the asset with the UV layer updated, suitable for use with background ops. 
	 */
	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> AppliedPreview;

	/**
	 * Optional: a wireframe to track the mesh in unwrap preview. If set, it gets updated whenever the
	 * class updates the unwrap preview, and it is destroyed on Shutdown().
	 * TODO: We should have a fast path for updating the wireframe...
	 */
	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> WireframeDisplay = nullptr;

	/**
	* When enabled, this causes the unwrap preview to recieve updates to the vertex color buffers when mesh updates occur.
	*/
	UPROPERTY()
	bool bEnableTriangleVertexColors = false;

	// Information about an OnCanonicalModified broadcast. This is a struct so that we can
	// add more information if we someday need to.
	struct FCanonicalModifiedInfo
	{
		// It's not yet clear whether we need either of these. bAppliedOverlayChanged seems
		// like it would be always true whenever a change is emitted. bUnwrapMeshShapeChanged would
		// seeminly only be false if we edited some other layer in the applied canonical without
		// displaying it. That is rare and it would probably be safer to just do a full update
		// in most of those cases (i.e. still assume that unwrap changed).
		// So for now we don't use them.
		//bool bUnwrapMeshShapeChanged = true;
		//bool bAppliedOverlayChanged = true;

		// This would only be true if we udpated the mesh after editing it externally, since
		// the UV editor doesn't change mesh shape.
		bool bAppliedMeshShapeChanged = false;
	};

	/**
	 * Broadcast when the canonical unwrap or applied meshes change. This gets broadcast by
	 * utility functions in this class that update those meshes (except for 
	 * UpdateUnwrapCanonicalOverlayFromPositions, which is usually followed by a more complete
	 * update). If those utility functions are not used and the client updates one of those meshes,
	 * the client should broadcast the change themselves so that important related info can be 
	 * updated (for instance, so that the mode can mark those objects as modified).
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnObjectModified, UUVEditorToolMeshInput* InputObject, const FCanonicalModifiedInfo&);
	FOnObjectModified OnCanonicalModified;

	// Additional needed information
	TObjectPtr<UToolTarget> SourceTarget = nullptr;
	int32 AssetID = -1; // Index into targets array
	int32 UVLayerIndex = 0;

	// Mappings used for generating and baking back the unwrap.
	TFunction<FVector3d(const FVector2f&)> UVToVertPosition;
	TFunction<FVector2f(const FVector3d&)> VertPositionToUV;

	UE_API bool InitializeMeshes(UToolTarget* Target, 
		TSharedPtr<UE::Geometry::FDynamicMesh3> AppliedCanonicalIn,
		UMeshOpPreviewWithBackgroundCompute* AppliedPreviewIn,
		int32 AssetID, int32 UVLayerIndex,
		UWorld* UnwrapWorld, UWorld* LivePreviewWorld,
		UMaterialInterface* WorkingMaterialIn,
		TFunction<FVector3d(const FVector2f&)> UVToVertPositionFuncIn,
		TFunction<FVector2f(const FVector3d&)> VertPositionToUVFuncIn);

	UE_API void Shutdown();

	// Notes above the below convenience functions:
	// 1. Passing nullptr for ChangedVids/ChagnedElementIDs/ChangedConnectivityTids means that all vids/elements/tids
	//  need updating, respectively. Passing UUVEditorToolMeshInput::NONE_CHANGED_ARG is equivalent to passing a pointer 
	//  to an empty array, and means that none changed. Otherwise only the vids/elements/tids in the pointed-to arrays
	//  are iterated over in the updated mesh.
	//  While passing these allows updates to be quicker, note that failing to include the relevant tids/vids/elements
	//  can put the input object into an invalid state or hit a check, for instance if a triangle refers to a new vid
	//  that was not added in.
	// 2. ChangedVids/ChangedElementIDs is allowed to have new elements, since it is natural to split UVs. However, 
	//  ChangedConnectivityTids must not have new Tids, because the tids form our correspondence between the unwrap mesh and
	//  the original layer.
	//  Also note that if you are gathering the changed vids/elements based off of triangles, you should gather them
	//  from the post-change mesh/overlay so that you do not miss any added elements, and since removed elements are
	//  captured by changed tri connectivity.
	// 3. FastRenderUpdateTids is an optional list of triangles whose render data needs updating, which can allow for faster preview
	//  updates if the render buffers are properly split apart. If provided, it should be a superset of ChangedConnectivityTids,
	//  and should contain the one-ring triangles of ChangedVids.
	// 4. If updating the preview objects, note that the functions do not try to cancel any active computations, so an active
	//  computation could reset things once it completes.

	// Can be passed in as a ChangedVids/ChangedElements/ChangedConnectivityTids arguments as an equivalent of
	// passing a pointer to an empty array.
	static UE_API const TArray<int32>* const NONE_CHANGED_ARG;

	/**
	 * Updates UnwrapPreview UV Overlay from UnwrapPreview vert positions. Issues a NotifyDeferredEditCompleted
	 * for both positions and UVs.
	 */
	UE_API void UpdateUnwrapPreviewOverlayFromPositions(const TArray<int32>* ChangedVids = nullptr, 
		const TArray<int32>* ChangedConnectivityTids = nullptr, const TArray<int32>* FastRenderUpdateTids = nullptr);

	/**
	 * Updates UnwrapCanonical UV Overlay from UnwrapCanonical vert positions. Issues a NotifyDeferredEditCompleted
	 * for both positions and UVs.
	 * 
	 * Doesn't broadcast because it is expected that this call is followed by another call that updates the rest of 
	 * the input object.
	 */
	UE_API void UpdateUnwrapCanonicalOverlayFromPositions(const TArray<int32>* ChangedVids = nullptr, 
		const TArray<int32>* ChangedConnectivityTids = nullptr);

	/**
	 * Updates the AppliedPreview from UnwrapPreview, without updating the non-preview meshes. Useful for updates during
	 * a drag, etc, when we only care about updating the visible items.
	 * Assumes that the overlay in UnwrapPreview is already updated.
	 */
	UE_API void UpdateAppliedPreviewFromUnwrapPreview(const TArray<int32>* ChangedVids = nullptr, 
		const TArray<int32>* ChangedConnectivityTids = nullptr, const TArray<int32>* FastRenderUpdateTids = nullptr);

	/**
	 * Updates only the UnwrapPreview from AppliedPreview, without updating the non-preview meshes. Useful for transient
	 * updates when we only care about updating the visible items.
	 */
	UE_API void UpdateUnwrapPreviewFromAppliedPreview(
		const TArray<int32>* ChangedElementIDs = nullptr, const TArray<int32>* ChangedConnectivityTids = nullptr, 
		const TArray<int32>* FastRenderUpdateTids = nullptr);

	/**
	 * Updates the non-preview meshes from their preview counterparts. Useful, for instance, after the completion of a drag
	 * to update the canonical objects.
	 */
	UE_API void UpdateCanonicalFromPreviews(const TArray<int32>* ChangedVids = nullptr, 
		const TArray<int32>* ChangedConnectivityTids = nullptr, bool bBroadcast = true);

	/**
	 * Updates the preview meshes from their canonical counterparts. Useful mainly as a way to reset the previews.
	 */
	UE_API void UpdatePreviewsFromCanonical(const TArray<int32>* ChangedVids = nullptr, const TArray<int32>* ChangedConnectivityTids = nullptr,
		const TArray<int32>* FastRenderUpdateTids = nullptr);

	/**
	 * Updates the other meshes using the mesh in UnwrapPreview. Assumes that the overlay in UnwrapPreview is updated.
	 * Used to update everything when the applied preview was not being updated in tandem with the unwrap preview (otherwise one 
	 * would use UpdateCanonicalFromPreviews).
	 */
	UE_API void UpdateAllFromUnwrapPreview(const TArray<int32>* ChangedVids = nullptr, const TArray<int32>* ChangedConnectivityTids = nullptr, 
		const TArray<int32>* FastRenderUpdateTids = nullptr, bool bBroadcast = true);

	/**
	 * Updates the other meshes using the mesh in UnwrapCanonical. Useful when immediately applying an operation that
	 * operates directly on the unwrap mesh.
	 */
	UE_API void UpdateAllFromUnwrapCanonical(const TArray<int32>* ChangedVids = nullptr, const TArray<int32>* ChangedConnectivityTids = nullptr, 
		const TArray<int32>* FastRenderUpdateTids = nullptr, bool bBroadcast = true);

	/**
	 * Updates the other meshes using the mesh in AppliedCanonical. Useful when switching UV layer indices or otherwise
	 * resetting the collection from the "original" data.
	 */
	UE_API void UpdateAllFromAppliedCanonical(const TArray<int32>* ChangedVids = nullptr, const TArray<int32>* ChangedConnectivityTids = nullptr, 
		const TArray<int32>* FastRenderUpdateTids = nullptr, bool bBroadcast = true);

	/**
	 * Updates the other meshes using the UV overlay in the live preview.
	 */
	UE_API void UpdateAllFromAppliedPreview(const TArray<int32>* ChangedElementIDs = nullptr, 
		const TArray<int32>* ChangedConnectivityTids = nullptr, const TArray<int32>* FastRenderUpdateTids = nullptr, bool bBroadcast = true);

	/**
	 * Uses the stored triangles/vertices in the mesh change to update everything from the canonical unwrap. 
	 * Assumes the change has already been applied to the canonical unwrap.
	 */
	UE_API void UpdateFromCanonicalUnwrapUsingMeshChange(const UE::Geometry::FDynamicMeshChange& UnwrapCanonicalMeshChange, bool bBroadcast = true);

	/**
	 * Convert a vid in the unwrap mesh to the corresponding vid in the applied mesh (i.e., parent vertex of the
	 * element corresponding to the unwrap vertex).
	 */
	UE_API int32 UnwrapVidToAppliedVid(int32 UnwrapVid);

	/**
	 * Get the unwrap vids corresponding to a given applied vid. These will be multiple if the vertex is
	 * a seam vertex and therefore has multiple UV elements associated with it.
	 */
	UE_API void AppliedVidToUnwrapVids(int32 AppliedVid, TArray<int32>& UnwrapVidsOut);

	/**
	 * Returns whether or not the underlying source ToolTarget is still valid.
	 * This is separated from IsValid, which also checks the ToolTarget, in case
	 * we are interested in the status of just the ToolTarget.
	*/
	UE_API virtual bool IsToolTargetValid() const;

	/**
	 * Returns whether or not the canonical and preview meshes are still valid.
	 * This is separated from IsValid, in case we are interested in the status
	 * of just the canonical and preview meshes.
	 */
	UE_API virtual bool AreMeshesValid() const;

	// UToolTarget
	UE_API virtual bool IsValid() const override;

protected:	

	/** 
	* Returns the currently configured mesh attribute buffer flags enabled on this target for updating the unwrap previews
	*/
	UE_API virtual EMeshRenderAttributeFlags GetUnwrapUpdateAttributeFlags() const;

	/**
	* Returns the currently configured mesh attribute buffer flags enabled on this target for updating the applied previews
	*/
	UE_API virtual EMeshRenderAttributeFlags GetAppliedUpdateAttributeFlags() const;
};

#undef UE_API
