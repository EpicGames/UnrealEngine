// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UVEditorToolBase.h" // IUVEditorGenericBuildableTool
#include "InteractiveTool.h"
#include "Selection/UVEditorMeshSelectionMechanic.h"
#include "Selection/UVToolSelectionAPI.h" // IUVToolSupportsSelection
#include "InteractiveToolQueryInterfaces.h" // IInteractiveToolNestedAcceptCancelAPI

#include "UVEditorBrushSelectTool.generated.h"

#define UE_API UVEDITORTOOLS_API

class UBrushStampIndicator;
class ULocalMouseHoverBehavior;
class ULocalClickDragInputBehavior;

UCLASS()
class UUVEditorBrushSelectToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** 
	 * When true, each drag will clear the existing selection if neither Shift (add to selection) nor
	 *  Ctrl (remove from selection) nor both (toggle selection) are pressed. When false, selection
	 *  will not clear, requiring manual removal from selection using Ctrl.
	 */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bClearSelectionOnEachDrag = true;
	
	/** When true, brush selects whole UV islands instead of individual triangles. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bExpandToIslands = false;
	
	//~ The radius defaults are set up so that after doubling via hotkeys some number of times (bitshift
	//~  amount), we get roughly 1.0 and 100.

	/** Radius of the brush in the 2D UV unwrap view. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Unwrap Radius", 
		ClampMin = "0", ClampMax = "1000000", UIMax = "1"))
	float UnwrapBrushRadius = 1.0f / (1 << 5);

	/** Radius of the brush in the 3D live preview view. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Preview Radius", 
		ClampMin = "0", ClampMax = "1000000", UIMax = "100"))
	float LivePreviewBrushRadius = 100.0f / (1 << 4);
};

//~ We could potentially fold this tool's functionality into the UVSelectTool as a sub activity
//~  of some kind.
/**
 * The brush select tool allows for brush selection of triangles on the unwrap or the live
 *  preview. 
 */
UCLASS(MinimalAPI)
class UUVEditorBrushSelectTool : public UInteractiveTool
	, public IInteractiveToolNestedAcceptCancelAPI
	, public IUVToolSupportsSelection
	, public IUVEditorGenericBuildableTool
{
	GENERATED_BODY()

public:
	
	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	// IInteractiveToolNestedAcceptCancelAPI
	virtual bool SupportsNestedCancelCommand() override { return true; }
	UE_API virtual bool CanCurrentlyNestedCancel() override;
	UE_API virtual bool ExecuteNestedCancelCommand() override;

	// IUVEditorGenericBuildableTool
	virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
	{
		Targets = TargetsIn;
	}

	virtual bool SupportsUnsetElementAppliedMeshSelections() const { return true; }

private:
	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	UPROPERTY()
	TObjectPtr<UUVToolEmitChangeAPI> EmitChangeAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolSelectionAPI> SelectionAPI = nullptr;

	TWeakObjectPtr<const UUVEditorMeshSelectionMechanic> SelectionMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolLivePreviewAPI> LivePreviewAPI = nullptr;

	TWeakObjectPtr<UInputRouter> LivePreviewInputRouter = nullptr;
	UPROPERTY()
	TObjectPtr<UInputBehaviorSet> LivePreviewBehaviorSet = nullptr;
	UPROPERTY()
	TObjectPtr<ULocalInputBehaviorSource> LivePreviewBehaviorSource = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorBrushSelectToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UBrushStampIndicator> UnwrapBrushIndicator = nullptr;

	UPROPERTY()
	TObjectPtr<UBrushStampIndicator> LivePreviewBrushIndicator = nullptr;

	TArray<UUVEditorMeshSelectionMechanic::FRaycastResult> PendingLivePreviewHits;
	// For unwrap, we don't necessarily need to hit the mesh to be able to select things
	//  with the brush- we just need the locations of all the hits.
	TArray<FVector2d> PendingUnwrapHits;

	// Updates internal structures in addition to the actual selection api
	UE_API void ClearSelections(bool bBroadcastAndEmit = false);

	UE_API void ProcessPendingUnwrapHits();
	UE_API void ProcessPendingLivePreviewHits();

	bool bHavePendingUnwrapHit = false;
	bool bHavePendingLivePreviewHit = false;

	int32 ShiftModifierID = 1;
	int32 CtrlModifierID = 2;
	bool bShiftToggle = false;
	bool bCtrlToggle = false;
	bool bCurrentStrokeIsSubtracting = false;

	// TODO: It would be nice to have Ctrl+Shift brush inverting, so that a user could brush
	//  an inverted selection. However that requires a bit of tedium, so we'll leave it as
	//  a todo. We can't just update the selection as we go along in that case- we have to 
	//  store the pre-stroke selection, additively add to a stroke selection, and then use
	//  the two to update the actual selection mechanic. Plus we have to do that both for
	//  unset selection and regular selection. Still it would be nice to add, especially if we add
	//  that to the regular select tool.
	// bool bCurrentStrokeIsInverting = false;
	
	TArray<int32> TempROIBuffer;

	// Currently used just so that the Esc key exits out of the tool instead of clearing
	//  the selection if we haven't yet done anything.
	bool bHaveInteracted = false;

	bool bHoveringUnwrap = false;
	bool bHoveringLivePreview = false;
	bool bDraggingUnwrap = false;
	bool bDraggingLivePreview = false;
	UE_API void UpdateViewportStateFromHoverOrDragEvent(bool bFromUnwrap, bool bIsEndEvent, bool bDragging);
	
	UE_API void IncreaseBrushRadiusAction();
	UE_API void DecreaseBrushRadiusAction();
};

#undef UE_API
