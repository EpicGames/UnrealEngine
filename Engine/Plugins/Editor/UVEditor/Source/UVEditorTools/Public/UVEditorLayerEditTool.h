// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "UVEditorToolAnalyticsUtils.h"

#include "UVEditorLayerEditTool.generated.h"

#define UE_API UVEDITORTOOLS_API

class UUVEditorToolMeshInput;
class UUVEditorChannelEditTool;
class UUVToolEmitChangeAPI;

UCLASS(MinimalAPI)
class UUVEditorChannelEditToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	// This is a pointer so that it can be updated under the builder without
	// having to set it in the mode after initializing targets.
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;
};

UENUM()
enum class EChannelEditToolAction
{
	NoAction,

	Add,
	Copy,
	Delete
};


UCLASS(MinimalAPI)
class UUVEditorChannelEditSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** UV Layer Edit action to preform */
	UPROPERTY(EditAnywhere, Category = Options, meta = (InvalidEnumValues = "NoAction"))
	EChannelEditToolAction Action = EChannelEditToolAction::Add;
};

UCLASS(MinimalAPI)
class UUVEditorChannelEditTargetProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "UVChannels", meta = (DisplayName = "Asset", GetOptions = GetAssetNames, EditCondition = "bActionNeedsAsset", EditConditionHides = true, HideEditConditionToggle = true))
	FString Asset;

	UFUNCTION()
	UE_API const TArray<FString>& GetAssetNames();

	UPROPERTY(EditAnywhere, Category = "UVChannels", meta = (DisplayName = "Target UV Channel", GetOptions = GetUVChannelNames, EditCondition = "bActionNeedsTarget", EditConditionHides = true, HideEditConditionToggle = true))
	FString TargetChannel;

	UPROPERTY(EditAnywhere, Category = "UVChannels", meta = (DisplayName = "Source UV Channel", GetOptions = GetUVChannelNames, EditCondition = "bActionNeedsReference", EditConditionHides = true, HideEditConditionToggle = true))
	FString ReferenceChannel;

	UFUNCTION()
	UE_API const TArray<FString>& GetUVChannelNames();

	TArray<FString> UVChannelNames;
	TArray<FString> UVAssetNames;

	// 1:1 with UVAssetNames
	TArray<int32> NumUVChannelsPerAsset;

	UPROPERTY(meta = (TransientToolProperty))
	bool bActionNeedsAsset = true;

	UPROPERTY(meta = (TransientToolProperty))
	bool bActionNeedsReference = false;

	UPROPERTY(meta = (TransientToolProperty))
	bool bActionNeedsTarget = true;

public:
	UE_API void Initialize(
		const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn,
		bool bInitializeSelection);

	UE_API void SetUsageFlags(EChannelEditToolAction Action);

	/**
	 * Verify that the UV asset selection is valid
	 * @param bUpdateIfInvalid if selection is not valid, reset UVAsset to Asset0 or empty if no assets exist
	 * @return true if selection in UVAsset is an entry in UVAssetNames.
	 */
	UE_API bool ValidateUVAssetSelection(bool bUpdateIfInvalid);

	/**
	 * Verify that the UV channel selection is valid
	 * @param bUpdateIfInvalid if selection is not valid, UVChannel to UV0 or empty if no UV channels exist
	 * @return true if selection in UVChannel is an entry in UVChannelNamesList.
	 */
	UE_API bool ValidateUVChannelSelection(bool bUpdateIfInvalid);


	/**
	 * @return selected UV asset ID, or -1 if invalid selection
	 */
	UE_API int32 GetSelectedAssetID();

	/**
	 * @param bForceToZeroOnFailure if true, then instead of returning -1 we return 0 so calling code can fallback to default UV paths
	 * @param bUseReference if true, get the selected reference channel index, otherwise return the target channel's index.
	 * @return selected UV channel index, or -1 if invalid selection, or 0 if invalid selection and bool bForceToZeroOnFailure = true
	 */
	UE_API int32 GetSelectedChannelIndex(bool bForceToZeroOnFailure = false, bool bUseReference = false);
};

UCLASS(MinimalAPI)
class UUVEditorChannelEditAddProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/* Placeholder for future per action settings */
};

UCLASS(MinimalAPI)
class UUVEditorChannelEditCopyProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/* Placeholder for future per action settings */
};

UCLASS(MinimalAPI)
class UUVEditorChannelEditDeleteProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/* Placeholder for future per action settings */
};

UCLASS(MinimalAPI)
class UUVEditorChannelEditToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UUVEditorChannelEditTool> ParentTool;

	void Initialize(UUVEditorChannelEditTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(EChannelEditToolAction Action);
	
	UFUNCTION(CallInEditor, Category = Actions)
	UE_API void Apply();
};

/**
 *
 */
UCLASS(MinimalAPI)
class UUVEditorChannelEditTool : public UInteractiveTool
{
	GENERATED_BODY()

public:

	/**
	 * The tool will operate on the meshes given here.
	 */
	virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
	{
		Targets = TargetsIn;
	}

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API void RequestAction(EChannelEditToolAction ActionType);
	UE_API EChannelEditToolAction ActiveAction() const;


protected:
	UE_API void ApplyVisbleChannelChange();
	UE_API void UpdateChannelSelectionProperties(int32 ChangingAsset);

	UE_API void AddChannel();
	UE_API void CopyChannel();
	UE_API void DeleteChannel();
	int32 ActiveAsset;
	int32 ActiveChannel;
	int32 ReferenceChannel;

	EChannelEditToolAction PendingAction = EChannelEditToolAction::NoAction;

	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	UPROPERTY()
	TObjectPtr<UUVEditorChannelEditToolActionPropertySet> ToolActions = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorChannelEditSettings> ActionSelectionProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorChannelEditTargetProperties> SourceChannelProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorChannelEditAddProperties> AddActionProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorChannelEditCopyProperties> CopyActionProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorChannelEditDeleteProperties> DeleteActionProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolEmitChangeAPI> EmitChangeAPI = nullptr;

	//
	// Analytics
	//

	struct FActionHistoryItem
	{
		FDateTime Timestamp;
		EChannelEditToolAction ActionType = EChannelEditToolAction::NoAction;

		// if ActionType == Add    then FirstOperandIndex is the index of the added UV layer
		// if ActionType == Delete then FirstOperandIndex is the index of the deleted UV layer
		// if ActionType == Copy   then FirstOperandIndex is the index of the source UV layer
		int32 FirstOperandIndex = -1;
		
		// if ActionType == Add    then SecondOperandIndex is unused
		// if ActionType == Delete then SecondOperandIndex is unused
		// if ActionType == Copy   then SecondOperandIndex is the index of the target UV layer
		int32 SecondOperandIndex = -1;

		bool bDeleteActionWasActuallyClear = false;
	};
	
	TArray<FActionHistoryItem> AnalyticsActionHistory;
	UE::Geometry::UVEditorAnalytics::FTargetAnalytics InputTargetAnalytics;
	FDateTime ToolStartTimeAnalytics;
	UE_API void RecordAnalytics();
};

#undef UE_API
