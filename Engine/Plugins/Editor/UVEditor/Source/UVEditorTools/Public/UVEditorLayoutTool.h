// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "Selection/UVToolSelectionAPI.h"

#include "UVEditorLayoutTool.generated.h"

#define UE_API UVEDITORTOOLS_API

class UUVEditorToolMeshInput;
class UUVLayoutProperties;
class UUVLayoutOperatorFactory;
class UUVTool2DViewportAPI;

UCLASS(MinimalAPI)
class UUVEditorLayoutToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	// This is a pointer so that it can be updated under the builder without
	// having to set it in the mode after initializing targets.
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;
};

/**
 * 
 */
UCLASS(MinimalAPI)
class UUVEditorLayoutTool : public UInteractiveTool, public IUVToolSupportsSelection
{
	GENERATED_BODY()

public:

	/**
	 * The tool will operate on the meshes given here.
	 */
	virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
	{
		Targets = TargetsIn;
	}

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
protected:

	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	UPROPERTY()
	TObjectPtr<UUVLayoutProperties> Settings = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UUVLayoutOperatorFactory>> Factories;

	UPROPERTY()
	TObjectPtr<UUVToolSelectionAPI> UVToolSelectionAPI = nullptr;

	//~ For UDIM information access
	UPROPERTY()
	TObjectPtr< UUVTool2DViewportAPI> UVTool2DViewportAPI = nullptr; 

	//
	// Analytics
	//

	UE::Geometry::UVEditorAnalytics::FTargetAnalytics InputTargetAnalytics;
	FDateTime ToolStartTimeAnalytics;
	UE_API void RecordAnalytics();
};

#undef UE_API
