// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

UVEDITORTOOLS_API DECLARE_LOG_CATEGORY_EXTERN(LogUVEditor, Log, All);

