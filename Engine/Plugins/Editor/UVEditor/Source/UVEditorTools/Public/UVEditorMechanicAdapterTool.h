// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h"

#include "UVEditorMechanicAdapterTool.generated.h"

class UInteractiveToolManager;

/**
 * This is a dummy tool that only exists to make it possible for mechanics to be used in
 * outside of an actual tool. The tool can be passed in to the mechanics' Setup() calls. 
 * If necessary, the behaviors can be extracted via GetInputBehaviors() to be placed into 
 * an input router.
 */
UCLASS(MinimalAPI)
class UUVEditorMechanicAdapterTool : public UInteractiveTool
{
	GENERATED_BODY()

public:
	UInteractiveToolManager* ToolManager = nullptr;

	virtual UInteractiveToolManager* GetToolManager() const override { return ToolManager; }
};
