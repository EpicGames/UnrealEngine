// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Polygroups/PolygroupSet.h"
#include "Drawing/UVLayoutPreview.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "Selection/UVToolSelectionAPI.h"

#include "UVEditorRecomputeUVsTool.generated.h"

#define UE_API UVEDITORTOOLS_API


// Forward declarations
class UDynamicMeshComponent;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class UUVEditorToolMeshInput;
class URecomputeUVsToolProperties;
class URecomputeUVsOpFactory;

/**
 *
 */
UCLASS(MinimalAPI)
class UUVEditorRecomputeUVsToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	// This is a pointer so that it can be updated under the builder without
	// having to set it in the mode after initializing targets.
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;
};


/**
 * UUVEditorRecomputeUVsTool Recomputes UVs based on existing segmentations of the mesh
 */
UCLASS(MinimalAPI)
class UUVEditorRecomputeUVsTool : public UInteractiveTool, public IUVToolSupportsSelection
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	
	UE_API virtual void OnTick(float DeltaTime) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;


	/**
     * The tool will operate on the meshes given here.
     */
	virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
	{
		Targets = TargetsIn;
	}

protected:
	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	UPROPERTY()
	TObjectPtr<URecomputeUVsToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<URecomputeUVsOpFactory>> Factories;

	UPROPERTY()
	TObjectPtr<UUVToolSelectionAPI> UVToolSelectionAPI = nullptr;


	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> ActiveGroupSet;
	UE_API void OnSelectedGroupLayerChanged();
	UE_API void UpdateActiveGroupLayer(bool bUpdateFactories = true);

	//
	// Analytics
	//
	
	UE::Geometry::UVEditorAnalytics::FTargetAnalytics InputTargetAnalytics;
	FDateTime ToolStartTimeAnalytics;
	UE_API void RecordAnalytics();
};

#undef UE_API
