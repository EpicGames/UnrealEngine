// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "IndexTypes.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h" // IInteractiveToolNestedAcceptCancelAPI
#include "GeometryBase.h"
#include "UVEditorToolAnalyticsUtils.h"

#include "UVEditorSeamTool.generated.h"

#define UE_API UVEDITORTOOLS_API

//class ULocalSingleClickInputBehavior;
class UInputRouter;
class ULocalInputBehaviorSource;
class UPreviewGeometry;
class UUVEditorToolMeshInput;
class UUVToolEmitChangeAPI;
class UUVToolLivePreviewAPI;

PREDECLARE_GEOMETRY(class FDynamicMesh);

UENUM()
enum class EUVEditorSeamMode : uint8
{
	/** Marked path will cut the UV island, creating new seams.*/	
	Cut = 0,
	/** Marked path will join the UV island, removing seams.*/
	Join = 1
};

UCLASS(MinimalAPI)
class UUVEditorSeamToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Options)
	EUVEditorSeamMode Mode = EUVEditorSeamMode::Cut;

	/**
	 * Setting this above 0 will include a measure of path similarity to seam transfer, so that among
	 *  similarly short paths, we pick one that lies closer to the edge. Useful in cases where the path
	 *  is on the wrong diagonal to the triangulation, because it prefers a closely zigzagging path over
	 *  a wider "up and over" path that has similar length. If set to 0, only path length is used.
	 */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (
		ClampMin = 0, UIMax = 1000))
	double PathSimilarityWeight = 200;
};

UCLASS(MinimalAPI)
class UUVEditorSeamToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	// These are pointers so that they can be updated under the builder without
	// having to reset them after things are reinitialized.
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;
};

UCLASS(MinimalAPI)
class UUVEditorSeamTool : public UInteractiveTool,
	public IInteractiveToolNestedAcceptCancelAPI
{
	GENERATED_BODY()

public:
	using FDynamicMeshAABBTree3 = UE::Geometry::FDynamicMeshAABBTree3;

	UE_API virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn);

	// For use by undo/redo
	UE_API void EditLockedPath(
		TUniqueFunction<void(TArray<int32>& LockedPathInOut)> EditFunction, int32 MeshIndex);

	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }

	// IInteractiveToolNestedAcceptCancelAPI
	virtual bool SupportsNestedCancelCommand() override { return true; }
	UE_API virtual bool CanCurrentlyNestedCancel() override;
	UE_API virtual bool ExecuteNestedCancelCommand() override;
	virtual bool SupportsNestedAcceptCommand() override { return true; }
	UE_API virtual bool CanCurrentlyNestedAccept() override;
	UE_API virtual bool ExecuteNestedAcceptCommand() override;
protected:
	UE_API void ReconstructExistingSeamsVisualization();
	UE_API void ReconstructLockedPathVisualization();

	UE_API int32 Get2DHitVertex(const FRay& WorldRayIn, int32* IndexOf2DSpatialOut = nullptr);
	UE_API int32 Get3DHitVertex(const FRay& WorldRayIn, int32* IndexOf3DSpatialOut = nullptr);

	UE_API void OnMeshVertexClicked(int32 Vid, int32 IndexOfMesh, bool bVidIsFromUnwrap);
	UE_API void OnMeshVertexHovered(int32 Vid, int32 IndexOfMesh, bool bVidIsFromUnwrap);
	UE_API void UpdateHover();
	/** @param bClearHoverInfo If true, also clears HoverVid, etc in addition to just clearing display. */
	UE_API void ClearHover(bool bClearHoverInfo = true);
	UE_API void ResetPreviewColors();
	UE_API void ApplyClick();
	/** Clears LockedAppliedVids, but builds seam off of AppliedVidsIn */
	UE_API void ApplySeam(const TArray<int32>& AppliedVidsIn);
	UE_API void ClearLockedPath(bool bEmitChange = true);

	UE_API void UpdateToolMessage();

	int32 TemporaryModeToggleModifierID = 1;
	bool bModeIsTemporarilyToggled;
	UE_API bool IsInJoinMode() const;
	UE_API void OnSeamModeChanged();

	enum class EState
	{
		WaitingToStart,
		SeamInProgress
	};

	EState State = EState::WaitingToStart;

	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	UPROPERTY()
	TObjectPtr< UUVEditorSeamToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolLivePreviewAPI> LivePreviewAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolEmitChangeAPI> EmitChangeAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UInputBehaviorSet> LivePreviewBehaviorSet = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalInputBehaviorSource> LivePreviewBehaviorSource = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> UnwrapGeometry = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> LivePreviewGeometry = nullptr;

	TWeakObjectPtr<UInputRouter> LivePreviewInputRouter = nullptr;

	TArray<TSharedPtr<FDynamicMeshAABBTree3>> Spatials2D; // 1:1 with targets
	TArray<TSharedPtr<FDynamicMeshAABBTree3>> Spatials3D; // 1:1 with targets

	FViewCameraState LivePreviewCameraState;

	// Used to remember click info to apply on tick
	int32 ClickedVid = IndexConstants::InvalidID;
	int32 ClickedMeshIndex = -1;
	bool bClickWasInUnwrap = false;

	// Used to remember hover info to apply on tick
	int32 HoverVid = IndexConstants::InvalidID;
	int32 HoverMeshIndex = IndexConstants::InvalidID;
	bool bHoverVidIsFromUnwrap = false;
	int32 LastHoverVid = IndexConstants::InvalidID;
	int32 LastHoverMeshIndex = IndexConstants::InvalidID;
	bool bLastHoverVidWasFromUnwrap = false;

	// Used to know when to end the seam.
	int32 SeamStartAppliedVid = IndexConstants::InvalidID;
	int32 LastLockedAppliedVid = IndexConstants::InvalidID;

	TArray<int32> LockedPath;
	TArray<int32> PreviewPath;

	UE_API FColor GetLockedPathColor() const;
	UE_API FColor GetExtendPathColor() const;

	// When true, the entire path is changed to the "completion" color to show
	// that the next click will complete the path.
	bool bCompletionColorOverride = false;
	
	//
	// Analytics
	//
	
	UE::Geometry::UVEditorAnalytics::FTargetAnalytics InputTargetAnalytics;
	FDateTime ToolStartTimeAnalytics;
	UE_API void RecordAnalytics();

private:
	UE_API void GetVidPath(const UUVEditorToolMeshInput& Target, bool bUnwrapMeshSource,
		const TArray<int32>& StartVids, int32 EndVid, TArray<int32>& VidPathOut);

	TArray<double> UnwrapMaxDims;
	TArray<double> AppliedMeshMaxDims;
};

#undef UE_API
