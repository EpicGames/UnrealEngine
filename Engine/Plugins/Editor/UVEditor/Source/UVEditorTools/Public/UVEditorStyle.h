// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define UE_API UVEDITORTOOLS_API

/**
 * Slate style set for UV Editor
 */
class FUVEditorStyle
	: public FSlateStyleSet
{
public:
	static UE_API FName StyleName;

	/** Access the singleton instance for this style set */
	static UE_API FUVEditorStyle& Get();

private:

	UE_API FUVEditorStyle();
	UE_API ~FUVEditorStyle();
};

#undef UE_API
