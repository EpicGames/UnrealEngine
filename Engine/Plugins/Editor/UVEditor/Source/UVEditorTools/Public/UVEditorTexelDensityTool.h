// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Polygroups/PolygroupSet.h"
#include "Drawing/UVLayoutPreview.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "Selection/UVToolSelectionAPI.h"
#include "ParameterizationOps/TexelDensityOp.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include "UVEditorTexelDensityTool.generated.h"

#define UE_API UVEDITORTOOLS_API


// Forward declarations
class UDynamicMeshComponent;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class UUVEditorToolMeshInput;
class UUVEditorUVTransformProperties;
class UUVTexelDensityOperatorFactory;
class UUVEditorTexelDensityTool;
class UUVTool2DViewportAPI;
class UPreviewGeometry;


/**
 *
 */
UCLASS(MinimalAPI)
class UUVEditorTexelDensityToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	// This is a pointer so that it can be updated under the builder without
	// having to set it in the mode after initializing targets.
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;
};

UENUM()
enum class ETexelDensityToolAction
{
	NoAction,
	Processing,

	BeginSamping,
	Sampling
};


UCLASS(MinimalAPI)
class UUVEditorTexelDensityActionSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	TWeakObjectPtr<UUVEditorTexelDensityTool> ParentTool;

	void Initialize(UUVEditorTexelDensityTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(ETexelDensityToolAction Action);

	UFUNCTION(CallInEditor, Category = Actions)
	UE_API void SampleTexelDensity();
};

UCLASS(MinimalAPI)
class UUVEditorTexelDensityToolSettings : public UUVEditorTexelDensitySettings
{
	GENERATED_BODY()
public:

	TWeakObjectPtr<UUVEditorTexelDensityTool> ParentTool;
	void Initialize(UUVEditorTexelDensityTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API virtual bool InSamplingMode() const override;
};

/**
 * UUVEditorRecomputeUVsTool Recomputes UVs based on existing segmentations of the mesh
 */
UCLASS(MinimalAPI)
class UUVEditorTexelDensityTool : public UInteractiveTool, public IUVToolSupportsSelection
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;


	/**
	 * The tool will operate on the meshes given here.
	 */
	virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
	{
		Targets = TargetsIn;
	}


	UE_API void RequestAction(ETexelDensityToolAction ActionType);
	UE_API ETexelDensityToolAction ActiveAction() const;

private:
	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	UPROPERTY()
	TObjectPtr<UUVEditorTexelDensitySettings> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorTexelDensityActionSettings> ActionSettings = nullptr;

	ETexelDensityToolAction PendingAction = ETexelDensityToolAction::NoAction;

	UPROPERTY()
	TObjectPtr<UUVToolSelectionAPI> UVToolSelectionAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolLivePreviewAPI> LivePreviewAPI = nullptr;

	//~ For UDIM information access
	UPROPERTY()
	TObjectPtr< UUVTool2DViewportAPI> UVTool2DViewportAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolEmitChangeAPI> EmitChangeAPI = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UUVTexelDensityOperatorFactory>> Factories;

	int32 RemainingTargetsToProcess;
	UE_API void PerformBackgroundScalingTask();

	UPROPERTY()
	TObjectPtr<UInputBehaviorSet> LivePreviewBehaviorSet = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalInputBehaviorSource> LivePreviewBehaviorSource = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> UnwrapGeometry = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> LivePreviewGeometry = nullptr;

	TWeakObjectPtr<UInputRouter> LivePreviewInputRouter = nullptr;

	TArray<TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3>> Spatials2D; // 1:1 with targets
	TArray<TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3>> Spatials3D; // 1:1 with targets

	FViewCameraState LivePreviewCameraState;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> TriangleSetMaterial = nullptr;

	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> ActiveGroupSet;
	UE_API void OnSelectedGroupLayerChanged();
	UE_API void UpdateActiveGroupLayer(bool bUpdateFactories = true);

	UE_API int32 Get2DHitTriangle(const FRay& WorldRayIn, int32* IndexOf2DSpatialOut = nullptr);
	UE_API int32 Get3DHitTriangle(const FRay& WorldRayIn, int32* IndexOf2DSpatialOut = nullptr);

	UE_API void OnMeshTriangleClicked(int32 Tid, int32 IndexOfMesh, bool bTidIsFromUnwrap);
	UE_API void OnMeshTriangleHovered(int32 Tid, int32 IndexOfMesh, bool bTidIsFromUnwrap);
	UE_API void UpdateHover();
	/** @param bClearHoverInfo If true, also clears HoverVid, etc in addition to just clearing display. */
	UE_API void ClearHover(bool bClearHoverInfo = true);

	UE_API void ApplyClick();
	UE_API void UpdateToolMessage();

	// Used to remember click info to apply on tick
	int32 ClickedTid = IndexConstants::InvalidID;
	int32 ClickedMeshIndex = -1;
	bool bClickWasInUnwrap = false;

	// Used to remember hover info to apply on tick
	int32 HoverTid = IndexConstants::InvalidID;
	int32 HoverMeshIndex = IndexConstants::InvalidID;
	bool bHoverTidIsFromUnwrap = false;
	int32 LastHoverTid = IndexConstants::InvalidID;
	int32 LastHoverMeshIndex = IndexConstants::InvalidID;
	bool bLastHoverTidWasFromUnwrap = false;

	//
	// Analytics
	//

	UE::Geometry::UVEditorAnalytics::FTargetAnalytics InputTargetAnalytics;
	FDateTime ToolStartTimeAnalytics;
	UE_API void RecordAnalytics();


};

#undef UE_API
