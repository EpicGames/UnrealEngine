// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineAnalytics.h"
#include "ToolTargets/UVEditorToolMeshInput.h"

#define UE_API UVEDITORTOOLS_API

namespace UE
{
namespace Geometry
{
	
namespace UVEditorAnalytics
{
	
template <typename EnumType>
FAnalyticsEventAttribute AnalyticsEventAttributeEnum(const FString& AttributeName, EnumType EnumValue)
{
	static_assert(TIsEnum<EnumType>::Value, "");
	const FString EnumValueName = StaticEnum<EnumType>()->GetNameStringByValue(static_cast<int64>(EnumValue));
	return FAnalyticsEventAttribute(AttributeName, EnumValueName);
}

UVEDITORTOOLS_API FString UVEditorAnalyticsEventName(const FString& ToolEventName);

struct FTargetAnalytics
{
	int64 AllAssetsNumVertices = 0;
	int64 AllAssetsNumTriangles = 0;
	int64 AllAssetsNumUVLayers = 0;
	
	TArray<int32> PerAssetNumVertices;
	TArray<int32> PerAssetNumTriangles;
	TArray<int32> PerAssetNumUVLayers;
	TArray<int32> PerAssetActiveUVChannelIndex;
	TArray<int32> PerAssetActiveUVChannelNumVertices;
	TArray<int32> PerAssetActiveUVChannelNumTriangles;

	UE_API void AppendToAttributes(TArray<FAnalyticsEventAttribute>& Attributes, FString Prefix = "") const;
};

UVEDITORTOOLS_API FTargetAnalytics CollectTargetAnalytics(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& Targets);
	
} // end namespace UVEditorAnalytics
	
} // end namespace Geometry
} // end namespace UE

#undef UE_API
