// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveToolBuilder.h"
#include "Templates/SubclassOf.h"
#include "UObject/Interface.h"

#include "UVEditorToolBase.generated.h"

#define UE_API UVEDITORTOOLS_API

class UUVEditorToolMeshInput;

/** UObject for IUVEditorGenericBuildableTool */
UINTERFACE(MinimalAPI)
class UUVEditorGenericBuildableTool : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that allows a component to receive various gizmo-specific callbacks while
 * still inheriting from some class other than UGizmoBaseComponent.
 */
class IUVEditorGenericBuildableTool
{
	GENERATED_BODY()
public:
	virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn) = 0;
};

//~ TODO: We can use this builder for pretty much all our UV tools and remove the other builder classes
/**
 * Simple builder that just instantiates the given class and passes in the targets. Can be used
 * for any UV tools that don't need special handling, as long as they implement IUVEditorGenericBuildableTool.
 */
UCLASS(MinimalAPI)
class UGenericUVEditorToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:

	/**
	 * @param ToolClassIn Must implement IUVEditorGenericBuildableTool.
	 */
	UE_API virtual void Initialize(TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn, TSubclassOf<UInteractiveTool> ToolClassIn);

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

private:
	// This is a pointer so that it can be updated under the builder without
	// having to set it in the mode after initializing targets.
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;

	TSubclassOf<UInteractiveTool> ToolClass;
};

#undef UE_API
