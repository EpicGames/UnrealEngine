// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "Selection/UVToolSelectionAPI.h"
#include "Operators/UVEditorUVTransformOp.h"

#include "UVEditorTransformTool.generated.h"

#define UE_API UVEDITORTOOLS_API

class UUVEditorToolMeshInput;
class UUVEditorUVTransformProperties;
enum class EUVEditorUVTransformType;
class UUVEditorUVTransformOperatorFactory;
class UCanvas;
class UUVEditorTransformTool;

/**
 * Visualization settings for the TransformTool
 */
UCLASS(MinimalAPI)
class UUVEditorTransformToolDisplayProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Draw the tool's active pivot location if needed.*/
	UPROPERTY(EditAnywhere, Category = "Tool Display Settings", meta = (DisplayName = "Display Tool Pivots"))
	bool bDrawPivots = true;
};

UCLASS(MinimalAPI)
class UUVEditorBaseTransformToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	// This is a pointer so that it can be updated under the builder without
	// having to set it in the mode after initializing targets.
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;

protected:
	UE_API virtual void ConfigureTool(UUVEditorTransformTool* NewTool) const;
};

UCLASS(MinimalAPI)
class UUVEditorTransformToolBuilder : public UUVEditorBaseTransformToolBuilder
{
	GENERATED_BODY()

protected:
	UE_API virtual void ConfigureTool(UUVEditorTransformTool* NewTool) const override;
};

UCLASS(MinimalAPI)
class UUVEditorAlignToolBuilder : public UUVEditorBaseTransformToolBuilder
{
	GENERATED_BODY()

protected:
	UE_API virtual void ConfigureTool(UUVEditorTransformTool* NewTool) const  override;
};

UCLASS(MinimalAPI)
class UUVEditorDistributeToolBuilder : public UUVEditorBaseTransformToolBuilder
{
	GENERATED_BODY()

protected:
	UE_API virtual void ConfigureTool(UUVEditorTransformTool* NewTool) const  override;
};

/**
 * 
 */
UCLASS(MinimalAPI)
class UUVEditorTransformTool : public UInteractiveTool, public IUVToolSupportsSelection
{
	GENERATED_BODY()

public:

	/**
	 * The tool will operate on the meshes given here.
	 */
	virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
	{
		Targets = TargetsIn;
	}

	UE_API void SetToolMode(const EUVEditorUVTransformType& Mode);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

protected:	

	TOptional<EUVEditorUVTransformType> ToolMode;

	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	UPROPERTY()
	TObjectPtr<UUVEditorUVTransformPropertiesBase> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorTransformToolDisplayProperties> DisplaySettings = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UUVEditorUVTransformOperatorFactory>> Factories;

	UPROPERTY()
	TObjectPtr<UUVToolSelectionAPI> UVToolSelectionAPI = nullptr;

	TArray<TArray<FVector2D>> PerTargetPivotLocations;

	//
	// Analytics
	//

	UE::Geometry::UVEditorAnalytics::FTargetAnalytics InputTargetAnalytics;
	FDateTime ToolStartTimeAnalytics;
	UE_API void RecordAnalytics();
};

#undef UE_API
