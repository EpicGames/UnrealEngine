// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Baking/BakingTypes.h"
#include "CoreMinimal.h"
#include "Drawing/PreviewGeometryActor.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h"
#include "MeshOpPreviewHelpers.h"
#include "Sampling/MeshMapBaker.h"
#include "Selection/UVToolSelectionAPI.h"

#include "UVEditorUVSnapshotTool.generated.h"

#define UE_API UVEDITORTOOLS_API


// Forward Declarations
class UUVEditorBakeUVShellProperties;

/**
 * ToolBuilder
 */
UCLASS(MinimalAPI)
class UUVEditorUVSnapshotToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	// This is a pointer so that it can be updated under the builder without
	// having to set it in the mode after initializing targets.
	// Supports only one target
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;
};

/**
 * UV Snapshot Tool
 *
 * Exports a texture asset of a UV Layout
 */
UCLASS(MinimalAPI)
class UUVEditorUVSnapshotTool : public UInteractiveTool, public IUVToolSupportsSelection, public UE::Geometry::IGenericDataOperatorFactory<UE::Geometry::FMeshMapBaker>
{
	GENERATED_BODY()
public:
	// Begin UInteractiveTool interface
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	
	UE_API virtual void OnTick(float DeltaTime) override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override { };
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	// End UInteractiveTool interface

	// Begin IGenericDataOperatorFactory interface
	UE_API virtual TUniquePtr<UE::Geometry::TGenericDataOperator<UE::Geometry::FMeshMapBaker>> MakeNewOperator() override;
	// End IGenericDataOperatorFactory interface
	
	/**
	 * The tool will operate on the mesh given here. Supports only one mesh.
	 */
	virtual void SetTarget(const TObjectPtr<UUVEditorToolMeshInput>& TargetIn)
	{
		Target = TargetIn;
	}
private:
	//
	// Mesh input to UV Editor
	//
	UPROPERTY()
	TObjectPtr<UUVEditorToolMeshInput> Target;

	//
	// Property set for bake and result
	//
	UPROPERTY()
	TObjectPtr<UUVEditorBakeUVShellProperties> UVShellSettings;

	//
	// Preview Geometry for display in Unwrapped viewport
	//
	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeoBackgroundQuad;

	//
	// Background compute
	//
	TUniquePtr<TGenericDataBackgroundCompute<UE::Geometry::FMeshMapBaker>> Compute = nullptr;

	// used in the bake
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> DetailMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> DetailSpatial;

	/**
	 * Retrieves the result of the FMeshMapBaker and generated UTexture2D into the CachedUVMap
	 * 
	 * @param NewResult the resulting FMeshMapBaker from the background compute
	 */
	UE_API void OnMapUpdated(const TUniquePtr<UE::Geometry::FMeshMapBaker>& NewResult);

	/**
	 * Updates the preview material on the preview quad with the
	 * computed results. Invoked by OnMapsUpdated.
	 * Also sets the result to what is currently in CachedUVMap
	 */
	UE_API void UpdateVisualization();

	/**
	 * Internal cache of bake uv texture result
	 */
	UPROPERTY()
	TObjectPtr<UTexture2D> CachedUVMap = nullptr;

	/**
	 * Uses Preview Geometry to draw a preview of the bake in the unwrap viewport
	 */
	UE_API void SetUpPreviewQuad();

	/**
	 * Retrieves the Material for the preview quad, or the preview result
	 */
	UE_API UMaterialInstanceDynamic* GetMaterialForQuad();

	/**
	 * Create texture asset from our result Texture2D
	 * @param Texture the result texture to create
	 */
	UE_API void CreateTextureAsset(const TObjectPtr<UTexture2D>& Texture) const;

	/**
	 * Initialize the list of all UV Layer names
	 */
	static UE_API void InitializeUVLayerNames(TArray<FString>& UVLayerNamesList, const int16 NumUVLayers);
};

UCLASS(MinimalAPI)
class UUVEditorBakeUVShellProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** The source mesh UV Layer to sample. */
	UPROPERTY(meta = (DisplayName = "UV Layer", GetOptions = GetTargetUVLayerNamesFunc, NoResetToDefault))
	FString UVLayer;

	/** The thickness of the wireframe in pixels. */
	UPROPERTY(EditAnywhere, Category = "UV Snapshot Output", meta = (UIMin = "0.0", UIMax = "10.0", ClampMin = "0.0"))
	float WireframeThickness = 1.0f;

	/** The color of wireframe pixels. */
	UPROPERTY(EditAnywhere, Category = "UV Snapshot Output")
	FLinearColor WireframeColor = FLinearColor::Blue;

	/** The color of the UV shell interior pixels. */
	UPROPERTY(EditAnywhere, Category = "UV Snapshot Output")
	FLinearColor ShellColor = FLinearColor::Gray;

	/** The color of pixels external to UV shells. */
	UPROPERTY(EditAnywhere, Category = "UV Snapshot Output")
	FLinearColor BackgroundColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);

	/** The pixel resolution of the generated textures */
	UPROPERTY(EditAnywhere, Category = Textures)
	EBakeTextureResolution Resolution = EBakeTextureResolution::Resolution256;

	/** Number of samples per pixel */
	UPROPERTY(EditAnywhere, Category = Textures)
	EBakeTextureSamplesPerPixel SamplesPerPixel = EBakeTextureSamplesPerPixel::Sample4;

	/** Saved path where last UVSnapshot was saved to. Empty if this is first save out */
	UPROPERTY()
	FString SavedPath = FString();

	/** Bake */
	UPROPERTY(VisibleAnywhere, Category = Results, meta = (DisplayName = "Result Texture", TransientToolProperty))
	TObjectPtr<UTexture2D> Result;

	UFUNCTION()
	const TArray<FString>& GetTargetUVLayerNamesFunc() const
	{
		return TargetUVLayerNamesList;
	}

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> TargetUVLayerNamesList;
};

#undef UE_API
