// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CoreGlobals.h"
#include "HAL/IConsoleManager.h"

#define UE_API UVEDITORTOOLS_API
class FUVEditorUXSettings
{
public:
	// The following are UV Editor specific style items that don't, strictly, matter to the SlateStyleSet,
	// but seemed appropriate to place here.

	// General Display Properties

	// The ScaleFactor just scales the mesh up. Scaling the mesh up makes it easier to zoom in
	// further into the display before getting issues with the camera near plane distance.
	static UE_API const float UVMeshScalingFactor;

	// Position to place the 2D camera far plane relative to world z
	static UE_API const float CameraFarPlaneWorldZ;

	// The near plane gets positioned some proportion to z = 0. We don't use a constant value because our depth offset values are percentage-based
	// Lower proportion move the plane nearer to world z
	// Note: This serves as an upper bound for all other depth offsets - higher than this value risks being clipped
	static UE_API const float CameraNearPlaneProportionZ;;


	// 2D Unwrap Display Properties
	static UE_API const float UnwrapTriangleOpacity;
	static UE_API const float UnwrapTriangleDepthOffset;
	static UE_API const float UnwrapTriangleOpacityWithBackground;

	static UE_API const float WireframeDepthOffset;
	static UE_API const float UnwrapBoundaryHueShift;
	static UE_API const float UnwrapBoundarySaturation;
	static UE_API const float UnwrapBoundaryValue;
	static UE_API const FColor UnwrapTriangleFillColor;
	static UE_API const FColor UnwrapTriangleWireframeColor;

	static UE_API FLinearColor GetTriangleColorByTargetIndex(int32 TargetIndex);
	static UE_API FLinearColor GetWireframeColorByTargetIndex(int32 TargetIndex);
	static UE_API FLinearColor GetBoundaryColorByTargetIndex(int32 TargetIndex);

	// Provides a color blindness friendly ramp for visualizing metrics and other data
	// Domain is [0 , 1]
	static UE_API FColor MakeCividisColorFromScalar(float Scalar);

	// Provides a color blindness friendly ramp for visualizing diverging metrics and other data
	// Domain is [-0.5, 0.5] 
	static UE_API FColor MakeTurboColorFromScalar(float Scalar);

	// Wireframe Properties
	static UE_API const float WireframeThickness;
	static UE_API const float BoundaryEdgeThickness;

	// Selection Highlighting Properties
	static UE_API const float SelectionTriangleOpacity;
	static UE_API const FColor SelectionTriangleFillColor;
	static UE_API const FColor SelectionTriangleWireframeColor;

	static UE_API const float SelectionHoverTriangleOpacity;
	static UE_API const FColor SelectionHoverTriangleFillColor;
	static UE_API const FColor SelectionHoverTriangleWireframeColor;

	static UE_API const float LivePreviewHighlightThickness;
	static UE_API const float LivePreviewHighlightPointSize;
	static UE_API const float LivePreviewHighlightDepthOffset;

	static UE_API const float SelectionLineThickness;
	static UE_API const float SelectionPointThickness;
	static UE_API const float SelectionWireframeDepthBias;
	static UE_API const float SelectionTriangleDepthBias;
	static UE_API const float SelectionHoverWireframeDepthBias;
	static UE_API const float SelectionHoverTriangleDepthBias;

	static UE_API const FColor LivePreviewExistingSeamColor;
	static UE_API const float LivePreviewExistingSeamThickness;
	static UE_API const float LivePreviewExistingSeamDepthBias;

	// These are currently used by the seam tool but can be generally used by
	// tools for displaying paths.
	static UE_API const FColor ToolLockedCutPathColor;
	static UE_API const FColor ToolLockedJoinPathColor;
	static UE_API const float ToolLockedPathThickness;
	static UE_API const float ToolLockedPathDepthBias;
	static UE_API const FColor ToolExtendCutPathColor;
	static UE_API const FColor ToolExtendJoinPathColor;
	static UE_API const float ToolExtendPathThickness;
	static UE_API const float ToolExtendPathDepthBias;
	static UE_API const FColor ToolCompletionPathColor;

	static UE_API const float ToolPointSize;

	// Sew Action styling
	static UE_API const float SewLineHighlightThickness;
	static UE_API const float SewLineDepthOffset;
	static UE_API const FColor SewSideLeftColor;
	static UE_API const FColor SewSideRightColor;

	// Grid 
	static UE_API const float AxisThickness;
	static UE_API const float GridMajorThickness;
	static UE_API const FColor XAxisColor;
	static UE_API const FColor YAxisColor;
	static UE_API const FColor RulerXColor;
	static UE_API const FColor RulerYColor;
	static UE_API const FColor GridMajorColor;
	static UE_API const FColor GridMinorColor;
	static UE_API const int32 GridSubdivisionsPerLevel;
	static UE_API const int32 GridLevels;
	static UE_API const int32 RulerSubdivisionLevel;

	// Pivots
	static UE_API const int32 PivotCircleNumSides;
	static UE_API const float PivotCircleRadius;
	static UE_API const float PivotLineThickness;
	static UE_API const FColor PivotLineColor;

	// Background
	static UE_API const float BackgroundQuadDepthOffset;

	//---------------------------
	//  Common Utility Methods
	//---------------------------


	// Note about the conversions from unwrap world to UV, below:
	// Nothing should make assumptions about the mapping between world space and UV coordinates. Instead,
	// tools should use the conversion functions below so that the mapping is free to change without affecting 
	// the tools.

	/**
	 * Converts from UV value as displayed to user or seen before import, to point in unwrap world space. This
	 * is useful for mapping visualizations to world unwrap space.
	 *
	 * Like other world-UV conversion functions, clients should not know about the details of this conversion.
	 */
	static UE_API FVector3d ExternalUVToUnwrapWorldPosition(const FVector2f& UV);

	/**
	 * Converts from point in unwrap world space, to UV value as displayed to user or seen before import. This
	 * is useful for labeling UV values in the unwrap world, even though the UV values are stored slightly differently
	 * inside the mesh itself.
	 *
	 * Like other world-UV conversion functions, clients should not know about the details of this conversion.
	 */
	static UE_API FVector2f UnwrapWorldPositionToExternalUV(const FVector3d& VertPosition);

	/**
	 * Converts from UV value as displayed to user or seen before import, to UV as stored on a mesh in Unreal.
	 */
	static UE_API FVector2f ExternalUVToInternalUV(const FVector2f& UV);

	/**
	 * Converts from UV as stored on a mesh in Unreal to UV as displayed to user or seen in an external program.
	 */
	static UE_API FVector2f InternalUVToExternalUV(const FVector2f& UV);

	/**
	 * Convert from UV as stored on a mesh in Unreal to world position in the UV editor unwrap world. This allows
	 * changes in UVs of a mesh to be mapped to its unwrap representation.
	 * 
	 * Like other world-UV conversion functions, clients should not know about the details of this conversion.
	 */
	static UE_API FVector3d UVToVertPosition(const FVector2f& UV);

	/**
	 * Converts from position in UV editor unwrap world to UV value as stored on a mesh in Unreal. This allows
	 * changes in the unwrap world to be mapped to actual mesh UVs.
	 * 
	 * Like other world-UV conversion functions, clients should not know about the details of this conversion.
	 */
	static UE_API FVector2f VertPositionToUV(const FVector3d& VertPosition);


	//--------------------------------
	// CVARs for Experimental Features
	//--------------------------------

	static UE_API TAutoConsoleVariable<int32> CVarEnablePrototypeUDIMSupport;

	//--------------------------------
	// Values for Snapping
	//--------------------------------
	static UE_API float LocationSnapValue(int32 LocationSnapMenuIndex);
	static UE_API int32 MaxLocationSnapValue();

private:

	UE_API FUVEditorUXSettings();
	UE_API ~FUVEditorUXSettings();
};

#undef UE_API
