// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "FrameTypes.h"
#include "GeometryBase.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "Selection/UVToolSelection.h"
#include "Selection/UVToolSelectionAPI.h" // IUVToolSupportsSelection
#include "InteractiveToolQueryInterfaces.h" // IInteractiveToolNestedAcceptCancelAPI

#include "UVSelectTool.generated.h"

#define UE_API UVEDITORTOOLS_API

class UCombinedTransformGizmo;
class UTransformProxy;
class UUVEditorToolMeshInput;
class UUVToolEmitChangeAPI;
class UUVToolViewportButtonsAPI;

UCLASS(MinimalAPI)
class UUVSelectToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	// This is a pointer so that it can be updated under the builder without
	// having to set it in the mode after initializing targets.
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;
};

/**
 * The tool in the UV editor that secretly runs when other tools are not running. It uses the
 * selection api to allow the user to select elements, and has a gizmo that can be used to
 * transform these elements.
 */
UCLASS(MinimalAPI)
class UUVSelectTool : public UInteractiveTool, public IInteractiveToolNestedAcceptCancelAPI, public IUVToolSupportsSelection
{
	GENERATED_BODY()

public:
	
	/**
	 * The tool will operate on the meshes given here.
	 */
	virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
	{
		Targets = TargetsIn;
	}

	UE_API void SelectAll();

	// Used by undo/redo.
	UE_API FTransform GetGizmoTransform() const;
	UE_API void SetGizmoTransform(const FTransform& NewTransform);

	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }

	// IInteractiveToolNestedAcceptCancelAPI
	virtual bool SupportsNestedCancelCommand() override { return true; }
	UE_API virtual bool CanCurrentlyNestedCancel() override;
	UE_API virtual bool ExecuteNestedCancelCommand() override;

protected:
	UE_API virtual void OnSelectionChanged(bool bEmitChangeAllowed, uint32 SelectionChangeType);

	// Callbacks we'll receive from the gizmo proxy
	UE_API virtual void GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);
	UE_API virtual void GizmoTransformStarted(UTransformProxy* Proxy);
	UE_API virtual void GizmoTransformEnded(UTransformProxy* Proxy);

	UE_API virtual void ApplyGizmoTransform();
	UE_API virtual void UpdateGizmo(bool bForceRecomputeSelectionCenters = false);

	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	UPROPERTY()
	TObjectPtr<UUVToolViewportButtonsAPI> ViewportButtonsAPI = nullptr;
	
	UPROPERTY()
	TObjectPtr<UUVToolEmitChangeAPI> EmitChangeAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVToolSelectionAPI> SelectionAPI = nullptr;

	UPROPERTY()
	TObjectPtr<UUVEditorMeshSelectionMechanic> SelectionMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;

	UE::Geometry::FFrame3d InitialGizmoFrame;
	FTransform UnappliedGizmoTransform;
	bool bInDrag = false;
	bool bUpdateGizmoOnCanonicalChange = true;
	bool bGizmoTransformNeedsApplication = false;

	TArray<UE::Geometry::FUVToolSelection> CurrentSelections;
	// The outer arrays are 1:1 with the Selections array obtained from the Selection API
	TArray<TArray<int32>> RenderUpdateTidsPerSelection;
	// Inner arrays for these two are 1:1 with each other
	TArray<TArray<int32>> MovingVidsPerSelection;
	TArray<TArray<FVector3d>> MovingVertOriginalPositionsPerSelection;

private:
	UE_API void ReinitializeFromSelection(bool bForceRecomputeSelectionCenters);
};

#undef UE_API
