// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UVEditorTools : ModuleRules
{
	public UVEditorTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"InteractiveToolsFramework",
				
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				// NOTE: UVEditorTools is a separate module so that it doesn't rely on the editor.
				// So, do not add editor dependencies here.
				
				"Engine",
				"SlateCore",
				"RenderCore",
				"RHI",
					
				"DynamicMesh",
				"GeometryCore",
				"InputCore", // EKey
				"GeometryAlgorithms",
				"ModelingComponents",
				"ModelingOperators",
				"TextureUtilitiesCommon"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
