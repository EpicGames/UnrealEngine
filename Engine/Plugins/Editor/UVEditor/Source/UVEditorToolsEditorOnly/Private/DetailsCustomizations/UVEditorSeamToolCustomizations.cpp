// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/UVEditorSeamToolCustomizations.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/Images/SLayeredImage.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/BreakIterator.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "SAssetView.h"
#include "Widgets/Input/SSegmentedControl.h"

#include "UVEditorStyle.h"
#include "UVEditorSeamTool.h"


#define LOCTEXT_NAMESPACE "UVEditorSeamToolDetailsCustomization"

namespace FUVEditorSeamToolPropertiesDetailsLocal
{
	template<class UENUM_TYPE>
	FPropertyAccess::Result GetPropertyValueAsEnum(const TSharedPtr<IPropertyHandle> Property, UENUM_TYPE& Value)
	{
		FPropertyAccess::Result Result;
		if (Property.IsValid())
		{
			uint8 ValueAsInt8;
			Result = Property->GetValue(/*out*/ ValueAsInt8);
			if (Result == FPropertyAccess::Success)
			{
				Value = (UENUM_TYPE)ValueAsInt8;
				return FPropertyAccess::Success;
			}
			uint16 ValueAsInt16;
			Result = Property->GetValue(/*out*/ ValueAsInt16);
			if (Result == FPropertyAccess::Success)
			{
				Value = (UENUM_TYPE)ValueAsInt16;
				return FPropertyAccess::Success;
			}
			uint32 ValueAsInt32;
			Result = Property->GetValue(/*out*/ ValueAsInt32);
			if (Result == FPropertyAccess::Success)
			{
				Value = (UENUM_TYPE)ValueAsInt32;
				return FPropertyAccess::Success;
			}
		}
		return FPropertyAccess::Fail;
	}

	template<class UENUM_TYPE>
	FPropertyAccess::Result SetPropertyValueAsEnum(const TSharedPtr<IPropertyHandle> Property, const UENUM_TYPE& Value)
	{
		FPropertyAccess::Result Result;

		if (Property.IsValid())
		{
			uint32 ValueAsInt32 = (uint32)Value;;
			Result = Property->SetValue(ValueAsInt32);
			if (Result == FPropertyAccess::Success)
			{
				return Result;
			}
			uint16 ValueAsInt16 = (uint16)Value;;
			Result = Property->SetValue(ValueAsInt16);
			if (Result == FPropertyAccess::Success)
			{
				return Result;
			}
			uint8 ValueAsInt8 = (uint8)Value;;
			Result = Property->SetValue(ValueAsInt8);
			if (Result == FPropertyAccess::Success)
			{
				return Result;
			}
		}
		return FPropertyAccess::Fail;
	}

	template<class PROPERTIES_TYPE, class TOOLBARBUILDER_TYPE, class UENUM_TYPE>
	void AddToggleButtonForEnum(PROPERTIES_TYPE& Properties, TOOLBARBUILDER_TYPE& ToolbarBuilder, TSharedPtr<IPropertyHandle> EnumPropertyHandle, UENUM_TYPE ButtonValue, FText Label, FName IconName)
	{
		ToolbarBuilder.AddToolBarButton(FUIAction(
			FExecuteAction::CreateWeakLambda(Properties, [EnumPropertyHandle, ButtonValue]()
			{
				SetPropertyValueAsEnum(EnumPropertyHandle, ButtonValue);
			}),
			FCanExecuteAction::CreateWeakLambda(Properties, [EnumPropertyHandle]() {
				return EnumPropertyHandle->IsEditable();
			}),
			FIsActionChecked::CreateWeakLambda(Properties, [EnumPropertyHandle, ButtonValue]()
			{
				UENUM_TYPE Value;
				GetPropertyValueAsEnum(EnumPropertyHandle, Value);
				return Value == ButtonValue;
			})),
			NAME_None, // Extension hook
			TAttribute<FText>(Label), // Label
			TAttribute<FText>(Label), // Tooltip
			TAttribute<FSlateIcon>(FSlateIcon(FUVEditorStyle::Get().GetStyleSetName(), IconName)),
			EUserInterfaceActionType::ToggleButton);
	};

}


//
// UVEditorTransformTool
//


TSharedRef<IDetailCustomization> FUVEditorSeamToolPropertiesDetails::MakeInstance()
{
	return MakeShareable(new FUVEditorSeamToolPropertiesDetails);
}


void FUVEditorSeamToolPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace FUVEditorSeamToolPropertiesDetailsLocal;

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	UUVEditorSeamToolProperties* Properties = CastChecked<UUVEditorSeamToolProperties>(ObjectsBeingCustomized[0]);

	const FName OptionsCategoryName = "Options";
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(OptionsCategoryName);

	TSharedRef<IPropertyHandle> ModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorSeamToolProperties, Mode), UUVEditorSeamToolProperties::StaticClass());
	ensure(ModeHandle->IsValidHandle());
	ModeHandle->MarkHiddenByCustomization();

	// Create a toolbar for the selection filter
	TSharedRef<FUICommandList> CommandList = MakeShared<FUICommandList>();
	FToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None);

	FName ToolBarStyle = "SeamTool.ModeToolbar";
	ToolbarBuilder.SetStyle(&FUVEditorStyle::Get(), ToolBarStyle);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Visible);

	ToolbarBuilder.BeginSection("Tool Mode");
	ToolbarBuilder.BeginBlockGroup();

	FUVEditorSeamToolPropertiesDetailsLocal::AddToggleButtonForEnum(Properties, ToolbarBuilder, ModeHandle, EUVEditorSeamMode::Cut, LOCTEXT("SeamToolSplitMode", "Split"), "UVEditor.SplitAction");
	FUVEditorSeamToolPropertiesDetailsLocal::AddToggleButtonForEnum(Properties, ToolbarBuilder, ModeHandle, EUVEditorSeamMode::Join, LOCTEXT("SeamToolSewMode", "Sew"), "UVEditor.SewAction");

	ToolbarBuilder.EndBlockGroup();
	ToolbarBuilder.EndSection();

	FDetailWidgetRow& CustomRow = CategoryBuilder.AddCustomRow(FText::GetEmpty())
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("SeamToolModeLabel", "Mode"))
	]
	.ValueContent()
	[
	
		SNew(SBox)
		[
			SNew(SSegmentedControl<EUVEditorSeamMode>)
			.Value(this, &FUVEditorSeamToolPropertiesDetails::GetCurrentMode, ModeHandle)
			.OnValueChanged(this, &FUVEditorSeamToolPropertiesDetails::OnCurrentModeChanged, ModeHandle)
			+ SSegmentedControl<EUVEditorSeamMode>::Slot(EUVEditorSeamMode::Cut)
		        .Text(LOCTEXT("SeamToolSplitMode", "Split"))
				.Icon(FUVEditorStyle::Get().GetBrush("UVEditor.SplitAction"))
				.ToolTip(LOCTEXT("SeamToolSplitMode", "Split"))
			+ SSegmentedControl<EUVEditorSeamMode>::Slot(EUVEditorSeamMode::Join)
		        .Text(LOCTEXT("SeamToolSewMode", "Sew"))
				.Icon(FUVEditorStyle::Get().GetBrush("UVEditor.SewAction"))
				.ToolTip(LOCTEXT("SeamToolSewMode", "Sew"))
		]
	];

}


EUVEditorSeamMode FUVEditorSeamToolPropertiesDetails::GetCurrentMode(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	EUVEditorSeamMode ModeValue = EUVEditorSeamMode::Cut;
	FUVEditorSeamToolPropertiesDetailsLocal::GetPropertyValueAsEnum(PropertyHandle, ModeValue);
	return ModeValue;
}

void FUVEditorSeamToolPropertiesDetails::OnCurrentModeChanged(EUVEditorSeamMode ModeValue, TSharedRef<IPropertyHandle> PropertyHandle)
{
	FUVEditorSeamToolPropertiesDetailsLocal::SetPropertyValueAsEnum(PropertyHandle, ModeValue);
}

#undef LOCTEXT_NAMESPACE