// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/UVTransformToolCustomizations.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/Images/SLayeredImage.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/BreakIterator.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "SAssetView.h"

#include "Editor.h" // For supporting Undo transactions in place

#include "PropertyRestriction.h"

#include "UVEditorStyle.h"
#include "UVEditorTransformTool.h"
#include "Operators/UVEditorUVTransformOp.h"


#define LOCTEXT_NAMESPACE "UVEditorDetailsCustomization"

namespace UVEditorTransformDetailsCustomizationLocal
{
	void CustomSortTransformToolCategories(const TMap<FName, IDetailCategoryBuilder*>&  AllCategoryMap )
	{
		(*AllCategoryMap.Find(FName("Quick Translate")))->SetSortOrder(0);
		(*AllCategoryMap.Find(FName("Quick Rotate")))->SetSortOrder(1);
		(*AllCategoryMap.Find(FName("Quick Transform")))->SetSortOrder(2);
	}

	template<class UENUM_TYPE>
	FPropertyAccess::Result GetPropertyValueAsEnum(const TSharedPtr<IPropertyHandle> Property, UENUM_TYPE& Value)
	{
		FPropertyAccess::Result Result;
		if (Property.IsValid())
		{
			uint8 ValueAsInt8;
			Result = Property->GetValue(/*out*/ ValueAsInt8);
			if (Result == FPropertyAccess::Success)
			{
				Value = (UENUM_TYPE)ValueAsInt8;
				return FPropertyAccess::Success;
			}
			uint16 ValueAsInt16;
			Result = Property->GetValue(/*out*/ ValueAsInt16);
			if (Result == FPropertyAccess::Success)
			{
				Value = (UENUM_TYPE)ValueAsInt16;
				return FPropertyAccess::Success;
			}
			uint32 ValueAsInt32;
			Result = Property->GetValue(/*out*/ ValueAsInt32);
			if (Result == FPropertyAccess::Success)
			{
				Value = (UENUM_TYPE)ValueAsInt32;
				return FPropertyAccess::Success;
			}
		}
		return FPropertyAccess::Fail;
	}

	template<class UENUM_TYPE>
	FPropertyAccess::Result SetPropertyValueAsEnum(const TSharedPtr<IPropertyHandle> Property, const UENUM_TYPE& Value)
	{
		FPropertyAccess::Result Result;

		if (Property.IsValid())
		{
			uint32 ValueAsInt32 = (uint32)Value;;
			Result = Property->SetValue(ValueAsInt32);
			if (Result == FPropertyAccess::Success)
			{
				return Result;
			}
			uint16 ValueAsInt16 = (uint16)Value;;
			Result = Property->SetValue(ValueAsInt16);
			if (Result == FPropertyAccess::Success)
			{
				return Result;
			}
			uint8 ValueAsInt8 = (uint8)Value;;
			Result = Property->SetValue(ValueAsInt8);
			if (Result == FPropertyAccess::Success)
			{
				return Result;
			}
		}
		return FPropertyAccess::Fail;
	}
}


//
// UVEditorTransformTool
//


TSharedRef<IDetailCustomization> FUVEditorUVTransformToolDetails::MakeInstance()
{
	return MakeShareable(new FUVEditorUVTransformToolDetails);
}


void FUVEditorUVTransformToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UVEditorTransformDetailsCustomizationLocal;

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	DetailBuilder.EditCategory("AdvancedTransform", LOCTEXT("AdvancedTransformCategoryName", "Advanced Transform"));
	check(ObjectsBeingCustomized.Num() > 0);
	BuildQuickTranslateMenu(DetailBuilder);
	BuildQuickRotateMenu(DetailBuilder);

	DetailBuilder.SortCategories(&CustomSortTransformToolCategories);
}

void FUVEditorUVTransformToolDetails::BuildQuickTranslateMenu(IDetailLayoutBuilder& DetailBuilder)
{
	const FName QuickTranslateCategoryName = "Quick Translate";
	const FText QuickTranslateCategoryLocName = LOCTEXT("QuickTranslateCategoryName", "Quick Translate");
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(QuickTranslateCategoryName, QuickTranslateCategoryLocName);

	TSharedPtr<IPropertyHandle> QuickTranslateOffsetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVTransformProperties, QuickTranslateOffset), UUVEditorUVTransformProperties::StaticClass());
	ensure(QuickTranslateOffsetHandle->IsValidHandle());
	QuickTranslateOffsetHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> QuickTranslateHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVTransformProperties, QuickTranslation), UUVEditorUVTransformProperties::StaticClass());
	ensure(QuickTranslateHandle->IsValidHandle());
	QuickTranslateHandle->MarkHiddenByCustomization();

	auto ApplyTranslation = [this, QuickTranslateHandle, QuickTranslateOffsetHandle](FText TransactionDescription, const FVector2D& Direction)
	{
		GEditor->BeginTransaction(TransactionDescription);
		float TranslationValue;
		FVector2D TotalTranslationValue;
		QuickTranslateOffsetHandle->GetValue(TranslationValue);
		QuickTranslateHandle->GetValue(TotalTranslationValue);
		QuickTranslateHandle->SetValue(TotalTranslationValue + (Direction * TranslationValue) );
		GEditor->EndTransaction();
		return FReply::Handled();
	};

	FDetailWidgetRow& CustomRow = CategoryBuilder.AddCustomRow(FText::GetEmpty())
	[
		SNew(SUniformGridPanel)
		.SlotPadding(FMargin(5.0f))
		+ SUniformGridPanel::Slot(0, 0)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickMoveTopLeft", "TL"))
			.ToolTipText(LOCTEXT("QuickMoveTopLeftToolTip", "Applies the translation offset in the negative X axis and the positive Y axis"))
			.OnClicked_Lambda([ApplyTranslation]() { return ApplyTranslation(LOCTEXT("TransactionTopLeft", "Top Left Translation"), FVector2D(-1.0, 1.0)); })
		]
		+ SUniformGridPanel::Slot(1, 0)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickMoveTop", "T"))
			.ToolTipText(LOCTEXT("QuickMoveTopToolTip", "Applies the translation offset in the positive Y axis."))
			.OnClicked_Lambda([ApplyTranslation]() { return ApplyTranslation(LOCTEXT("TransactionTop", "Top Translation"), FVector2D(0.0, 1.0)); })
		]
		+ SUniformGridPanel::Slot(2, 0)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickMoveTopRight", "TR"))
			.ToolTipText(LOCTEXT("QuickMoveTopRightToolTip", "Applies the translation offset in the positive X axis and the positive Y axis."))
			.OnClicked_Lambda([ApplyTranslation]() { return ApplyTranslation(LOCTEXT("TransactionTopRight", "Top Right Translation"), FVector2D(1.0, 1.0)); })
		]
		+ SUniformGridPanel::Slot(0, 1)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickMoveLeft", "L"))
			.ToolTipText(LOCTEXT("QuickMoveLeftToolTip", "Applies the translation offset in the negative X axis."))
			.OnClicked_Lambda([ApplyTranslation]() { return ApplyTranslation(LOCTEXT("TransactionLeft", "Left Translation"), FVector2D(-1.0, 0.0)); })
		]
		+ SUniformGridPanel::Slot(1, 1)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SProperty, QuickTranslateOffsetHandle)
			.ShouldDisplayName(false)
		]
		+ SUniformGridPanel::Slot(2, 1)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickMoveRight", "R"))
			.ToolTipText(LOCTEXT("QuickMoveRightToolTip", "Applies the translation offset in the positive X axis."))
			.OnClicked_Lambda([ApplyTranslation]() { return ApplyTranslation(LOCTEXT("TransactionRight", "Right Translation"), FVector2D(1.0, 0.0)); })
		]
		+ SUniformGridPanel::Slot(0, 2)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickMoveBottomLeft", "BL"))
			.ToolTipText(LOCTEXT("QuickMoveBottomLeftToolTip", "Applies the translation offset in the negative X axis and the negative Y axis"))
			.OnClicked_Lambda([ApplyTranslation]() { return ApplyTranslation(LOCTEXT("TransactionBottomLeft", "Bottom Left Translation"), FVector2D(-1.0, -1.0)); })
		]
		+ SUniformGridPanel::Slot(1, 2)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickMoveBottom", "B"))
			.ToolTipText(LOCTEXT("QuickMoveBottomToolTip", "Applies the translation offset in the negative Y axis."))
			.OnClicked_Lambda([ApplyTranslation]() { return ApplyTranslation(LOCTEXT("TransactionBottom", "Bottom Translation"), FVector2D(0.0, -1.0)); })
		]
		+ SUniformGridPanel::Slot(2, 2)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickMoveBottomRight", "BR"))
			.ToolTipText(LOCTEXT("QuickMoveBottomRightToolTip", "Applies the translation offset in the positive X axis and the negative Y axis."))
			.OnClicked_Lambda([ApplyTranslation]() { return ApplyTranslation(LOCTEXT("TransactionBottomRight", "Bottom Right Translation"), FVector2D(1.0, -1.0)); })
		]

	];

}


void FUVEditorUVTransformToolDetails::BuildQuickRotateMenu(IDetailLayoutBuilder& DetailBuilder)
{
	const FName QuickRotateCategoryName = "Quick Rotate";
	const FText QuickRotateCategoryLocName = LOCTEXT("QuickRotateCategoryName", "Quick Rotate");
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(QuickRotateCategoryName, QuickRotateCategoryLocName);

	TSharedPtr<IPropertyHandle> QuickRotationOffsetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVTransformProperties, QuickRotationOffset), UUVEditorUVTransformProperties::StaticClass());
	ensure(QuickRotationOffsetHandle->IsValidHandle());
	QuickRotationOffsetHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> QuickRotationHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVTransformProperties, QuickRotation), UUVEditorUVTransformProperties::StaticClass());
	ensure(QuickRotationHandle->IsValidHandle());
	QuickRotationHandle->MarkHiddenByCustomization();

	auto ApplyRotation = [this, QuickRotationHandle](const float& RotationOffset)
	{		
		float TotalRotationValue;		
		QuickRotationHandle->GetValue(TotalRotationValue);
		QuickRotationHandle->SetValue(TotalRotationValue + RotationOffset);
		return FReply::Handled();
	};

	FDetailWidgetRow& CustomRow = CategoryBuilder.AddCustomRow(FText::GetEmpty())
	[
		SNew(SUniformGridPanel)
		.SlotPadding(FMargin(5.0f))
		+ SUniformGridPanel::Slot(2, 0)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickRotateClockwise", "CW"))
			.ToolTipText(LOCTEXT("QuickRotateClockwiseToolTip", "Applies the rotation in a clockwise orientation"))
			.OnClicked_Lambda([this, QuickRotationOffsetHandle, ApplyRotation]() {
				float RotationValue;
				QuickRotationOffsetHandle->GetValue(RotationValue);
				return ApplyRotation(RotationValue);
			})
		]
		+ SUniformGridPanel::Slot(1, 0)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SProperty, QuickRotationOffsetHandle)
			.ShouldDisplayName(false)
		]
		+ SUniformGridPanel::Slot(0, 0)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickRotateCounterclockwise", "CCW"))
			.ToolTipText(LOCTEXT("QuickRotateCounterclockwiseToolTip", "Applies the rotation in a counter clockwise orientation"))
			.OnClicked_Lambda([this, QuickRotationOffsetHandle, ApplyRotation]() {
				float RotationValue;
				QuickRotationOffsetHandle->GetValue(RotationValue);
				return ApplyRotation(-RotationValue);
			})
		]

		+ SUniformGridPanel::Slot(2, 1)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickRotateClockwise10Deg", "10°"))
			.ToolTipText(LOCTEXT("QuickRotateClockwise10DegToolTip", "Applies a 10 degree clockwise orientation"))
			.OnClicked_Lambda([this, ApplyRotation]() { return ApplyRotation(10.0); })
		]
		+ SUniformGridPanel::Slot(0, 1)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickRotateCounterclockwise10Deg", "-10°"))
			.ToolTipText(LOCTEXT("QuickRotateCounterclockwise10DegToolTip", "Applies a 10 degree counter clockwise orientation"))
			.OnClicked_Lambda([this, ApplyRotation]() { return ApplyRotation(-10.0); })
		]
		+ SUniformGridPanel::Slot(2, 2)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickRotateClockwise45Deg", "45°"))
			.ToolTipText(LOCTEXT("QuickRotateClockwise45DegToolTip", "Applies a 45 degree clockwise orientation"))
			.OnClicked_Lambda([this, ApplyRotation]() { return ApplyRotation(45.0); })
		]
		+ SUniformGridPanel::Slot(0, 2)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickRotateCounterclockwise45Deg", "-45°"))
			.ToolTipText(LOCTEXT("QuickRotateCounterclockwise45DegToolTip", "Applies a 45 degree counter clockwise orientation"))
			.OnClicked_Lambda([this, ApplyRotation]() { return ApplyRotation(-45.0); })
		]
		+ SUniformGridPanel::Slot(2, 3)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickRotateClockwise90Deg", "90°"))
			.ToolTipText(LOCTEXT("QuickRotateClockwise90DegToolTip", "Applies a 90 degree clockwise orientation"))
			.OnClicked_Lambda([this, ApplyRotation]() { return ApplyRotation(90.0); })
		]
		+ SUniformGridPanel::Slot(0, 3)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("QuickRotateCounterclockwise90Deg", "-90°"))
			.ToolTipText(LOCTEXT("QuickRotateCounterclockwise90DegToolTip", "Applies a 90 degree counter clockwise orientation"))
			.OnClicked_Lambda([this, ApplyRotation]() { return ApplyRotation(-90.0); })
		]
	];
}

//
// UVEditorTransformTool - Quick Transform version
//


TSharedRef<IDetailCustomization> FUVEditorUVQuickTransformToolDetails::MakeInstance()
{
	return MakeShareable(new FUVEditorUVTransformToolDetails);
}


void FUVEditorUVQuickTransformToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UVEditorTransformDetailsCustomizationLocal;

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);

	DetailBuilder.HideCategory("Advanced Transform");

	BuildQuickTranslateMenu(DetailBuilder);
	BuildQuickRotateMenu(DetailBuilder);

	DetailBuilder.SortCategories(&CustomSortTransformToolCategories);
}

//
// UVEditorDistributeTool
//


TSharedRef<IDetailCustomization> FUVEditorUVDistributeToolDetails::MakeInstance()
{
	return MakeShareable(new FUVEditorUVDistributeToolDetails);
}


void FUVEditorUVDistributeToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UVEditorTransformDetailsCustomizationLocal;

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	BuildDistributeModeButtons(DetailBuilder);	
}

void FUVEditorUVDistributeToolDetails::BuildDistributeModeButtons(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UVEditorTransformDetailsCustomizationLocal;

	const FName DistributeCategoryName = "Distribute";
	const FText DistributeCategoryLocName = LOCTEXT("DistributeCategoryName", "Distribute");
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(DistributeCategoryName, DistributeCategoryLocName);

	DistributeModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVDistributeProperties, DistributeMode), UUVEditorUVDistributeProperties::StaticClass());
	ensure(DistributeModeHandle->IsValidHandle());
	DistributeModeHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> GroupingHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVDistributeProperties, Grouping), UUVEditorUVDistributeProperties::StaticClass());
	ensure(GroupingHandle->IsValidHandle());

	EnableManualDistancesHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVDistributeProperties, bEnableManualDistances), UUVEditorUVDistributeProperties::StaticClass());
	ensure(EnableManualDistancesHandle->IsValidHandle());

	OrigManualExtentHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVDistributeProperties, ManualExtent), UUVEditorUVDistributeProperties::StaticClass());
	ensure(OrigManualExtentHandle->IsValidHandle());

	OrigManualSpacingHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVDistributeProperties, ManualSpacing), UUVEditorUVDistributeProperties::StaticClass());
	ensure(OrigManualSpacingHandle->IsValidHandle());


	auto BuildButton = [this](const FName& Icon, const FText& Tooltip, TFunction<FReply()> OnClickCallback)
	{

		TSharedRef<SLayeredImage> IconWidget =
			SNew(SLayeredImage)
			.Visibility(EVisibility::HitTestInvisible)
			.Image(FUVEditorStyle::Get().GetBrush(Icon));

		TSharedRef<SWidget> ButtonContent = SNullWidget::NullWidget;

		ButtonContent =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(0)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)
				// Icon image
				+ SVerticalBox::Slot()
				.FillHeight(1)
				.Padding(0)
				.HAlign(HAlign_Center)	// Center the icon horizontally, so that large labels don't stretch out the artwork
				[
					IconWidget
				]
			];


		TSharedRef<SWidget> Button = SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
			.ContentPadding(FMargin(0, 5.0))
			.ContentScale(FVector2D(1, 1))
			.ToolTipText(Tooltip)
			.OnClicked_Lambda(OnClickCallback)
			[
				ButtonContent
			];
		return Button;
	};

	FDetailWidgetRow& CustomRow = CategoryBuilder.AddCustomRow(FText::GetEmpty())
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SUniformWrapPanel)
				.SlotPadding(FMargin(5.0f))
				.EvenRowDistribution(true)
				+ SUniformWrapPanel::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						BuildButton("UVEditor.DistributeSpaceVertically",
						LOCTEXT("DistributeVerticalSpaceToolTip", "Distributes objects so that vertical space between them is equal."),
						[this]()
							{
								SetPropertyValueAsEnum(DistributeModeHandle, EUVEditorDistributeMode::VerticalSpace);
								return FReply::Handled();
							}
						)
					]
				+ SUniformWrapPanel::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						BuildButton("UVEditor.DistributeSpaceHorizontally",
						LOCTEXT("DistributeHorizontalSpaceToolTip", "Distributes objects so that horizontal space between them is equal."),
						[this]()
							{
								SetPropertyValueAsEnum(DistributeModeHandle, EUVEditorDistributeMode::HorizontalSpace);
								return FReply::Handled();
							}
						)
					]
			]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SUniformWrapPanel)
			.SlotPadding(FMargin(5.0f))
			.EvenRowDistribution(true)
			+ SUniformWrapPanel::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					BuildButton("UVEditor.DistributeTopEdges",
					LOCTEXT("DistributeTopEdgesToolTip", "Distributes objects so that distance between top edges is equal."),
					[this]()
						{
							SetPropertyValueAsEnum(DistributeModeHandle, EUVEditorDistributeMode::TopEdges);
							return FReply::Handled();
						}
					)
				]
			+ SUniformWrapPanel::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					BuildButton("UVEditor.DistributeBottomEdges",
					LOCTEXT("DistributeBottomEdgesToolTip", "Distributes objects so that distance between bottom edges is equal."),
					[this]()
						{
							SetPropertyValueAsEnum(DistributeModeHandle, EUVEditorDistributeMode::BottomEdges);
							return FReply::Handled();
						}
					)
				]
			+ SUniformWrapPanel::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					BuildButton("UVEditor.DistributeLeftEdges",
					LOCTEXT("DistributeLeftEdgesToolTip", "Distributes objects so that distance between left edges is equal."),
					[this]()
						{
							SetPropertyValueAsEnum(DistributeModeHandle, EUVEditorDistributeMode::LeftEdges);
							return FReply::Handled();
						}
					)
				]
			+ SUniformWrapPanel::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					BuildButton("UVEditor.DistributeRightEdges",
					LOCTEXT("DistributeRightEdgesToolTip", "Distributes objects so that distance between right edges is equal."),
					[this]()
						{
							SetPropertyValueAsEnum(DistributeModeHandle, EUVEditorDistributeMode::RightEdges);
							return FReply::Handled();
						}
					)
				]
			+ SUniformWrapPanel::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					BuildButton("UVEditor.DistributeCentersHorizontally",
					LOCTEXT("DistributeCenterHorizontalToolTip", "Distributes objects so that distance between horizontal centers is equal."),
					[this]()
						{
							SetPropertyValueAsEnum(DistributeModeHandle, EUVEditorDistributeMode::CentersHorizontally);
							return FReply::Handled();
						}
					)
				]
			+ SUniformWrapPanel::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					BuildButton("UVEditor.DistributeCentersVertically",
					LOCTEXT("DistributeCenterVerticalToolTip", "Distributes objects so that distance between vertical centers is equal."),
					[this]()
						{
							SetPropertyValueAsEnum(DistributeModeHandle, EUVEditorDistributeMode::CentersVertically);
							return FReply::Handled();
						}
					)
				]
			]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SUniformWrapPanel)
			.SlotPadding(FMargin(5.0f))
			.EvenRowDistribution(true)
			+ SUniformWrapPanel::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					BuildButton("UVEditor.DistributeRemoveOverlap",
					LOCTEXT("DistributeMinimalOverlapToolTip", "Distributes objects so that overlap is removed, moving objects minimally as possible."),
					[this]()
						{
							SetPropertyValueAsEnum(DistributeModeHandle, EUVEditorDistributeMode::MinimallyRemoveOverlap);
							return FReply::Handled();
						}
					)
				]
		]
	];

	
	static FText RestrictReason = LOCTEXT("GroupingRestrictionText", "Not supported in Distribute Tool.");
	TSharedPtr<FPropertyRestriction> GroupingEnumRestriction = MakeShareable(new FPropertyRestriction(RestrictReason));
	GroupingHandle->AddRestriction(GroupingEnumRestriction.ToSharedRef());
	GroupingEnumRestriction->AddHiddenValue("EnclosingBoundingBox");

	CategoryBuilder.AddProperty(GroupingHandle);

	CategoryBuilder.AddProperty(EnableManualDistancesHandle);
	IDetailPropertyRow& ManualSpacingRow = CategoryBuilder.AddProperty(OrigManualSpacingHandle);
	IDetailPropertyRow& ManualExtentRow = CategoryBuilder.AddProperty(OrigManualExtentHandle);

	ManualSpacingRow.Visibility(TAttribute<EVisibility>::CreateLambda([this]()
		{
			bool bManualDistance;
			EnableManualDistancesHandle->GetValue(bManualDistance);

			EUVEditorDistributeMode DistributeMode;
			GetPropertyValueAsEnum(DistributeModeHandle, DistributeMode);

			EVisibility Visibility;

			if (bManualDistance)
			{
				if (DistributeMode == EUVEditorDistributeMode::VerticalSpace ||
					DistributeMode == EUVEditorDistributeMode::HorizontalSpace ||
					DistributeMode == EUVEditorDistributeMode::MinimallyRemoveOverlap)
				{
					return EVisibility::Visible;
				}
			}
			return EVisibility::Collapsed;
		}));

	ManualExtentRow.Visibility(TAttribute<EVisibility>::CreateLambda([this]()
		{
			bool bManualDistance;
			EnableManualDistancesHandle->GetValue(bManualDistance);

			EUVEditorDistributeMode DistributeMode;
			GetPropertyValueAsEnum(DistributeModeHandle, DistributeMode);

			if (bManualDistance)
			{
				if (DistributeMode == EUVEditorDistributeMode::TopEdges ||
					DistributeMode == EUVEditorDistributeMode::BottomEdges ||
					DistributeMode == EUVEditorDistributeMode::LeftEdges ||
					DistributeMode == EUVEditorDistributeMode::RightEdges ||
					DistributeMode == EUVEditorDistributeMode::CentersHorizontally ||
					DistributeMode == EUVEditorDistributeMode::CentersVertically)
				{
					return EVisibility::Visible;
				}
			}
			return EVisibility::Collapsed;
		}));

}

//
// UVEditorAlignTool
//

TSharedRef<IDetailCustomization> FUVEditorUVAlignToolDetails::MakeInstance()
{
	return MakeShareable(new FUVEditorUVAlignToolDetails);
}


void FUVEditorUVAlignToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UVEditorTransformDetailsCustomizationLocal;

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	BuildAlignModeButtons(DetailBuilder);
}

void FUVEditorUVAlignToolDetails::BuildAlignModeButtons(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UVEditorTransformDetailsCustomizationLocal;

	const FName AlignCategoryName = "Align";
	const FText AlignCategoryLocName = LOCTEXT("AlignCategoryName", "Align");
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(AlignCategoryName, AlignCategoryLocName);

	AlignDirectionHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVAlignProperties, AlignDirection), UUVEditorUVAlignProperties::StaticClass());
	ensure(AlignDirectionHandle->IsValidHandle());
	AlignDirectionHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> GroupingHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVAlignProperties, Grouping), UUVEditorUVAlignProperties::StaticClass());
	ensure(GroupingHandle->IsValidHandle());

	TSharedPtr<IPropertyHandle> AlignAnchorHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVAlignProperties, AlignAnchor), UUVEditorUVAlignProperties::StaticClass());
	ensure(AlignAnchorHandle->IsValidHandle());

	TSharedPtr<IPropertyHandle> ManualAnchorHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUVEditorUVAlignProperties, ManualAnchor), UUVEditorUVAlignProperties::StaticClass());
	ensure(ManualAnchorHandle->IsValidHandle());

	auto BuildButton = [this](const FName& Icon, const FText& Tooltip, TFunction<FReply()> OnClickCallback)
	{

		TSharedRef<SLayeredImage> IconWidget =
			SNew(SLayeredImage)
			.Visibility(EVisibility::HitTestInvisible)
			.Image(FUVEditorStyle::Get().GetBrush(Icon));

		TSharedRef<SWidget> ButtonContent = SNullWidget::NullWidget;

		ButtonContent =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(0)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)
				// Icon image
				+ SVerticalBox::Slot()
				.FillHeight(1)
				.Padding(0)
				.HAlign(HAlign_Center)	// Center the icon horizontally, so that large labels don't stretch out the artwork
				[
					IconWidget
				]
			];


		TSharedRef<SWidget> Button = SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
			.ContentPadding(FMargin(0, 5.0))
			.ContentScale(FVector2D(1, 1))
			.ToolTipText(Tooltip)
			.OnClicked_Lambda(OnClickCallback)
			[
				ButtonContent
			];
		return Button;
	};


	FDetailWidgetRow& CustomRow = CategoryBuilder.AddCustomRow(FText::GetEmpty())
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SUniformWrapPanel)
				.SlotPadding(FMargin(5.0f))
				.EvenRowDistribution(true)
				+ SUniformWrapPanel::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						BuildButton("UVEditor.AlignDirectionTopEdges",
						LOCTEXT("AlignTopEdgesToolTip", "Aligns the top edges of objects with the reference point."),
						[this]()
							{
								SetPropertyValueAsEnum(AlignDirectionHandle, EUVEditorAlignDirection::Top);
								return FReply::Handled();
							}
						)
					]
				+ SUniformWrapPanel::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						BuildButton("UVEditor.AlignDirectionBottomEdges",
						LOCTEXT("AlignBottomEdgesToolTip", "Aligns the bottom edges of objects with the reference point."),
						[this]()
							{
								SetPropertyValueAsEnum(AlignDirectionHandle, EUVEditorAlignDirection::Bottom);
								return FReply::Handled();
							}
						)
					]
				+ SUniformWrapPanel::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						BuildButton("UVEditor.AlignDirectionLeftEdges",
						LOCTEXT("AlignLeftEdgesToolTip", "Aligns the left edges of objects with the reference point."),
						[this]()
							{
								SetPropertyValueAsEnum(AlignDirectionHandle, EUVEditorAlignDirection::Left);
								return FReply::Handled();
							}
						)
					]
				+ SUniformWrapPanel::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						BuildButton("UVEditor.AlignDirectionRightEdges",
						LOCTEXT("AlignRightEdgesToolTip", "Aligns the right edges of objects with the reference point."),
						[this]()
							{
								SetPropertyValueAsEnum(AlignDirectionHandle, EUVEditorAlignDirection::Right);
								return FReply::Handled();
							}
						)
					]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SUniformWrapPanel)
				.SlotPadding(FMargin(5.0f))
				.EvenRowDistribution(true)
				+ SUniformWrapPanel::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						BuildButton("UVEditor.AlignDirectionCentersVertically",
						LOCTEXT("AlignCenterVerticalToolTip", "Aligns the vertical centerline of objects with the reference point."),
						[this]()
							{
								SetPropertyValueAsEnum(AlignDirectionHandle, EUVEditorAlignDirection::CenterVertically);
								return FReply::Handled();
							}
						)
					]
				+ SUniformWrapPanel::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						BuildButton("UVEditor.AlignDirectionCentersHorizontally",
						LOCTEXT("AlignCenterHorizontalToolTip", "Aligns the horizontal centerline of objects with the reference point."),
						[this]()
							{
								SetPropertyValueAsEnum(AlignDirectionHandle, EUVEditorAlignDirection::CenterHorizontally);
								return FReply::Handled();
							}
						)
					]
			]
		];


	CategoryBuilder.AddProperty(AlignAnchorHandle);
	CategoryBuilder.AddProperty(ManualAnchorHandle);
	CategoryBuilder.AddProperty(GroupingHandle);
}


#undef LOCTEXT_NAMESPACE