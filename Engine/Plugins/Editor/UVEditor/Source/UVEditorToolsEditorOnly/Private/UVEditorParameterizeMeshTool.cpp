// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorParameterizeMeshTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "ParameterizationOps/ParameterizeMeshOp.h"
#include "Properties/ParameterizeMeshProperties.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Polygroups/PolygroupUtil.h"
#include "ToolTargets/UVEditorToolMeshInput.h"
#include "ContextObjects/UVToolContextObjects.h"
#include "ContextObjectStore.h"
#include "EngineAnalytics.h"
#include "UVEditorUXSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVEditorParameterizeMeshTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UParameterizeMeshTool"


// Tool builder
// TODO: Could consider sharing some of the tool builder boilerplate for UV editor tools in a common base class.

bool UUVEditorParameterizeMeshToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return Targets && Targets->Num() >= 1;
}

UInteractiveTool* UUVEditorParameterizeMeshToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UUVEditorParameterizeMeshTool* NewTool = NewObject<UUVEditorParameterizeMeshTool>(SceneState.ToolManager);
	NewTool->SetTargets(*Targets);

	return NewTool;
}


/*
 * Tool
 */


void UUVEditorParameterizeMeshTool::Setup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorParameterizeMeshTool_Setup);
	
	ToolStartTimeAnalytics = FDateTime::UtcNow();

	check(Targets.Num() >= 1);

	UInteractiveTool::Setup();

	// initialize our properties
	Settings = NewObject<UParameterizeMeshToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);
	Settings->WatchProperty(Settings->Method, [&](EParameterizeMeshUVMethod) { OnMethodTypeChanged(); });

	UVAtlasProperties = NewObject<UParameterizeMeshToolUVAtlasProperties>(this);
	UVAtlasProperties->RestoreProperties(this);
	AddToolPropertySource(UVAtlasProperties);
	SetToolPropertySourceEnabled(UVAtlasProperties, true);

	XAtlasProperties = NewObject<UParameterizeMeshToolXAtlasProperties>(this);
	XAtlasProperties->RestoreProperties(this);
	AddToolPropertySource(XAtlasProperties);
	SetToolPropertySourceEnabled(XAtlasProperties, true);
	
	PatchBuilderProperties = NewObject<UParameterizeMeshToolPatchBuilderProperties>(this);
	PatchBuilderProperties->RestoreProperties(this);
	AddToolPropertySource(PatchBuilderProperties);
	SetToolPropertySourceEnabled(PatchBuilderProperties, true);

	if (Targets.Num() == 1)
	{
		PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
		PolygroupLayerProperties->RestoreProperties(this, TEXT("UVEditorRecomputeUVsTool"));
		PolygroupLayerProperties->InitializeGroupLayers(Targets[0]->AppliedCanonical.Get());
		AddToolPropertySource(PolygroupLayerProperties);
		PatchBuilderProperties->bPolygroupsEnabled = true;
		UVAtlasProperties->bPolygroupsEnabled = true;
		SetToolPropertySourceEnabled(PolygroupLayerProperties, (Settings->Method == EParameterizeMeshUVMethod::PatchBuilder && 
		                                                    PatchBuilderProperties->bUsePolygroups)
														   || (Settings->Method == EParameterizeMeshUVMethod::UVAtlas &&
															   UVAtlasProperties->bUsePolygroups));
	}
	else
	{
		PatchBuilderProperties->bPolygroupsEnabled = false;
		UVAtlasProperties->bPolygroupsEnabled = false;
	}

	PatchBuilderProperties->bUDIMsEnabled = (FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport.GetValueOnGameThread() > 0);
	UVAtlasProperties->bUDIMsEnabled = (FUVEditorUXSettings::CVarEnablePrototypeUDIMSupport.GetValueOnGameThread() > 0);


	Factories.SetNum(Targets.Num());
	for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
	{
		TObjectPtr<UUVEditorToolMeshInput> Target = Targets[TargetIndex];
		Factories[TargetIndex] = NewObject<UParameterizeMeshOperatorFactory>();
		Factories[TargetIndex]->TargetTransform = Target->AppliedPreview->PreviewMesh->GetTransform();
		Factories[TargetIndex]->Settings = Settings;
		Factories[TargetIndex]->GetPolygroups = [this]() -> TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> {
			if (Targets.Num() == 1)
			{
				if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
				{
					return MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(Targets[0]->AppliedCanonical.Get());
				}
				else
				{
					FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
					FDynamicMeshPolygroupAttribute* FoundAttrib = UE::Geometry::FindPolygroupLayerByName(*Targets[0]->AppliedCanonical, SelectedName);
					ensureMsgf(FoundAttrib, TEXT("Selected attribute not found! Falling back to Default group layer."));
					return MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(Targets[0]->AppliedCanonical.Get(), FoundAttrib);
				}
			}
			else
			{
				return nullptr;
			}
		};
		Factories[TargetIndex]->UVAtlasProperties = UVAtlasProperties;
		Factories[TargetIndex]->XAtlasProperties = XAtlasProperties;
		Factories[TargetIndex]->PatchBuilderProperties = PatchBuilderProperties;
		Factories[TargetIndex]->OriginalMesh = Target->AppliedCanonical;
		Factories[TargetIndex]->GetSelectedUVChannel = [Target]() { return Target->UVLayerIndex; };

		Target->AppliedPreview->ChangeOpFactory(Factories[TargetIndex]);
		Target->AppliedPreview->OnMeshUpdated.AddWeakLambda(this, [Target](UMeshOpPreviewWithBackgroundCompute* Preview) {
			Target->UpdateUnwrapPreviewFromAppliedPreview();
			});

		Target->AppliedPreview->InvalidateResult();
	}

	SetToolDisplayName(LOCTEXT("ToolNameGlobal", "AutoUV"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool_Global", "Automatically partition the selected Mesh into UV islands, flatten, and pack into a single UV chart"),
		EToolMessageLevel::UserNotification);

	// Analytics
	InputTargetAnalytics = UVEditorAnalytics::CollectTargetAnalytics(Targets);
}

void UUVEditorParameterizeMeshTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorParameterizeMeshTool_OnPropertyModified);

	SetToolPropertySourceEnabled(PolygroupLayerProperties, (Settings->Method == EParameterizeMeshUVMethod::PatchBuilder && 
		                                                    PatchBuilderProperties->bUsePolygroups)
														   || (Settings->Method == EParameterizeMeshUVMethod::UVAtlas &&
															   UVAtlasProperties->bUsePolygroups));

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->InvalidateResult();
	}
	
}


void UUVEditorParameterizeMeshTool::OnMethodTypeChanged()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorParameterizeMeshTool_OnMethodTypeChanged);
	
	SetToolPropertySourceEnabled(UVAtlasProperties, Settings->Method == EParameterizeMeshUVMethod::UVAtlas);
	SetToolPropertySourceEnabled(XAtlasProperties, Settings->Method == EParameterizeMeshUVMethod::XAtlas);
	SetToolPropertySourceEnabled(PatchBuilderProperties, Settings->Method == EParameterizeMeshUVMethod::PatchBuilder);

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->InvalidateResult();
	}
}


void UUVEditorParameterizeMeshTool::Shutdown(EToolShutdownType ShutdownType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVEditorParameterizeMeshTool_Shutdown);

	Settings->SaveProperties(this);
	UVAtlasProperties->SaveProperties(this);
	XAtlasProperties->SaveProperties(this);
	PatchBuilderProperties->SaveProperties(this);

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->OnMeshUpdated.RemoveAll(this);
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		UUVToolEmitChangeAPI* ChangeAPI = GetToolManager()->GetContextObjectStore()->FindContext<UUVToolEmitChangeAPI>();
		const FText TransactionName(LOCTEXT("ParameterizeMeshTransactionName", "Auto UV Tool"));
		ChangeAPI->BeginUndoTransaction(TransactionName);

		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Set things up for undo. 
			FDynamicMeshChangeTracker ChangeTracker(Target->UnwrapCanonical.Get());
			ChangeTracker.BeginChange();

			for (int32 Tid : Target->UnwrapCanonical->TriangleIndicesItr())
			{
				ChangeTracker.SaveTriangle(Tid, true);
			}

			Target->UpdateCanonicalFromPreviews();

			ChangeAPI->EmitToolIndependentUnwrapCanonicalChange(Target, ChangeTracker.EndChange(), LOCTEXT("ApplyParameterizeMeshTool", "Auto UV Tool"));
		}

		ChangeAPI->EndUndoTransaction();

		// Analytics
		RecordAnalytics();
	}
	else
	{
		// Reset the inputs
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			Target->UpdatePreviewsFromCanonical();
		}
	}

	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->ClearOpFactory();
		Target->AppliedPreview->OverrideMaterial = nullptr;
	}

	for (int32 FactoryIndex = 0; FactoryIndex < Factories.Num(); ++FactoryIndex)
	{
		Factories[FactoryIndex] = nullptr;
	}

	Settings = nullptr;
	Targets.Empty();
}

void UUVEditorParameterizeMeshTool::OnTick(float DeltaTime)
{
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		Target->AppliedPreview->Tick(DeltaTime);
	}
}

bool UUVEditorParameterizeMeshTool::CanAccept() const
{
	for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
	{
		if (!Target->AppliedPreview->HaveValidResult())
		{
			return false;
		}
	}
	return true;
}

void UUVEditorParameterizeMeshTool::RecordAnalytics()
{
	using namespace UVEditorAnalytics;
	
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));
	
	// Tool inputs
	InputTargetAnalytics.AppendToAttributes(Attributes, "Input");

	// Tool outputs
	const FTargetAnalytics OutputTargetAnalytics = CollectTargetAnalytics(Targets);
	OutputTargetAnalytics.AppendToAttributes(Attributes, "Output");

	// Tool stats
	if (CanAccept())
	{
		TArray<double> PerAssetValidResultComputeTimes;
		for (const TObjectPtr<UUVEditorToolMeshInput>& Target : Targets)
		{
			// Note: This would log -1 if the result was invalid, but checking CanAccept above ensures results are valid
			PerAssetValidResultComputeTimes.Add(Target->AppliedPreview->GetValidResultComputeTime());
		}
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.PerAsset.ComputeTimeSeconds"), PerAssetValidResultComputeTimes));
	}
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Stats.ToolActiveDuration"), (FDateTime::UtcNow() - ToolStartTimeAnalytics).ToString()));

	// Tool settings chosen by the user (Volatile! Sync with EditCondition meta-tags in *Properties members)
	const FString MethodName = StaticEnum<EParameterizeMeshUVMethod>()->GetNameStringByIndex(static_cast<int>(Settings->Method));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Method"), MethodName));
	switch (Settings->Method)
	{
		case EParameterizeMeshUVMethod::PatchBuilder:
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.InitialPatches"), *MethodName), PatchBuilderProperties->InitialPatches));
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.CurvatureAlignment"), *MethodName), PatchBuilderProperties->CurvatureAlignment));
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.MergingDistortionThreshold"), *MethodName), PatchBuilderProperties->MergingDistortionThreshold));
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.MergingAngleThreshold"), *MethodName), PatchBuilderProperties->MergingAngleThreshold));
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.SmoothingSteps"), *MethodName), PatchBuilderProperties->SmoothingSteps));
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.SmoothingAlpha"), *MethodName), PatchBuilderProperties->SmoothingAlpha));
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.Repack"), *MethodName), PatchBuilderProperties->bRepack));
			if (PatchBuilderProperties->bRepack)
			{
				Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.TextureResolution"), *MethodName), PatchBuilderProperties->TextureResolution));
			}
			break;
		case EParameterizeMeshUVMethod::UVAtlas:
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.IslandStretch"), *MethodName), UVAtlasProperties->IslandStretch));
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.NumIslands"), *MethodName), UVAtlasProperties->NumIslands));
			break;
		case EParameterizeMeshUVMethod::XAtlas:
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.%s.MaxIterations"), *MethodName), XAtlasProperties->MaxIterations));
			break;
		default:
			break;
	}
	
	FEngineAnalytics::GetProvider().RecordEvent(UVEditorAnalyticsEventName(TEXT("AutoUVTool")), Attributes);

	if constexpr (false)
	{
		for (const FAnalyticsEventAttribute& Attr : Attributes)
		{
			UE_LOGF(LogGeometry, Log, "Debug %ls.%ls = %ls", *UVEditorAnalyticsEventName(TEXT("AutoUVTool")), *Attr.GetName(), *Attr.GetValue());
		}
	}
}


#undef LOCTEXT_NAMESPACE

