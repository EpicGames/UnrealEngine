// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVEditorToolsEditorOnlyModule.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#include "DetailsCustomizations/UVTransformToolCustomizations.h"
#include "DetailsCustomizations/UVEditorSeamToolCustomizations.h"

#include "UVEditorTransformTool.h"
#include "Operators/UVEditorUVTransformOp.h"
#include "UVEditorSeamTool.h"

#define LOCTEXT_NAMESPACE "FUVEditorToolsEditorOnlyModule"

void FUVEditorToolsEditorOnlyModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FUVEditorToolsEditorOnlyModule::OnPostEngineInit);

}

void FUVEditorToolsEditorOnlyModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.


	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	// Unregister customizations
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (const FName& ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}
		for (const FName& PropertyName : PropertiesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(PropertyName);
		}
	}
}

void FUVEditorToolsEditorOnlyModule::OnPostEngineInit()
{

	// Register details view customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	// Transform
	PropertyModule.RegisterCustomClassLayout(UUVEditorUVTransformProperties::StaticClass()->GetFName(),
		                                     FOnGetDetailCustomizationInstance::CreateStatic(&FUVEditorUVTransformToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UUVEditorUVTransformProperties::StaticClass()->GetFName());

	// Quick Transform
	PropertyModule.RegisterCustomClassLayout(UUVEditorUVQuickTransformProperties::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FUVEditorUVQuickTransformToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UUVEditorUVQuickTransformProperties::StaticClass()->GetFName());

	// Align
	PropertyModule.RegisterCustomClassLayout(UUVEditorUVAlignProperties::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FUVEditorUVAlignToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UUVEditorUVAlignProperties::StaticClass()->GetFName());

	// Distribute
	PropertyModule.RegisterCustomClassLayout(UUVEditorUVDistributeProperties::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FUVEditorUVDistributeToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UUVEditorUVDistributeProperties::StaticClass()->GetFName());

	// Seam Tool
	PropertyModule.RegisterCustomClassLayout(UUVEditorSeamToolProperties::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FUVEditorSeamToolPropertiesDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UUVEditorSeamToolProperties::StaticClass()->GetFName());
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FUVEditorToolsEditorOnlyModule, UVEditorToolsEditorOnly)
