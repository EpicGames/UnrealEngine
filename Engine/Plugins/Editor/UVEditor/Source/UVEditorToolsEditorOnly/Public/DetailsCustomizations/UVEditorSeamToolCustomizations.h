// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UVEditorSeamTool.h"

class IPropertyHandle;
class FPropertyRestriction;
class IUVEditorTransformToolQuickAction;

// Customization for UUVEditorSeamToolProperties
class FUVEditorSeamToolPropertiesDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	EUVEditorSeamMode GetCurrentMode(TSharedRef<IPropertyHandle> PropertyHandle) const;
	void OnCurrentModeChanged(EUVEditorSeamMode Mode, TSharedRef<IPropertyHandle> PropertyHandle);
};