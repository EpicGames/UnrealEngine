// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class IPropertyHandle;
class FPropertyRestriction;
class IUVEditorTransformToolQuickAction;

// Customization for UUVEditorUVTransformProperties
class FUVEditorUVTransformToolDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	void BuildQuickTranslateMenu(IDetailLayoutBuilder& DetailBuilder);
	void BuildQuickRotateMenu(IDetailLayoutBuilder& DetailBuilder);
};

// Customization for UUVEditorUVQuickTransformProperties
class FUVEditorUVQuickTransformToolDetails : public FUVEditorUVTransformToolDetails
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

// Customization for UUVEditorUVDistributeProperties
class FUVEditorUVDistributeToolDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:

	TSharedPtr<IPropertyHandle> DistributeModeHandle;
	TSharedPtr<IPropertyHandle> EnableManualDistancesHandle;

	TSharedPtr<IPropertyHandle> OrigManualExtentHandle;
	TSharedPtr<IPropertyHandle> OrigManualSpacingHandle;

	void BuildDistributeModeButtons(IDetailLayoutBuilder& DetailBuilder);

};

// Customization for UUVEditorUVAlignProperties
class FUVEditorUVAlignToolDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:

	TSharedPtr<IPropertyHandle> AlignDirectionHandle;

	void BuildAlignModeButtons(IDetailLayoutBuilder& DetailBuilder);

};