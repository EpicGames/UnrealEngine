// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "MeshOpPreviewHelpers.h"
#include "Properties/MeshMaterialProperties.h"
#include "Properties/MeshUVChannelProperties.h"
#include "UVEditorToolAnalyticsUtils.h"
#include "Polygroups/PolygroupSet.h"

#include "UVEditorParameterizeMeshTool.generated.h"

#define UE_API UVEDITORTOOLSEDITORONLY_API

// predeclarations
class UDynamicMeshComponent;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class UUVEditorToolMeshInput;
class UParameterizeMeshToolProperties;
class UParameterizeMeshToolUVAtlasProperties;
class UParameterizeMeshToolXAtlasProperties;
class UParameterizeMeshToolPatchBuilderProperties;
class UParameterizeMeshOperatorFactory;
class UPolygroupLayersProperties;

UCLASS(MinimalAPI)
class UUVEditorParameterizeMeshToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	// This is a pointer so that it can be updated under the builder without
	// having to set it in the mode after initializing targets.
	const TArray<TObjectPtr<UUVEditorToolMeshInput>>* Targets = nullptr;
};


/**
 * UParameterizeMeshTool automatically decomposes the input mesh into charts, solves for UVs,
 * and then packs the resulting charts
 */
UCLASS(MinimalAPI)
class UUVEditorParameterizeMeshTool : public UInteractiveTool
{
	GENERATED_BODY()

public:
	/**
     * The tool will operate on the meshes given here.
     */
	virtual void SetTargets(const TArray<TObjectPtr<UUVEditorToolMeshInput>>& TargetsIn)
	{
		Targets = TargetsIn;
	}


	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

protected:
	UPROPERTY()
	TArray<TObjectPtr<UUVEditorToolMeshInput>> Targets;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolUVAtlasProperties> UVAtlasProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolXAtlasProperties> XAtlasProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolPatchBuilderProperties> PatchBuilderProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UParameterizeMeshOperatorFactory>> Factories;

	UE_API void OnMethodTypeChanged();

	//
	// Analytics
	//
	
	UE::Geometry::UVEditorAnalytics::FTargetAnalytics InputTargetAnalytics;
	FDateTime ToolStartTimeAnalytics;
	UE_API void RecordAnalytics();
};

#undef UE_API
