// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UVEditorToolsEditorOnly : ModuleRules
{
	public UVEditorToolsEditorOnly(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				
				"InteractiveToolsFramework",
				
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				// NOTE: UVEditorTools is a separate module so that it doesn't rely on the editor.
				// So, do not add editor dependencies here.
				
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"UnrealEd",

				"MeshModelingTools",
				"MeshModelingToolsExp",
				"DynamicMesh",
				"GeometryCore",
				"ModelingComponents",
				"ModelingOperators",
				"ModelingOperatorsEditorOnly",
				"MeshModelingToolsEditorOnly",
				"UVEditorTools",

				"ToolWidgets",
				"EditorWidgets",
				"ModelingEditorUI",
				"PropertyEditor",

				"MeshDescription",
				"StaticMeshDescription",
				"MeshConversion",
				"GeometryAlgorithms", // required for constrained Delaunay triangulation
				"MeshUtilitiesCommon", // required by uvlayoutop
				// ... add private dependencies that you statically link with here ...	
			}
			);
		bool bWithProxyLOD = Target.Platform == UnrealTargetPlatform.Win64;
		PrivateDefinitions.Add("WITH_PROXYLOD=" + (bWithProxyLOD ? '1' : '0'));
		if (bWithProxyLOD)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ProxyLODMeshReduction", // currently Win64-only
				}
				);
		}

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
