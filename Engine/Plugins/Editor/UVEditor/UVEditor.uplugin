{
	"FileVersion": 3,
	"Version": 2,
	"VersionName": "0.2",
	"FriendlyName": "UVEditor",
	"Description": "Asset editor for modifying the UV mapping of a mesh",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "UVEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "UVEditorTools",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "UVEditorToolsEditorOnly",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		}
	]
}